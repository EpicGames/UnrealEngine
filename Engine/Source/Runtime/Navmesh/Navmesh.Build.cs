// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Navmesh : ModuleRules
	{
		public Navmesh(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(new string[] { "Core" });

			// This is an unsupported feature and has not been finished to production quality.
			if (Target.bCompileNavmeshSegmentLinks)
			{
				PublicDefinitions.Add("WITH_NAVMESH_SEGMENT_LINKS=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_NAVMESH_SEGMENT_LINKS=0");
			}

			if (Target.bCompileNavmeshClusterLinks)
			{
				PublicDefinitions.Add("WITH_NAVMESH_CLUSTER_LINKS=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_NAVMESH_CLUSTER_LINKS=0");
			}
		}
	}
}
