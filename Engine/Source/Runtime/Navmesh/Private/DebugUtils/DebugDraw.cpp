// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DebugUtils/DebugDraw.h"
#include "Recast/RecastAlloc.h" // UE
#define _USE_MATH_DEFINES


duDebugDraw::~duDebugDraw()
{
	// Empty
}
	

inline int bit(int a, int b)
{
	return (a & (1 << b)) >> b;
}

unsigned int duIntToCol(int i, int a)
{
	int	r = bit(i, 1) + bit(i, 3) * 2 + 1;
	int	g = bit(i, 2) + bit(i, 4) * 2 + 1;
	int	b = bit(i, 0) + bit(i, 5) * 2 + 1;
	return duRGBA(r*63,g*63,b*63,a);
}

void duIntToCol(int i, float* col)
{
	int	r = bit(i, 0) + bit(i, 3) * 2 + 1;
	int	g = bit(i, 1) + bit(i, 4) * 2 + 1;
	int	b = bit(i, 2) + bit(i, 5) * 2 + 1;
	col[0] = 1 - r*63.0f/255.0f;
	col[1] = 1 - g*63.0f/255.0f;
	col[2] = 1 - b*63.0f/255.0f;
}

//@UE BEGIN
// Color palette
unsigned int duColor::red			= duRGBA(0xe53935ff);		
unsigned int duColor::pink			= duRGBA(0xd81b60ff);
unsigned int duColor::purple		= duRGBA(0x8e24aaff);
unsigned int duColor::deepPurple	= duRGBA(0x5e35b1ff);
unsigned int duColor::indigo		= duRGBA(0x3949abff);
unsigned int duColor::blue			= duRGBA(0x1e88e5ff);
unsigned int duColor::lightBlue		= duRGBA(0x039be5ff);
unsigned int duColor::cyan			= duRGBA(0x00acc1ff);
unsigned int duColor::teal			= duRGBA(0x00897bff);
unsigned int duColor::green			= duRGBA(0x43a047ff);
unsigned int duColor::lightGreen	= duRGBA(0x7cb342ff);
unsigned int duColor::lime			= duRGBA(0xc0ca33ff);
unsigned int duColor::yellow		= duRGBA(0xfdd835ff);
unsigned int duColor::amber			= duRGBA(0xffb300ff);
unsigned int duColor::orange		= duRGBA(0xfb8c00ff);
unsigned int duColor::orangeRed		= duRGBA(0xf4511eff);
unsigned int duColor::brown			= duRGBA(0x6d4c41ff);

unsigned int duColor::white			= duRGBA(0xffffffff);
unsigned int duColor::lightGrey		= duRGBA(0xe0e0e0ff);
unsigned int duColor::grey			= duRGBA(0x9e9e9eff);
unsigned int duColor::darkGrey		= duRGBA(0x616161ff);
unsigned int duColor::black			= duRGBA(0x000000ff);
//@UE END

void duCalcBoxColors(unsigned int* colors, unsigned int colTop, unsigned int colSide)
{
	if (!colors) return;
	
	colors[0] = duMultCol(colTop, 250);
	colors[1] = duMultCol(colSide, 140);
	colors[2] = duMultCol(colSide, 165);
	colors[3] = duMultCol(colSide, 217);
	colors[4] = duMultCol(colSide, 165);
	colors[5] = duMultCol(colSide, 217);
}

void duDebugDrawCylinderWire(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
							 duReal maxx, duReal maxy, duReal maxz, unsigned int col, const float lineWidth)
{
	if (!dd) return;
	
	dd->begin(DU_DRAW_LINES, lineWidth);
	duAppendCylinderWire(dd, minx,miny,minz, maxx,maxy,maxz, col);
	dd->end();
}

void duDebugDrawBoxWire(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
						duReal maxx, duReal maxy, duReal maxz, unsigned int col, const float lineWidth)
{
	if (!dd) return;
	
	dd->begin(DU_DRAW_LINES, lineWidth);
	duAppendBoxWire(dd, minx,miny,minz, maxx,maxy,maxz, col);
	dd->end();
}

void duDebugDrawArc(struct duDebugDraw* dd, const duReal x0, const duReal y0, const duReal z0,
					const duReal x1, const duReal y1, const duReal z1, const duReal h,
					const duReal as0, const duReal as1, unsigned int col, const float lineWidth)
{
	if (!dd) return;
	
	dd->begin(DU_DRAW_LINES, lineWidth);
	duAppendArc(dd, x0,y0,z0, x1,y1,z1, h, as0, as1, col);
	dd->end();
}

void duDebugDrawArrow(struct duDebugDraw* dd, const duReal x0, const duReal y0, const duReal z0,
					  const duReal x1, const duReal y1, const duReal z1,
					  const duReal as0, const duReal as1, unsigned int col, const float lineWidth)
{
	if (!dd) return;
	
	dd->begin(DU_DRAW_LINES, lineWidth);
	duAppendArrow(dd, x0,y0,z0, x1,y1,z1, as0, as1, col);
	dd->end();
}

void duDebugDrawCircle(struct duDebugDraw* dd, const duReal x, const duReal y, const duReal z,
					   const duReal r, unsigned int col, const float lineWidth)
{
	if (!dd) return;
	
	dd->begin(DU_DRAW_LINES, lineWidth);
	duAppendCircle(dd, x,y,z, r, col);
	dd->end();
}

void duDebugDrawCross(struct duDebugDraw* dd, const duReal x, const duReal y, const duReal z,
					  const duReal size, unsigned int col, const float lineWidth)
{
	if (!dd) return;
	
	dd->begin(DU_DRAW_LINES, lineWidth);
	duAppendCross(dd, x,y,z, size, col);
	dd->end();
}

void duDebugDrawBox(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
					duReal maxx, duReal maxy, duReal maxz, const unsigned int* fcol)
{
	if (!dd) return;
	
	dd->begin(DU_DRAW_QUADS);
	duAppendBox(dd, minx,miny,minz, maxx,maxy,maxz, fcol);
	dd->end();
}

void duDebugDrawCylinder(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
						 duReal maxx, duReal maxy, duReal maxz, unsigned int col)
{
	if (!dd) return;
	
	dd->begin(DU_DRAW_TRIS);
	duAppendCylinder(dd, minx,miny,minz, maxx,maxy,maxz, col);
	dd->end();
}

void duDebugDrawGridXZ(struct duDebugDraw* dd, const duReal ox, const duReal oy, const duReal oz,
					   const int w, const int h, const duReal size,
					   const unsigned int col, const float lineWidth)
{
	if (!dd) return;

	dd->begin(DU_DRAW_LINES, lineWidth);
	for (int i = 0; i <= h; ++i)
	{
		dd->vertex(ox,oy,oz+i*size, col);
		dd->vertex(ox+w*size,oy,oz+i*size, col);
	}
	for (int i = 0; i <= w; ++i)
	{
		dd->vertex(ox+i*size,oy,oz, col);
		dd->vertex(ox+i*size,oy,oz+h*size, col);
	}
	dd->end();
}
		 

void duAppendCylinderWire(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
						  duReal maxx, duReal maxy, duReal maxz, unsigned int col)
{
	if (!dd) return;

	static const int NUM_SEG = 16;
	static duReal dir[NUM_SEG*2];
	static bool init = false;
	if (!init)
	{
		init = true;
		for (int i = 0; i < NUM_SEG; ++i)
		{
			const duReal a = (duReal)i/(duReal)NUM_SEG*DU_PI*2;
			dir[i*2] = duCos(a);
			dir[i*2+1] = duSin(a);
		}
	}
	
	const duReal cx = (maxx + minx)/2;
	const duReal cz = (maxz + minz)/2;
	const duReal rx = (maxx - minx)/2;
	const duReal rz = (maxz - minz)/2;
	
	for (int i = 0, j = NUM_SEG-1; i < NUM_SEG; j = i++)
	{
		dd->vertex(cx+dir[j*2+0]*rx, miny, cz+dir[j*2+1]*rz, col);
		dd->vertex(cx+dir[i*2+0]*rx, miny, cz+dir[i*2+1]*rz, col);
		dd->vertex(cx+dir[j*2+0]*rx, maxy, cz+dir[j*2+1]*rz, col);
		dd->vertex(cx+dir[i*2+0]*rx, maxy, cz+dir[i*2+1]*rz, col);
	}
	for (int i = 0; i < NUM_SEG; i += NUM_SEG/4)
	{
		dd->vertex(cx+dir[i*2+0]*rx, miny, cz+dir[i*2+1]*rz, col);
		dd->vertex(cx+dir[i*2+0]*rx, maxy, cz+dir[i*2+1]*rz, col);
	}
}

void duAppendBoxWire(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
					 duReal maxx, duReal maxy, duReal maxz, unsigned int col)
{
	if (!dd) return;
	// Top
	dd->vertex(minx, miny, minz, col);
	dd->vertex(maxx, miny, minz, col);
	dd->vertex(maxx, miny, minz, col);
	dd->vertex(maxx, miny, maxz, col);
	dd->vertex(maxx, miny, maxz, col);
	dd->vertex(minx, miny, maxz, col);
	dd->vertex(minx, miny, maxz, col);
	dd->vertex(minx, miny, minz, col);
	
	// bottom
	dd->vertex(minx, maxy, minz, col);
	dd->vertex(maxx, maxy, minz, col);
	dd->vertex(maxx, maxy, minz, col);
	dd->vertex(maxx, maxy, maxz, col);
	dd->vertex(maxx, maxy, maxz, col);
	dd->vertex(minx, maxy, maxz, col);
	dd->vertex(minx, maxy, maxz, col);
	dd->vertex(minx, maxy, minz, col);
	
	// Sides
	dd->vertex(minx, miny, minz, col);
	dd->vertex(minx, maxy, minz, col);
	dd->vertex(maxx, miny, minz, col);
	dd->vertex(maxx, maxy, minz, col);
	dd->vertex(maxx, miny, maxz, col);
	dd->vertex(maxx, maxy, maxz, col);
	dd->vertex(minx, miny, maxz, col);
	dd->vertex(minx, maxy, maxz, col);
}

void duAppendBoxPoints(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
					   duReal maxx, duReal maxy, duReal maxz, unsigned int col)
{
	if (!dd) return;
	// Top
	dd->vertex(minx, miny, minz, col);
	dd->vertex(maxx, miny, minz, col);
	dd->vertex(maxx, miny, minz, col);
	dd->vertex(maxx, miny, maxz, col);
	dd->vertex(maxx, miny, maxz, col);
	dd->vertex(minx, miny, maxz, col);
	dd->vertex(minx, miny, maxz, col);
	dd->vertex(minx, miny, minz, col);
	
	// bottom
	dd->vertex(minx, maxy, minz, col);
	dd->vertex(maxx, maxy, minz, col);
	dd->vertex(maxx, maxy, minz, col);
	dd->vertex(maxx, maxy, maxz, col);
	dd->vertex(maxx, maxy, maxz, col);
	dd->vertex(minx, maxy, maxz, col);
	dd->vertex(minx, maxy, maxz, col);
	dd->vertex(minx, maxy, minz, col);
}

void duAppendBox(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
				 duReal maxx, duReal maxy, duReal maxz, const unsigned int* fcol)
{
	if (!dd) return;
	const duReal verts[8*3] =
	{
		minx, miny, minz,
		maxx, miny, minz,
		maxx, miny, maxz,
		minx, miny, maxz,
		minx, maxy, minz,
		maxx, maxy, minz,
		maxx, maxy, maxz,
		minx, maxy, maxz,
	};
	static const unsigned char inds[6*4] =
	{
		7, 6, 5, 4,
		0, 1, 2, 3,
		1, 5, 6, 2,
		3, 7, 4, 0,
		2, 6, 7, 3,
		0, 4, 5, 1,
	};
	
	const unsigned char* in = inds;
	for (int i = 0; i < 6; ++i)
	{
		dd->vertex(&verts[*in*3], fcol[i]); in++;
		dd->vertex(&verts[*in*3], fcol[i]); in++;
		dd->vertex(&verts[*in*3], fcol[i]); in++;
		dd->vertex(&verts[*in*3], fcol[i]); in++;
	}
}

void duAppendCylinder(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
					  duReal maxx, duReal maxy, duReal maxz, unsigned int col)
{
	if (!dd) return;
	
	static const int NUM_SEG = 16;
	static duReal dir[NUM_SEG*2];
	static bool init = false;
	if (!init)
	{
		init = true;
		for (int i = 0; i < NUM_SEG; ++i)
		{
			const duReal a = (duReal)i/(duReal)NUM_SEG*DU_PI*2;
			dir[i*2] = duCos(a);
			dir[i*2+1] = duSin(a);
		}
	}
	
	unsigned int col2 = duMultCol(col, 160);
	
	const duReal cx = (maxx + minx)/2;
	const duReal cz = (maxz + minz)/2;
	const duReal rx = (maxx - minx)/2;
	const duReal rz = (maxz - minz)/2;

	for (int i = 2; i < NUM_SEG; ++i)
	{
		const int a = 0, b = i-1, c = i;
		dd->vertex(cx+dir[a*2+0]*rx, miny, cz+dir[a*2+1]*rz, col2);
		dd->vertex(cx+dir[b*2+0]*rx, miny, cz+dir[b*2+1]*rz, col2);
		dd->vertex(cx+dir[c*2+0]*rx, miny, cz+dir[c*2+1]*rz, col2);
	}
	for (int i = 2; i < NUM_SEG; ++i)
	{
		const int a = 0, b = i, c = i-1;
		dd->vertex(cx+dir[a*2+0]*rx, maxy, cz+dir[a*2+1]*rz, col);
		dd->vertex(cx+dir[b*2+0]*rx, maxy, cz+dir[b*2+1]*rz, col);
		dd->vertex(cx+dir[c*2+0]*rx, maxy, cz+dir[c*2+1]*rz, col);
	}
	for (int i = 0, j = NUM_SEG-1; i < NUM_SEG; j = i++)
	{
		dd->vertex(cx+dir[i*2+0]*rx, miny, cz+dir[i*2+1]*rz, col2);
		dd->vertex(cx+dir[j*2+0]*rx, miny, cz+dir[j*2+1]*rz, col2);
		dd->vertex(cx+dir[j*2+0]*rx, maxy, cz+dir[j*2+1]*rz, col);

		dd->vertex(cx+dir[i*2+0]*rx, miny, cz+dir[i*2+1]*rz, col2);
		dd->vertex(cx+dir[j*2+0]*rx, maxy, cz+dir[j*2+1]*rz, col);
		dd->vertex(cx+dir[i*2+0]*rx, maxy, cz+dir[i*2+1]*rz, col);
	}
}


inline void evalArc(const duReal x0, const duReal y0, const duReal z0,
					const duReal dx, const duReal dy, const duReal dz,
					const duReal h, const duReal u, duReal* res)
{
	res[0] = x0 + dx * u;
	res[1] = y0 + dy * u + h * (1-(u*2-1)*(u*2-1));
	res[2] = z0 + dz * u;
}


inline void vcross(duReal* dest, const duReal* v1, const duReal* v2)
{
	dest[0] = v1[1]*v2[2] - v1[2]*v2[1];
	dest[1] = v1[2]*v2[0] - v1[0]*v2[2];
	dest[2] = v1[0]*v2[1] - v1[1]*v2[0]; 
}

inline void vnormalize(duReal* v)
{
	duReal d = 1.0f / duSqrt(v[0]*v[0] + v[1]*v[1] + v[2]*v[2]);
	v[0] *= d;
	v[1] *= d;
	v[2] *= d;
}

inline void vsub(duReal* dest, const duReal* v1, const duReal* v2)
{
	dest[0] = v1[0]-v2[0];
	dest[1] = v1[1]-v2[1];
	dest[2] = v1[2]-v2[2];
}

inline duReal vdistSqr(const duReal* v1, const duReal* v2)
{
	const duReal x = v1[0]-v2[0];
	const duReal y = v1[1]-v2[1];
	const duReal z = v1[2]-v2[2];
	return x*x + y*y + z*z;
}


void appendArrowHead(struct duDebugDraw* dd, const duReal* p, const duReal* q,
					 const duReal s, unsigned int col)
{
	const duReal eps = 0.001f;
	if (!dd) return;
	if (vdistSqr(p,q) < eps*eps) return;
	duReal ax[3], ay[3] = {0,1,0}, az[3];
	vsub(az, q, p);
	vnormalize(az);
	vcross(ax, ay, az);
	vcross(ay, az, ax);
	vnormalize(ay);

	dd->vertex(p, col);
//	dd->vertex(p[0]+az[0]*s+ay[0]*s/2, p[1]+az[1]*s+ay[1]*s/2, p[2]+az[2]*s+ay[2]*s/2, col);
	dd->vertex(p[0]+az[0]*s+ax[0]*s/3, p[1]+az[1]*s+ax[1]*s/3, p[2]+az[2]*s+ax[2]*s/3, col);

	dd->vertex(p, col);
//	dd->vertex(p[0]+az[0]*s-ay[0]*s/2, p[1]+az[1]*s-ay[1]*s/2, p[2]+az[2]*s-ay[2]*s/2, col);
	dd->vertex(p[0]+az[0]*s-ax[0]*s/3, p[1]+az[1]*s-ax[1]*s/3, p[2]+az[2]*s-ax[2]*s/3, col);
	
}

void duAppendArc(struct duDebugDraw* dd, const duReal x0, const duReal y0, const duReal z0,
				 const duReal x1, const duReal y1, const duReal z1, const duReal h,
				 const duReal as0, const duReal as1, unsigned int col)
{
	if (!dd) return;
	static const int NUM_ARC_PTS = 8;
	static const duReal PAD = 0.05f;
	static const duReal ARC_PTS_SCALE = (1.0f-PAD*2) / (duReal)NUM_ARC_PTS;
	const duReal dx = x1 - x0;
	const duReal dy = y1 - y0;
	const duReal dz = z1 - z0;
	const duReal len = duSqrt(dx*dx + dy*dy + dz*dz);
	duReal prev[3];
	evalArc(x0,y0,z0, dx,dy,dz, len*h, PAD, prev);
	for (int i = 1; i <= NUM_ARC_PTS; ++i)
	{
		const duReal u = PAD + i * ARC_PTS_SCALE;
		duReal pt[3];
		evalArc(x0,y0,z0, dx,dy,dz, len*h, u, pt);
		dd->vertex(prev[0],prev[1],prev[2], col);
		dd->vertex(pt[0],pt[1],pt[2], col);
		prev[0] = pt[0]; prev[1] = pt[1]; prev[2] = pt[2];
	}
	
	// End arrows
	if (as0 > 0.001f)
	{
		duReal p[3], q[3];
		evalArc(x0,y0,z0, dx,dy,dz, len*h, PAD, p);
		evalArc(x0,y0,z0, dx,dy,dz, len*h, PAD+0.05f, q);
		appendArrowHead(dd, p, q, as0, col);
	}

	if (as1 > 0.001f)
	{
		duReal p[3], q[3];
		evalArc(x0,y0,z0, dx,dy,dz, len*h, 1-PAD, p);
		evalArc(x0,y0,z0, dx,dy,dz, len*h, 1-(PAD+0.05f), q);
		appendArrowHead(dd, p, q, as1, col);
	}
}

void duAppendArcSegment(struct duDebugDraw* dd, const duReal xA0, const duReal yA0, const duReal zA0,
	const duReal xA1, const duReal yA1, const duReal zA1,
	const duReal xB0, const duReal yB0, const duReal zB0,
	const duReal xB1, const duReal yB1, const duReal zB1,
	const duReal h, unsigned int col)
{
	if (!dd) return;
	static const int NUM_ARC_PTS = 8;
	static const duReal PAD = 0.05f;
	static const duReal ARC_PTS_SCALE = (1.0f-PAD*2) / (duReal)NUM_ARC_PTS;
	const duReal dx0 = xB0 - xA0;
	const duReal dy0 = yB0 - yA0;
	const duReal dz0 = zB0 - zA0;
	const duReal dx1 = xB1 - xA1;
	const duReal dy1 = yB1 - yA1;
	const duReal dz1 = zB1 - zA1;
	const duReal len0 = duSqrt(dx0*dx0 + dy0*dy0 + dz0*dz0);
	const duReal len1 = duSqrt(dx1*dx1 + dy1*dy1 + dz1*dz1);
	duReal prev0[3];
	duReal prev1[3];
	evalArc(xA0,yA0,zA0, dx0,dy0,dz0, len0*h, PAD, prev0);
	evalArc(xA1,yA1,zA1, dx1,dy1,dz1, len1*h, PAD, prev1);
	for (int i = 1; i <= NUM_ARC_PTS; ++i)
	{
		const duReal u = PAD + i * ARC_PTS_SCALE;
		duReal pt0[3];
		duReal pt1[3];
		evalArc(xA0,yA0,zA0, dx0,dy0,dz0, len0*h, u, pt0);
		evalArc(xA1,yA1,zA1, dx1,dy1,dz1, len1*h, u, pt1);
		
		dd->vertex(pt0[0],pt0[1],pt0[2], col);
		dd->vertex(pt1[0],pt1[1],pt1[2], col);
		dd->vertex(prev1[0],prev1[1],prev1[2], col);
		dd->vertex(prev0[0],prev0[1],prev0[2], col);

		dd->vertex(prev0[0],prev0[1],prev0[2], col);
		dd->vertex(prev1[0],prev1[1],prev1[2], col);
		dd->vertex(pt1[0],pt1[1],pt1[2], col);
		dd->vertex(pt0[0],pt0[1],pt0[2], col);

		prev0[0] = pt0[0]; prev0[1] = pt0[1]; prev0[2] = pt0[2];
		prev1[0] = pt1[0]; prev1[1] = pt1[1]; prev1[2] = pt1[2];
	}
}

void duAppendArrow(struct duDebugDraw* dd, const duReal x0, const duReal y0, const duReal z0,
	const duReal x1, const duReal y1, const duReal z1,
	const duReal as0, const duReal as1, unsigned int col)
{
	if (!dd) return;

	dd->vertex(x0,y0,z0, col);
	dd->vertex(x1,y1,z1, col);

	// End arrows
	const duReal p[3] = {x0,y0,z0}, q[3] = {x1,y1,z1};
	if (as0 > 0.001f)
		appendArrowHead(dd, p, q, as0, col);
	if (as1 > 0.001f)
		appendArrowHead(dd, q, p, as1, col);
}

void duAppendCircle(struct duDebugDraw* dd, const duReal x, const duReal y, const duReal z,
					const duReal r, unsigned int col)
{
	if (!dd) return;
	static const int NUM_SEG = 40;
	static duReal dir[40*2];
	static bool init = false;
	if (!init)
	{
		init = true;
		for (int i = 0; i < NUM_SEG; ++i)
		{
			const duReal a = (duReal)i/(duReal)NUM_SEG*DU_PI*2;
			dir[i*2] = duCos(a);
			dir[i*2+1] = duSin(a);
		}
	}
	
	for (int i = 0, j = NUM_SEG-1; i < NUM_SEG; j = i++)
	{
		dd->vertex(x+dir[j*2+0]*r, y, z+dir[j*2+1]*r, col);
		dd->vertex(x+dir[i*2+0]*r, y, z+dir[i*2+1]*r, col);
	}
}

void duAppendCross(struct duDebugDraw* dd, const duReal x, const duReal y, const duReal z,
				   const duReal s, unsigned int col)
{
	if (!dd) return;
	dd->vertex(x-s,y,z, col);
	dd->vertex(x+s,y,z, col);
	dd->vertex(x,y-s,z, col);
	dd->vertex(x,y+s,z, col);
	dd->vertex(x,y,z-s, col);
	dd->vertex(x,y,z+s, col);
}

duDisplayList::duDisplayList(int cap) :
	m_pos(0),
	m_color(0),
	m_size(0),
	m_cap(0),
	m_depthMask(true),
	m_prim(DU_DRAW_LINES),
	m_primSize(1.0f)
{
	if (cap < 8)
		cap = 8;
	resize(cap);
}

duDisplayList::~duDisplayList()
{
	rcFree(m_pos); // UE
	rcFree(m_color); // UE
}

void duDisplayList::resize(int cap)
{
	duReal* newPos = (duReal*)rcAlloc(sizeof(duReal)*3*cap, RC_ALLOC_PERM); // UE
	if (m_size)
		memcpy(newPos, m_pos, sizeof(duReal)*3*m_size);
	rcFree(m_pos); // UE
	m_pos = newPos;

	unsigned int* newColor = (unsigned int*)rcAlloc(sizeof(unsigned int)*cap, RC_ALLOC_PERM); // UE
	if (m_size)
		memcpy(newColor, m_color, sizeof(unsigned int)*m_size);
	rcFree(m_color); // UE
	m_color = newColor;
	
	m_cap = cap;
}

void duDisplayList::clear()
{
	m_size = 0;
}

void duDisplayList::depthMask(bool state)
{
	m_depthMask = state;
}

void duDisplayList::begin(duDebugDrawPrimitives prim, float size)
{
	clear();
	m_prim = prim;
	m_primSize = size;
}

void duDisplayList::vertex(const duReal x, const duReal y, const duReal z, unsigned int color)
{
	if (m_size+1 >= m_cap)
		resize(m_cap*2);
	duReal* p = &m_pos[m_size*3];
	p[0] = x;
	p[1] = y;
	p[2] = z;
	m_color[m_size] = color;
	m_size++;
}

void duDisplayList::vertex(const duReal* pos, unsigned int color)
{
	vertex(pos[0],pos[1],pos[2],color);
}

void duDisplayList::end()
{
}

void duDisplayList::draw(struct duDebugDraw* dd)
{
	if (!dd) return;
	if (!m_size) return;
	dd->depthMask(m_depthMask);
	dd->begin(m_prim, m_primSize);
	for (int i = 0; i < m_size; ++i)
		dd->vertex(&m_pos[i*3], m_color[i]);
	dd->end();
}
