// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DebugUtils/DetourDebugDraw.h"
#include "DebugUtils/DebugDraw.h"
#include "Detour/DetourNavMeshQuery.h"
#include "DetourTileCache/DetourTileCacheBuilder.h"
#include "Detour/DetourNode.h"

inline const dtLink& getLinkHelper(const dtMeshTile* tile, unsigned int link)
{
	return (link < (unsigned int)tile->header->maxLinkCount) ? tile->links[link] : tile->dynamicLinks[link - tile->header->maxLinkCount]; //@UE
}

static dtReal distancePtLine2d(const dtReal* pt, const dtReal* p, const dtReal* q)
{
	dtReal pqx = q[0] - p[0];
	dtReal pqz = q[2] - p[2];
	dtReal dx = pt[0] - p[0];
	dtReal dz = pt[2] - p[2];
	dtReal d = pqx*pqx + pqz*pqz;
	dtReal t = pqx*dx + pqz*dz;
	if (d != 0) t /= d;
	dx = p[0] + t*pqx - pt[0];
	dz = p[2] + t*pqz - pt[2];
	return dx*dx + dz*dz;
}

static void drawPolyBoundaries(duDebugDraw* dd, const dtMeshTile* tile,
							   const unsigned int col, const float linew,
							   bool inner)
{
	static const dtReal thr = 0.01f*0.01f;

	dd->begin(DU_DRAW_LINES, linew);

	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		const dtPoly* p = &tile->polys[i];
		
		if (p->getType() != DT_POLYTYPE_GROUND) continue;
		
		const dtPolyDetail* pd = &tile->detailMeshes[i];
		
		for (int j = 0, nj = (int)p->vertCount; j < nj; ++j)
		{
			unsigned int c = col;
			if (inner)
			{
				if (p->neis[j] == 0) continue;
				if (p->neis[j] & DT_EXT_LINK)
				{
					bool con = false;
					unsigned int k = p->firstLink;
					while (k != DT_NULL_LINK)
					{
						const dtLink& testLink = getLinkHelper(tile, k);
						if (testLink.edge == j)
						{
							con = true;
							break;
						}

						k = testLink.next;
					}

					if (con)
						c = duRGBA(255,255,255,48);
					else
						c = duRGBA(0,0,0,48);
				}
				else
					c = duRGBA(0,48,64,32);
			}
			else
			{
				if (p->neis[j] != 0) continue;
			}
			
			dtReal v0[3], v1[3]; //@UE
			tile->verts[p->verts[j]].toWorld(tile->header->bmin, v0); //@UE
			tile->verts[p->verts[(j+1) % nj]].toWorld(tile->header->bmin, v1); //@UE

			// Draw detail mesh edges which align with the actual poly edge.
			// This is really slow.
			for (int k = 0; k < pd->triCount; ++k)
			{
				const unsigned char* t = &tile->detailTris[(pd->triBase+k)*4];
				dtReal tvBuf[3][3]; //@UE
				const dtReal* tv[3];
				for (int m = 0; m < 3; ++m)
				{
					if (t[m] < p->vertCount)
						tile->verts[p->verts[t[m]]].toWorld(tile->header->bmin, tvBuf[m]); //@UE
					else
						tile->detailVerts[pd->vertBase+(t[m]-p->vertCount)].toWorld(tile->header->bmin, tvBuf[m]); //@UE

					tv[m] = tvBuf[m]; //@UE
				}
				for (int m = 0, n = 2; m < 3; n=m++)
				{
					if (((t[3] >> (n*2)) & 0x3) == 0) continue;	// Skip inner detail edges.
					if (distancePtLine2d(tv[n],v0,v1) < thr &&
						distancePtLine2d(tv[m],v0,v1) < thr)
					{
						dd->vertex(tv[n], c);
						dd->vertex(tv[m], c);
					}
				}
			}
		}
	}
	dd->end();
}

static void drawMeshTile(duDebugDraw* dd, const dtNavMesh& mesh, const dtNavMeshQuery* query,
						 const dtMeshTile* tile, unsigned char flags)
{
	dtPolyRef base = mesh.getPolyRefBase(tile);

	int tileNum = mesh.decodePolyIdTile(base);
	
	dd->depthMask(false);

	dd->begin(DU_DRAW_TRIS);
	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		const dtPoly* p = &tile->polys[i];
		if (p->getType() != DT_POLYTYPE_GROUND)	// Skip off-mesh links.
			continue;
			
		const dtPolyDetail* pd = &tile->detailMeshes[i];

		unsigned int col;
		if (query && query->isInClosedList(base | (dtPolyRef)i))
			col = duRGBA(255,196,0,64);
		else
		{
			if (flags & DU_DRAWNAVMESH_COLOR_TILES)
			{
				col = duIntToCol(tileNum, 128);
			}
			else
			{
				if (p->getArea() == 0) // Treat zero area type as default.
					col = duRGBA(0,192,255,64);
				else
					col = duIntToCol(p->getArea(), 64);
			}
		}
		
		for (int j = 0; j < pd->triCount; ++j)
		{
			const unsigned char* t = &tile->detailTris[(pd->triBase+j)*4];
			for (int k = 0; k < 3; ++k)
			{
				dtReal tv[3]; //@UE
				if (t[k] < p->vertCount)
					tile->verts[p->verts[t[k]]].toWorld(tile->header->bmin, tv); //@UE
				else
					tile->detailVerts[pd->vertBase+t[k]-p->vertCount].toWorld(tile->header->bmin, tv); //@UE

				dd->vertex(tv, col); //@UE
			}
		}
	}
	dd->end();
	
	// Draw inter poly boundaries
	drawPolyBoundaries(dd, tile, duRGBA(0,48,64,32), 1.5f, true);
	
	// Draw outer poly boundaries
	drawPolyBoundaries(dd, tile, duRGBA(0,48,64,220), 2.5f, false);

	if (flags & DU_DRAWNAVMESH_OFFMESHCONS)
	{
		dd->begin(DU_DRAW_LINES, 2.0f);
		for (int i = 0; i < tile->header->polyCount; ++i)
		{
			const dtPoly* p = &tile->polys[i];

			// draw offmesh connection - point type
			if (p->getType() != DT_POLYTYPE_OFFMESH_POINT)
				continue;

			unsigned int col, col2;
			if (query && query->isInClosedList(base | (dtPolyRef)i))
				col = duRGBA(255,196,0,220);
			else
				col = duDarkenCol(duIntToCol(p->getArea(), 220));

			const dtOffMeshConnection* con = &tile->offMeshCons[i - tile->header->offMeshBase];
			dtReal va[3], vb[3]; //@UE
			tile->verts[p->verts[0]].toWorld(tile->header->bmin, va); //@UE
			tile->verts[p->verts[1]].toWorld(tile->header->bmin, vb); //@UE

			// Check to see if start and end end-points have links.
			bool startSet = false;
			bool endSet = false;
			unsigned int k = p->firstLink;
			while (k != DT_NULL_LINK)
			{
				const dtLink& testLink = getLinkHelper(tile, k);
				if (testLink.edge == 0)
					startSet = true;
				if (testLink.edge == 1)
					endSet = true;

				k = testLink.next;
			}

			// End points and their on-mesh locations.
			dd->vertex(va[0],va[1],va[2], col);
			dd->vertex(con->pos[0],con->pos[1],con->pos[2], col);
			col2 = startSet ? col : duRGBA(220,32,16,196);
			duAppendCircle(dd, con->pos[0],con->pos[1]+0.1f,con->pos[2], con->rad, col2);

			dd->vertex(vb[0],vb[1],vb[2], col);
			dd->vertex(con->pos[3],con->pos[4],con->pos[5], col);
			col2 = endSet ? col : duRGBA(220,32,16,196);
			duAppendCircle(dd, con->pos[3],con->pos[4]+0.1f,con->pos[5], con->rad, col2);

			// End point vertices.
			dd->vertex(con->pos[0],con->pos[1],con->pos[2], duRGBA(0,48,64,196));
			dd->vertex(con->pos[0],con->pos[1]+0.2f,con->pos[2], duRGBA(0,48,64,196));

			dd->vertex(con->pos[3],con->pos[4],con->pos[5], duRGBA(0,48,64,196));
			dd->vertex(con->pos[3],con->pos[4]+0.2f,con->pos[5], duRGBA(0,48,64,196));

			// Connection arc.
			duAppendArc(dd, con->pos[0],con->pos[1],con->pos[2], con->pos[3],con->pos[4],con->pos[5], 0.25f,
				con->getBiDirectional() ? 0.6f : 0, 0.6f, col);
		}
		dd->end();

//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
		dd->begin(DU_DRAW_LINES, 4.0f);
		for (int i = 0; i < tile->header->polyCount; ++i)
		{
			const dtPoly* p = &tile->polys[i];
			if (p->getType() == DT_POLYTYPE_OFFMESH_SEGMENT)
			{
				dtReal vA0[3], vA1[3], vB0[3], vB1[3]; //@UE
				tile->verts[p->verts[0]].toWorld(tile->header->bmin, vA0); //@UE
				tile->verts[p->verts[1]].toWorld(tile->header->bmin, vA1); //@UE
				tile->verts[p->verts[2]].toWorld(tile->header->bmin, vB0); //@UE
				tile->verts[p->verts[3]].toWorld(tile->header->bmin, vB1); //@UE

				unsigned int col = duDarkenCol(duIntToCol(p->getArea(), 220));
				duAppendArc(dd, vA0[0], vA0[1], vA0[2], vB0[0], vB0[1], vB0[2], 0.25f, 0.0f, 0.0f, col);
				duAppendArc(dd, vA1[0], vA1[1], vA1[2], vB1[0], vB1[1], vB1[2], 0.25f, 0.0f, 0.0f, col);
			}
		}
		dd->end();

		dd->begin(DU_DRAW_QUADS);
		for (int i = 0; i < tile->header->polyCount; ++i)
		{
			const dtPoly* p = &tile->polys[i];
			if (p->getType() != DT_POLYTYPE_OFFMESH_SEGMENT)	// Skip regular polys.
				continue;

			unsigned int col;
			if (query && query->isInClosedList(base | (dtPolyRef)i))
				col = duRGBA(255,196,0,220);
			else
				col = duDarkenCol(duIntToCol(p->getArea(), 220));

			dtReal vA0[3], vA1[3], vB0[3], vB1[3]; //@UE
			tile->verts[p->verts[0]].toWorld(tile->header->bmin, vA0); //@UE
			tile->verts[p->verts[1]].toWorld(tile->header->bmin, vA1); //@UE
			tile->verts[p->verts[2]].toWorld(tile->header->bmin, vB0); //@UE
			tile->verts[p->verts[3]].toWorld(tile->header->bmin, vB1); //@UE
			duAppendArcSegment(dd, vA0[0], vA0[1], vA0[2], vA1[0], vA1[1], vA1[2], vB0[0], vB0[1], vB0[2], vB1[0], vB1[1], vB1[2], 0.25f, col);
		}
		dd->end();
#endif // WITH_NAVMESH_SEGMENT_LINKS
//@UE END
	}
	
	const unsigned int vcol = duRGBA(0,0,0,196);
	dd->begin(DU_DRAW_POINTS, 3.0f);
	for (int i = 0; i < tile->header->vertCount; ++i)
	{
		dtReal v[3]; //@UE
		tile->verts[i].toWorld(tile->header->bmin, v); //@UE
		dd->vertex(v[0], v[1], v[2], vcol);
	}
	dd->end();

	dd->depthMask(true);
}

void duDebugDrawNavMesh(duDebugDraw* dd, const dtNavMesh& mesh, unsigned char flags)
{
	if (!dd) return;
	
	for (int i = 0; i < mesh.getMaxTiles(); ++i)
	{
		const dtMeshTile* tile = mesh.getTile(i);
		if (!tile->header) continue;
		drawMeshTile(dd, mesh, 0, tile, flags);
	}
}

void duDebugDrawNavMeshWithClosedList(struct duDebugDraw* dd, const dtNavMesh& mesh, const dtNavMeshQuery& query, unsigned char flags)
{
	if (!dd) return;

	const dtNavMeshQuery* q = (flags & DU_DRAWNAVMESH_CLOSEDLIST) ? &query : 0;
	
	for (int i = 0; i < mesh.getMaxTiles(); ++i)
	{
		const dtMeshTile* tile = mesh.getTile(i);
		if (!tile->header) continue;
		drawMeshTile(dd, mesh, q, tile, flags);
	}
}

void duDebugDrawNavMeshNodes(struct duDebugDraw* dd, const dtNavMeshQuery& query)
{
	if (!dd) return;
	
	const dtNodePool* pool = query.getNodePool();
	if (pool)
	{
		const dtReal off = 0.5f;
		dd->begin(DU_DRAW_POINTS, 4.0f);
		for (int i = 0; i < pool->getHashSize(); ++i)
		{
			for (dtNodeIndex j = pool->getFirst(i); j != DT_NULL_IDX; j = pool->getNext(j))
			{
				const dtNode* node = pool->getNodeAtIdx(j+1);
				if (!node) continue;
				dd->vertex(node->pos[0],node->pos[1]+off,node->pos[2], duRGBA(255,192,0,255));
			}
		}
		dd->end();
		
		dd->begin(DU_DRAW_LINES, 2.0f);
		for (int i = 0; i < pool->getHashSize(); ++i)
		{
			for (dtNodeIndex j = pool->getFirst(i); j != DT_NULL_IDX; j = pool->getNext(j))
			{
				const dtNode* node = pool->getNodeAtIdx(j+1);
				if (!node) continue;
				if (!node->pidx) continue;
				const dtNode* parent = pool->getNodeAtIdx(node->pidx);
				if (!parent) continue;
				dd->vertex(node->pos[0],node->pos[1]+off,node->pos[2], duRGBA(255,192,0,128));
				dd->vertex(parent->pos[0],parent->pos[1]+off,parent->pos[2], duRGBA(255,192,0,128));
			}
		}
		dd->end();
	}
}


static void drawMeshTileBVTree(duDebugDraw* dd, const dtMeshTile* tile, const dtReal bvQuantFactor)
{
	// Draw BV nodes.
	const dtReal cs = 1.0f / bvQuantFactor;
	dd->begin(DU_DRAW_LINES, 1.0f);
	for (int i = 0; i < tile->header->bvNodeCount; ++i)
	{
		const dtBVNode* n = &tile->bvTree[i];
		if (n->i < 0) // Leaf indices are positive.
			continue;
		duAppendBoxWire(dd, tile->header->bmin[0] + n->bmin[0]*cs,
						tile->header->bmin[1] + n->bmin[1]*cs,
						tile->header->bmin[2] + n->bmin[2]*cs,
						tile->header->bmin[0] + n->bmax[0]*cs,
						tile->header->bmin[1] + n->bmax[1]*cs,
						tile->header->bmin[2] + n->bmax[2]*cs,
						duRGBA(255,255,255,128));
	}
	dd->end();
}

void duDebugDrawNavMeshBVTree(duDebugDraw* dd, const dtNavMesh& mesh)
{
	if (!dd) return;
	
	for (int i = 0; i < mesh.getMaxTiles(); ++i)
	{
		const dtMeshTile* tile = mesh.getTile(i);
		if (!tile->header) continue;

		const dtReal bvQuantFactor = mesh.getBVQuantFactor(tile->header->resolution);
		drawMeshTileBVTree(dd, tile, bvQuantFactor);
	}
}

static void drawMeshTilePortal(duDebugDraw* dd, const dtMeshTile* tile, const dtNavMesh& mesh)
{
	// Draw portals
	const dtReal padx = 0.04f;
	const dtReal pady = mesh.getWalkableClimb();

	dd->begin(DU_DRAW_LINES, 2.0f);

	for (int side = 0; side < 8; ++side)
	{
		unsigned short m = DT_EXT_LINK | (unsigned short)side;
		
		for (int i = 0; i < tile->header->polyCount; ++i)
		{
			dtPoly* poly = &tile->polys[i];
			
			// Create new links.
			const int nv = poly->vertCount;
			for (int j = 0; j < nv; ++j)
			{
				// Skip edges which do not point to the right side.
				if (poly->neis[j] != m)
					continue;
				
				// Create new links
				dtReal va[3], vb[3]; //@UE
				tile->verts[poly->verts[j]].toWorld(tile->header->bmin, va); //@UE
				tile->verts[poly->verts[(j+1) % nv]].toWorld(tile->header->bmin, vb); //@UE
				
				if (side == 0 || side == 4)
				{
					unsigned int col = side == 0 ? duRGBA(128,0,0,128) : duRGBA(128,0,128,128);

					const dtReal x = va[0] + ((side == 0) ? -padx : padx);
					
					dd->vertex(x,va[1]-pady,va[2], col);
					dd->vertex(x,va[1]+pady,va[2], col);

					dd->vertex(x,va[1]+pady,va[2], col);
					dd->vertex(x,vb[1]+pady,vb[2], col);

					dd->vertex(x,vb[1]+pady,vb[2], col);
					dd->vertex(x,vb[1]-pady,vb[2], col);

					dd->vertex(x,vb[1]-pady,vb[2], col);
					dd->vertex(x,va[1]-pady,va[2], col);
				}
				else if (side == 2 || side == 6)
				{
					unsigned int col = side == 2 ? duRGBA(0,128,0,128) : duRGBA(0,128,128,128);

					const dtReal z = va[2] + ((side == 2) ? -padx : padx);
					
					dd->vertex(va[0],va[1]-pady,z, col);
					dd->vertex(va[0],va[1]+pady,z, col);
					
					dd->vertex(va[0],va[1]+pady,z, col);
					dd->vertex(vb[0],vb[1]+pady,z, col);
					
					dd->vertex(vb[0],vb[1]+pady,z, col);
					dd->vertex(vb[0],vb[1]-pady,z, col);
					
					dd->vertex(vb[0],vb[1]-pady,z, col);
					dd->vertex(va[0],va[1]-pady,z, col);
				}

			}
		}
	}
	
	dd->end();
}

void duDebugDrawNavMeshPortals(duDebugDraw* dd, const dtNavMesh& mesh)
{
	if (!dd) return;
	
	for (int i = 0; i < mesh.getMaxTiles(); ++i)
	{
		const dtMeshTile* tile = mesh.getTile(i);
		if (!tile->header) continue;
		drawMeshTilePortal(dd, tile, mesh);
	}
}

void duDebugDrawNavMeshPolysWithFlags(struct duDebugDraw* dd, const dtNavMesh& mesh,
									  const unsigned short polyFlags, const unsigned int col)
{
	if (!dd) return;
	
	for (int i = 0; i < mesh.getMaxTiles(); ++i)
	{
		const dtMeshTile* tile = mesh.getTile(i);
		if (!tile->header) continue;
		dtPolyRef base = mesh.getPolyRefBase(tile);

		for (int j = 0; j < tile->header->polyCount; ++j)
		{
			const dtPoly* p = &tile->polys[j];
			if ((p->flags & polyFlags) == 0) continue;
			duDebugDrawNavMeshPoly(dd, mesh, base|(dtPolyRef)j, col);
		}
	}
}

void duDebugDrawNavMeshPoly(duDebugDraw* dd, const dtNavMesh& mesh, dtPolyRef ref, const unsigned int col)
{
	if (!dd) return;
	
	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	if (dtStatusFailed(mesh.getTileAndPolyByRef(ref, &tile, &poly)))
		return;
	
	dd->depthMask(false);
	
	const unsigned int c = (col & 0x00ffffff) | (64 << 24);
	const unsigned int ip = (unsigned int)(poly - tile->polys);

	if (poly->getType() == DT_POLYTYPE_OFFMESH_POINT)
	{
		dtOffMeshConnection* con = &tile->offMeshCons[ip - tile->header->offMeshBase];

		dd->begin(DU_DRAW_LINES, 2.0f);

		// Connection arc.
		duAppendArc(dd, con->pos[0],con->pos[1],con->pos[2], con->pos[3],con->pos[4],con->pos[5], 0.25f,
					con->getBiDirectional() ? 0.6f : 0, 0.6f, c);
		
		dd->end();
	}
//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	else if (poly->getType() == DT_POLYTYPE_OFFMESH_SEGMENT)
	{
		dd->begin(DU_DRAW_QUADS);
		dtReal vA0[3], vA1[3], vB0[3], vB1[3];
		tile->verts[poly->verts[0]].toWorld(tile->header->bmin, vA0);
		tile->verts[poly->verts[1]].toWorld(tile->header->bmin, vA1);
		tile->verts[poly->verts[2]].toWorld(tile->header->bmin, vB0);
		tile->verts[poly->verts[3]].toWorld(tile->header->bmin, vB1);
		duAppendArcSegment(dd, vA0[0], vA0[1], vA0[2], vA1[0], vA1[1], vA1[2], vB0[0], vB0[1], vB0[2], vB1[0], vB1[1], vB1[2], 0.25f, c);
		dd->end();
	}
#endif // WITH_NAVMESH_SEGMENT_LINKS
//@UE END
	else if (poly->getType() == DT_POLYTYPE_GROUND)
	{
		const dtPolyDetail* pd = &tile->detailMeshes[ip];

		dd->begin(DU_DRAW_TRIS);
		for (int i = 0; i < pd->triCount; ++i)
		{
			const unsigned char* t = &tile->detailTris[(pd->triBase+i)*4];
			for (int j = 0; j < 3; ++j)
			{
				dtReal tv[3]; //@UE
				if (t[j] < poly->vertCount)
					tile->verts[poly->verts[t[j]]].toWorld(tile->header->bmin, tv); //@UE
				else
					tile->detailVerts[pd->vertBase+t[j]-poly->vertCount].toWorld(tile->header->bmin, tv); //@UE
				dd->vertex(tv, c); //@UE
			}
		}
		dd->end();
	}
	
	dd->depthMask(true);

}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
void duDebugDrawNavMeshClusters(struct duDebugDraw* dd, const dtNavMesh& mesh)
{
	if (!dd) return;

	for (int i = 0; i < mesh.getMaxTiles(); ++i)
	{
		const dtMeshTile* tile = mesh.getTile(i);
		if (!tile->header) continue;

		dtPolyRef base = mesh.getPolyRefBase(tile);
		for (int ip = 0; ip < tile->header->polyCount; ip++)
		{
			duDebugDrawNavMeshPoly(dd, mesh, base | (dtPolyRef)ip, duIntToCol(tile->polyClusters[ip],255));
		}
	}
} 

void duDebugDrawNavMeshCluster(struct duDebugDraw* dd, const dtNavMesh& mesh, dtClusterRef ref, const unsigned int col)
{
	if (!dd) return;

	const dtMeshTile* tile = mesh.getTileByRef(ref);
	const unsigned int clusterIdx = mesh.decodeClusterIdCluster(ref);
	const dtPolyRef base = mesh.getPolyRefBase(tile);

	if (tile->header)
	{
		for (int ip = 0; ip < tile->header->polyCount; ip++)
		{
			if (tile->polyClusters[ip] == clusterIdx)
				duDebugDrawNavMeshPoly(dd, mesh, base | (dtPolyRef)ip, col);
		}
	}
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

static void debugDrawTileCachePortals(struct duDebugDraw* dd, const dtTileCacheLayer& layer, const dtReal cs, const dtReal ch)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal* bmin = layer.header->bmin;

	// Portals
	unsigned int pcol = duRGBA(255,255,255,255);
	
	const int segs[4*4] = {0,0,0,1, 0,1,1,1, 1,1,1,0, 1,0,0,0};
	
	// Layer portals
	dd->begin(DU_DRAW_LINES, 2.0f);
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int idx = x+y*w;
			const int lh = (int)layer.heights[idx];
			if (lh == 0xffff) continue;
			
			for (int dir = 0; dir < 4; ++dir)
			{
				if (layer.cons[idx] & (1<<(dir+4)))
				{
					const int* seg = &segs[dir*4];
					const dtReal ax = bmin[0] + (x+seg[0])*cs;
					const dtReal ay = bmin[1] + (lh+2)*ch;
					const dtReal az = bmin[2] + (y+seg[1])*cs;
					const dtReal bx = bmin[0] + (x+seg[2])*cs;
					const dtReal by = bmin[1] + (lh+2)*ch;
					const dtReal bz = bmin[2] + (y+seg[3])*cs;
					dd->vertex(ax, ay, az, pcol);
					dd->vertex(bx, by, bz, pcol);
				}
			}
		}
	}
	dd->end();
}

void duDebugDrawTileCacheLayerAreas(struct duDebugDraw* dd, const dtTileCacheLayer& layer, const dtReal cs, const dtReal ch)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal* bmin = layer.header->bmin;
	const dtReal* bmax = layer.header->bmax;
	const int idx = layer.header->tlayer;
	
	unsigned int color = duIntToCol(idx+1, 255);
	
	// Layer bounds
	dtReal lbmin[3], lbmax[3];
	lbmin[0] = bmin[0] + layer.header->minx*cs;
	lbmin[1] = bmin[1];
	lbmin[2] = bmin[2] + layer.header->miny*cs;
	lbmax[0] = bmin[0] + (layer.header->maxx+1)*cs;
	lbmax[1] = bmax[1];
	lbmax[2] = bmin[2] + (layer.header->maxy+1)*cs;
	duDebugDrawBoxWire(dd, lbmin[0],lbmin[1],lbmin[2], lbmax[0],lbmax[1],lbmax[2], duTransCol(color,128), 2.0f);
	
	// Layer height
	dd->begin(DU_DRAW_QUADS);
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int lidx = x+y*w;
			const int lh = (int)layer.heights[lidx];
			if (lh == 0xffff) continue;
			const unsigned char area = layer.areas[lidx];
			
			unsigned int col;
			if (area == 63)
				col = duLerpCol(color, duRGBA(0,192,255,64), 32);
			else if (area == 0)
				col = duLerpCol(color, duRGBA(0,0,0,64), 32);
			else
				col = duLerpCol(color, duIntToCol(area, 255), 32);
			
			const dtReal fx = bmin[0] + x*cs;
			const dtReal fy = bmin[1] + (lh+1)*ch;
			const dtReal fz = bmin[2] + y*cs;
			
			dd->vertex(fx, fy, fz, col);
			dd->vertex(fx, fy, fz+cs, col);
			dd->vertex(fx+cs, fy, fz+cs, col);
			dd->vertex(fx+cs, fy, fz, col);
		}
	}
	dd->end();
	
	debugDrawTileCachePortals(dd, layer, cs, ch);
}

void duDebugDrawTileCacheLayerDistance(struct duDebugDraw* dd, const dtTileCacheLayer& layer, const dtTileCacheDistanceField& dfield, const dtReal cs, const dtReal ch)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal* bmin = layer.header->bmin;
	const dtReal* bmax = layer.header->bmax;
	const int idx = layer.header->tlayer;

	dtReal maxd = dfield.maxDist;
	if (maxd < 1.0f) maxd = 1;
	const dtReal dscale = 255.0f / maxd;

	unsigned int color = duIntToCol(idx+1, 255);

	// Layer bounds
	dtReal lbmin[3], lbmax[3];
	lbmin[0] = bmin[0] + layer.header->minx*cs;
	lbmin[1] = bmin[1];
	lbmin[2] = bmin[2] + layer.header->miny*cs;
	lbmax[0] = bmin[0] + (layer.header->maxx+1)*cs;
	lbmax[1] = bmax[1];
	lbmax[2] = bmin[2] + (layer.header->maxy+1)*cs;
	duDebugDrawBoxWire(dd, lbmin[0],lbmin[1],lbmin[2], lbmax[0],lbmax[1],lbmax[2], duTransCol(color,128), 2.0f);

	// Layer Distance field
	dd->begin(DU_DRAW_QUADS);
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int lidx = x+y*w;
			const int lh = (int)layer.heights[lidx];
			if (lh == 0xffff) continue;

			const unsigned char cd = (unsigned char)(dfield.data[lidx] * dscale);
			const unsigned int col = duRGBA(cd,cd,cd,255);

			const dtReal fx = bmin[0] + x*cs;
			const dtReal fy = bmin[1] + (lh+1)*ch;
			const dtReal fz = bmin[2] + y*cs;

			dd->vertex(fx, fy, fz, col);
			dd->vertex(fx, fy, fz+cs, col);
			dd->vertex(fx+cs, fy, fz+cs, col);
			dd->vertex(fx+cs, fy, fz, col);
		}
	}
	dd->end();

	debugDrawTileCachePortals(dd, layer, cs, ch);
}

void duDebugDrawTileCacheLayerRegions(struct duDebugDraw* dd, const dtTileCacheLayer& layer, const dtReal cs, const dtReal ch)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal* bmin = layer.header->bmin;
	const dtReal* bmax = layer.header->bmax;
	const int idx = layer.header->tlayer;
	
	unsigned int color = duIntToCol(idx+1, 255);
	
	// Layer bounds
	dtReal lbmin[3], lbmax[3];
	lbmin[0] = bmin[0] + layer.header->minx*cs;
	lbmin[1] = bmin[1];
	lbmin[2] = bmin[2] + layer.header->miny*cs;
	lbmax[0] = bmin[0] + (layer.header->maxx+1)*cs;
	lbmax[1] = bmax[1];
	lbmax[2] = bmin[2] + (layer.header->maxy+1)*cs;
	duDebugDrawBoxWire(dd, lbmin[0],lbmin[1],lbmin[2], lbmax[0],lbmax[1],lbmax[2], duTransCol(color,128), 2.0f);
	
	// Layer height
	dd->begin(DU_DRAW_QUADS);
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int lidx = x+y*w;
			const int lh = (int)layer.heights[lidx];
			if (lh == 0xffff) continue;
			const unsigned short reg = layer.regs[lidx];
			
			unsigned int col = duLerpCol(color, duIntToCol(reg, 255), 192);
			
			const dtReal fx = bmin[0] + x*cs;
			const dtReal fy = bmin[1] + (lh+1)*ch;
			const dtReal fz = bmin[2] + y*cs;
			
			dd->vertex(fx, fy, fz, col);
			dd->vertex(fx, fy, fz+cs, col);
			dd->vertex(fx+cs, fy, fz+cs, col);
			dd->vertex(fx+cs, fy, fz, col);
		}
	}
	dd->end();
	
	debugDrawTileCachePortals(dd, layer, cs, ch);
}




/*struct dtTileCacheContour
{
	int nverts;
	unsigned char* verts;
	unsigned char reg;
	unsigned char area;
};

struct dtTileCacheContourSet
{
	int nconts;
	dtTileCacheContour* conts;
};*/

void duDebugDrawTileCacheContours(duDebugDraw* dd, const struct dtTileCacheContourSet& lcset,
								  const int32 layerIdx, const dtReal* orig, const dtReal cs, const dtReal ch)
{
	if (!dd) return;
	
	const unsigned char a = 255;// (unsigned char)(alpha*255.0f);
	
	const int offs[2*4] = {-1,0, 0,1, 1,0, 0,-1};
	
	dd->begin(DU_DRAW_LINES, 2.0f);
	
	for (int i = 0; i < lcset.nconts; ++i)
	{
		const dtTileCacheContour& c = lcset.conts[i];
		unsigned int color = 0;
		
		color = duIntToCol(i, a);
		
		for (int j = 0; j < c.nverts; ++j)
		{
			const int k = (j+1) % c.nverts;
			const unsigned short* va = &c.verts[j*4];
			const unsigned short* vb = &c.verts[k*4];
			const dtReal verticalOffset = 1 + (0.05*i) + (0.3*layerIdx);
			const dtReal ax = orig[0] + va[0]*cs;
			const dtReal ay = orig[1] + (va[1]+verticalOffset)*ch;
			const dtReal az = orig[2] + va[2]*cs;
			const dtReal bx = orig[0] + vb[0]*cs;
			const dtReal by = orig[1] + (vb[1]+verticalOffset)*ch;
			const dtReal bz = orig[2] + vb[2]*cs;
			unsigned int col = color;
			if ((va[3] & 0xf) != 0xf)
			{
				// Portal segment
				col = duRGBA(255,255,255,128);
				int d = va[3] & 0xf;
				
				const dtReal cx = (ax+bx)*0.5f;
				const dtReal cy = (ay+by)*0.5f;
				const dtReal cz = (az+bz)*0.5f;

				CA_SUPPRESS(6385);
				const dtReal dx = cx + offs[d*2+0]*2*cs;
				const dtReal dy = cy;
				const dtReal dz = cz + offs[d*2+1]*2*cs;
				
				dd->vertex(cx,cy,cz,duRGBA(255,0,0,255));
				dd->vertex(dx,dy,dz,duRGBA(255,0,0,255));
			}
			
			duAppendArrow(dd, ax,ay,az, bx,by,bz, 0.0, cs*0.2, col);
		}
	}
	dd->end();
	
	dd->begin(DU_DRAW_POINTS, 4.0f);	
	
	for (int i = 0; i < lcset.nconts; ++i)
	{
		const dtTileCacheContour& c = lcset.conts[i];
		unsigned int color = 0;
		
		for (int j = 0; j < c.nverts; ++j)
		{
			const unsigned short* va = &c.verts[j*4];
			
			color = duDarkenCol(duIntToCol(i, a));
			if (va[3] & 0x80)
			{
				// Border vertex
				color = duRGBA(255,0,0,255);
			}
			
			dtReal fx = orig[0] + va[0]*cs;
			const dtReal verticalOffset = 1 + (0.05*i) + (0.3*layerIdx);
			dtReal fy = orig[1] + (va[1]+verticalOffset)*ch;
			dtReal fz = orig[2] + va[2]*cs;
			dd->vertex(fx,fy,fz, color);

			// Add label for contour edge.
			constexpr int bufferSize = 128;
			char text[bufferSize];
			snprintf(text, bufferSize, "%i (%i:%i) (%u,%u,%u) neiReg:%u r:%u a:%u", layerIdx, i, j, va[0], va[1], va[2], va[3], c.reg, c.area);
			dd->text(fx,fy,fz, text);
		}
	}
	dd->end();
}

void duDebugDrawTileCacheClusters(duDebugDraw* dd, const struct dtTileCacheClusterSet& lclusters)
{
	// empty for now
}

void duDebugDrawTileCachePolyMesh(duDebugDraw* dd, const struct dtTileCachePolyMesh& lmesh,
								  const dtReal* orig, const dtReal cs, const dtReal ch)
{
	if (!dd) return;
	
	const int nvp = lmesh.nvp;
	
	const int offs[2*4] = {-1,0, 0,1, 1,0, 0,-1};
	
	dd->begin(DU_DRAW_TRIS);
	
	for (int i = 0; i < lmesh.npolys; ++i)
	{
		const unsigned short* p = &lmesh.polys[i*nvp*2];
		
		unsigned int color;
		if (lmesh.areas[i] == DT_TILECACHE_WALKABLE_AREA)
			color = duRGBA(0,192,255,64);
		else if (lmesh.areas[i] == DT_TILECACHE_NULL_AREA)
			color = duRGBA(0,0,0,64);
		else
			color = duIntToCol(lmesh.areas[i], 255);
		
		unsigned short vi[3];
		for (int j = 2; j < nvp; ++j)
		{
			if (p[j] == DT_TILECACHE_NULL_IDX) break;
			vi[0] = p[0];
			vi[1] = p[j-1];
			vi[2] = p[j];
			for (int k = 0; k < 3; ++k)
			{
				const unsigned short* v = &lmesh.verts[vi[k]*3];
				const dtReal x = orig[0] + v[0]*cs;
				const dtReal y = orig[1] + (v[1]+1)*ch;
				const dtReal z = orig[2] + v[2]*cs;
				dd->vertex(x,y,z, color);
			}
		}
	}
	dd->end();
	
	// Draw neighbours edges
	const unsigned int coln = duRGBA(0,48,64,32);
	dd->begin(DU_DRAW_LINES, 1.5f);
	for (int i = 0; i < lmesh.npolys; ++i)
	{
		const unsigned short* p = &lmesh.polys[i*nvp*2];
		for (int j = 0; j < nvp; ++j)
		{
			if (p[j] == DT_TILECACHE_NULL_IDX) break;
			if (p[nvp+j] & 0x8000) continue;
			const int nj = (j+1 >= nvp || p[j+1] == DT_TILECACHE_NULL_IDX) ? 0 : j+1; 
			int vi[2] = {p[j], p[nj]};
			
			for (int k = 0; k < 2; ++k)
			{
				const unsigned short* v = &lmesh.verts[vi[k]*3];
				const dtReal x = orig[0] + v[0]*cs;
				const dtReal y = orig[1] + (v[1]+1)*ch + 0.1f;
				const dtReal z = orig[2] + v[2]*cs;
				dd->vertex(x, y, z, coln);
			}
		}
	}
	dd->end();
	
	// Draw boundary edges
	const unsigned int colb = duRGBA(0,48,64,220);
	dd->begin(DU_DRAW_LINES, 2.5f);
	for (int i = 0; i < lmesh.npolys; ++i)
	{
		const unsigned short* p = &lmesh.polys[i*nvp*2];
		for (int j = 0; j < nvp; ++j)
		{
			if (p[j] == DT_TILECACHE_NULL_IDX) break;
			if ((p[nvp+j] & 0x8000) == 0) continue;
			const int nj = (j+1 >= nvp || p[j+1] == DT_TILECACHE_NULL_IDX) ? 0 : j+1; 
			int vi[2] = {p[j], p[nj]};
			
			unsigned int col = colb;
			if ((p[nvp+j] & 0xf) != 0xf)
			{
				const unsigned short* va = &lmesh.verts[vi[0]*3];
				const unsigned short* vb = &lmesh.verts[vi[1]*3];
				
				const dtReal ax = orig[0] + va[0]*cs;
				const dtReal ay = orig[1] + (va[1]+1+(i&1))*ch;
				const dtReal az = orig[2] + va[2]*cs;
				const dtReal bx = orig[0] + vb[0]*cs;
				const dtReal by = orig[1] + (vb[1]+1+(i&1))*ch;
				const dtReal bz = orig[2] + vb[2]*cs;
				
				const dtReal cx = (ax+bx)*0.5f;
				const dtReal cy = (ay+by)*0.5f;
				const dtReal cz = (az+bz)*0.5f;
				
				int d = p[nvp+j] & 0xf;

				CA_SUPPRESS(6385);
				const dtReal dx = cx + offs[d*2+0]*2*cs;
				const dtReal dy = cy;
				const dtReal dz = cz + offs[d*2+1]*2*cs;
				
				dd->vertex(cx,cy,cz,duRGBA(255,0,0,255));
				dd->vertex(dx,dy,dz,duRGBA(255,0,0,255));
				
				col = duRGBA(255,255,255,128);
			}
			
			for (int k = 0; k < 2; ++k)
			{
				const unsigned short* v = &lmesh.verts[vi[k]*3];
				const dtReal x = orig[0] + v[0]*cs;
				const dtReal y = orig[1] + (v[1]+1)*ch + 0.1f;
				const dtReal z = orig[2] + v[2]*cs;
				dd->vertex(x, y, z, col);
			}
		}
	}
	dd->end();
	
	dd->begin(DU_DRAW_POINTS, 3.0f);
	const unsigned int colv = duRGBA(0,0,0,220);
	for (int i = 0; i < lmesh.nverts; ++i)
	{
		const unsigned short* v = &lmesh.verts[i*3];
		const dtReal x = orig[0] + v[0]*cs;
		const dtReal y = orig[1] + (v[1]+1)*ch + 0.1f;
		const dtReal z = orig[2] + v[2]*cs;
		dd->vertex(x,y,z, colv);
	}
	dd->end();
}

void duDebugDrawTileCacheDetailMesh(duDebugDraw* dd, const struct dtTileCachePolyMeshDetail& dmesh)
{
	if (!dd) return;

	dd->begin(DU_DRAW_TRIS);

	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const unsigned int btris = m[2];
		const int ntris = (int)m[3];
		const dtReal* verts = &dmesh.verts[bverts*3];
		const unsigned char* tris = &dmesh.tris[btris*4];

		unsigned int color = duIntToCol(i, 192);

		for (int j = 0; j < ntris; ++j)
		{
			dd->vertex(&verts[tris[j*4+0]*3], color);
			dd->vertex(&verts[tris[j*4+1]*3], color);
			dd->vertex(&verts[tris[j*4+2]*3], color);
		}
	}
	dd->end();

	// Internal edges.
	dd->begin(DU_DRAW_LINES, 1.0f);
	const unsigned int coli = duRGBA(0,0,0,64);
	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const unsigned int btris = m[2];
		const int ntris = (int)m[3];
		const dtReal* verts = &dmesh.verts[bverts*3];
		const unsigned char* tris = &dmesh.tris[btris*4];

		for (int j = 0; j < ntris; ++j)
		{
			const unsigned char* t = &tris[j*4];
			for (int k = 0, kp = 2; k < 3; kp=k++)
			{
				unsigned char ef = (t[3] >> (kp*2)) & 0x3;
				if (ef == 0)
				{
					// Internal edge
					if (t[kp] < t[k])
					{
						dd->vertex(&verts[t[kp]*3], coli);
						dd->vertex(&verts[t[k]*3], coli);
					}
				}
			}
		}
	}
	dd->end();

	// External edges.
	dd->begin(DU_DRAW_LINES, 2.0f);
	const unsigned int cole = duRGBA(0,0,0,64);
	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const unsigned int btris = m[2];
		const int ntris = (int)m[3];
		const dtReal* verts = &dmesh.verts[bverts*3];
		const unsigned char* tris = &dmesh.tris[btris*4];

		for (int j = 0; j < ntris; ++j)
		{
			const unsigned char* t = &tris[j*4];
			for (int k = 0, kp = 2; k < 3; kp=k++)
			{
				unsigned char ef = (t[3] >> (kp*2)) & 0x3;
				if (ef != 0)
				{
					// Ext edge
					dd->vertex(&verts[t[kp]*3], cole);
					dd->vertex(&verts[t[k]*3], cole);
				}
			}
		}
	}
	dd->end();

	dd->begin(DU_DRAW_POINTS, 3.0f);
	const unsigned int colv = duRGBA(0,0,0,64);
	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const int nverts = (int)m[1];
		const dtReal* verts = &dmesh.verts[bverts*3];
		for (int j = 0; j < nverts; ++j)
			dd->vertex(&verts[j*3], colv);
	}
	dd->end();
}

