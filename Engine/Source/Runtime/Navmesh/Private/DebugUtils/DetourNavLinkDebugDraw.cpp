// Copyright Epic Games, Inc. All Rights Reserved.
#include "DebugUtils/DetourNavLinkDebugDraw.h"

#include "DebugUtils/DebugDraw.h"
#include "DebugUtils/RecastDebugDraw.h"
#include "Detour/DetourCommon.h"
#include "Detour/DetourNavLinkBuilder.h"
#include "Recast/Recast.h"

//@UE BEGIN
namespace UE::Detour::Private
{
	dtReal getPathLen(const dtReal* path, int npath)
	{
		if (npath < 2)
			return 0;
		
		dtReal totd = 0;
		for (int i = 0; i < npath-1; ++i)
		{
			const dtReal* sp = &path[i*3];
			const dtReal* sq = &path[(i+1)*3];
			totd += sqrtf(dtVdistSqr(sp,sq));
		}
		
		return totd;
	}

	void getPointAlongPath(dtReal dist, const dtReal* path, int npath, dtReal* pt, dtReal* dir)
	{
		if (npath == 0)
			return;
		
		if (npath == 1)
		{
			if (dir)
				dtVset(dir,1,0,0);
			dtVcopy(pt, path);
			return;
		}
		
		if (dist <= 0)
		{
			if (dir)
				dtVsub(dir, &path[1*3], &path[0*3]);
			
			dtVcopy(pt, path);
			return;
		}
		
		float totd = 0;
		for (int i = 0; i < npath-1; ++i)
		{
			const dtReal* sp = &path[i*3];
			const dtReal* sq = &path[(i+1)*3];
			const float sd = sqrtf(dtVdistSqr(sp,sq));
			if (dist >= totd && dist <= totd+sd)
			{
				dtVlerp(pt, sp,sq,(dist-totd)/sd);
				if (dir)
					dtVsub(dir, sq,sp);
				return;
			}
			totd += sd;
		}
		
		dtVcopy(pt, &path[(npath-1)*3]);
		if (dir)
		{
			dtVsub(dir, &path[(npath-1)*3], &path[(npath-2)*3]);
		}
	}

	void drawTrajectory(duDebugDraw* dd, const dtReal* pa, const dtReal* pb, const dtNavLinkBuilder::Trajectory2D* trajectory, const unsigned int color)
	{
		const float startx = trajectory->spine[0];
		const float endx = trajectory->spine[(trajectory->nspine-1)*2];
		const float deltax = endx - startx;
		
		const float starty = trajectory->spine[1];
		const float endy = trajectory->spine[(trajectory->nspine-1)*2+1];
		
		dtReal pts[3*dtNavLinkBuilder::MAX_SPINE];
		int npts = trajectory->nspine;
		for (int i = 0; i < trajectory->nspine; ++i)
		{
			const float* spt = &trajectory->spine[i*2];
			const float u = (spt[0] - startx)/deltax;
			const float dy = spt[1] - dtLerp(starty, endy, u);
			dtReal* p = &pts[i*3];
			dtVlerp(p, pa, pb, u);
			p[1] += dy;
		}
			
		const float len = getPathLen(pts, npts);
		const int nsegs = ceilf(len / 0.3f);
		
		for (int i = 0; i < nsegs*2; ++i)
		{
			float u = (float)i / (float)(nsegs*2);
			dtReal pt[3];
			getPointAlongPath(u*len, pts, npts, pt, nullptr);
			dd->vertex(pt, color);
		}
	}

	void drawTrajectorySlice(duDebugDraw* dd, const dtReal* pa, const dtReal* pb, const dtNavLinkBuilder::Trajectory2D* trajectory, const dtReal* trajectoryDir, const unsigned int color)
	{
		dtReal start[3];
		dtReal end[3];
		dtVcopy(start, pa);
		dtVcopy(end, pb);
		
		// Offset start and end points to account for the agent radius.
		dtVmad(start, pa, trajectoryDir, -trajectory->radiusOverflow);
		dtVmad(end, pb, trajectoryDir,  trajectory->radiusOverflow);
		
		unsigned int colt = duTransCol(color, 50);
		unsigned int colb = duTransCol(duLerpCol(color,duColor::black,96), 50);
		
		dd->begin(DU_DRAW_QUADS);
		dtReal p0[3], p1[3], p2[3], p3[3];
		const int nsamples = trajectory->samples.Num();
		for (int i = 0; i < nsamples; ++i)
		{
			const dtNavLinkBuilder::TrajectorySample& s = trajectory->samples[i];
			const dtReal u = (float)i / (float)(nsamples-1);
			dtVlerp(p1, start, end, u);
			dtVcopy(p0, p1);
			p0[1] += s.ymin;
			p1[1] += s.ymax;
			
			if (i > 0)
			{
				dd->vertex(p0, colb);
				dd->vertex(p1, colt);
				dd->vertex(p3, colt);
				dd->vertex(p2, colb);

				dd->vertex(p2, colb);
				dd->vertex(p3, colt);
				dd->vertex(p1, colt);
				dd->vertex(p0, colb);
			}
			
			dtVcopy(p2, p0);
			dtVcopy(p3, p1);
		}
		dd->end();

		colb = duDarkenCol(colb);
		colt = duDarkenCol(colt);
		
		dd->begin(DU_DRAW_LINES, 2.0f);
		for (int i = 0; i < nsamples; ++i)
		{
			const dtNavLinkBuilder::TrajectorySample& s = trajectory->samples[i];
			const float u = (float)i / (float)(nsamples-1);
			dtVlerp(p1, start, end, u);
			dtVcopy(p0, p1);
			p0[1] += s.ymin;
			p1[1] += s.ymax;
			
			if (i == 0 || i == (nsamples-1))
			{
				dd->vertex(p0, colb);
				dd->vertex(p1, colt);
			}
			if (i > 0)
			{
				dd->vertex(p0, colb);
				dd->vertex(p2, colb);
				dd->vertex(p1, colt);
				dd->vertex(p3, colt);
			}
			
			dtVcopy(p2, p0);
			dtVcopy(p3, p1);
		}
		dd->end();
	}
} // UE::Detour::Private

void duDebugDrawTrajectorySamples(duDebugDraw* dd, const dtNavLinkBuilder& linkBuilder, const dtReal* pa, const dtReal* pb,
	const dtNavLinkBuilder::Trajectory2D* trajectory, const dtReal* trajectoryDir)
{
	dtReal start[3];
	dtReal end[3];
	dtVcopy(start, pa);
	dtVcopy(end, pb);
	
	// Offset start and end points to account for the agent radius.
	dtVmad(start, pa, trajectoryDir, -trajectory->radiusOverflow);
	dtVmad(end, pb, trajectoryDir,  trajectory->radiusOverflow);

	const int nsamples = trajectory->samples.Num();
	const float invLastSample = 1.f / (nsamples-1);
	for (int i = 0; i < nsamples; ++i)
	{
		dtReal p[3];
		const dtNavLinkBuilder::TrajectorySample& s = trajectory->samples[i];
		const float u = (float)i * invLastSample;
		dtVlerp(p, start, end, u);

		// Draw an additional point if the sample was marked to be snap to the floor.
		if (s.floorStart)
		{
			dd->begin(DU_DRAW_POINTS, 5.0f);
			dd->vertex(p[0], p[1]+s.ymin, p[2], duColor::black);
			dd->end();
		}
		else if (s.floorEnd)
		{
			dd->begin(DU_DRAW_POINTS, 5.0f);
			dd->vertex(p[0], p[1]+s.ymin, p[2], duColor::yellow);
			dd->end();
		}

		// Check with the heightfield if a collision is hit.
		const bool hit = linkBuilder.checkHeightfieldCollision(p[0], p[1] + s.ymin, p[1] + s.ymax, p[2]);
		unsigned int color = hit ? duColor::orangeRed : duColor::lightGreen;

		// Draw a vertical line for the sample.
		dd->begin(DU_DRAW_LINES, 5.0f);
		dd->vertex(p[0], p[1]+s.ymin, p[2], color);
		dd->vertex(p[0], p[1]+s.ymax, p[2], color);
		dd->end();
	}
}

void duDebugDrawNavLinkBuilder(duDebugDraw* dd, const dtNavLinkBuilder& linkBuilder, unsigned int drawFlags, const dtNavLinkBuilder::EdgeSampler* es)
{
	if (dd == nullptr)
		return;
	
	if (drawFlags & DRAW_WALKABLE_SURFACE)
	{
		if (linkBuilder.m_chf)
		{
			duDebugDrawCompactHeightfieldSolid(dd, *linkBuilder.m_chf);	
		}
	}

	if (drawFlags & DRAW_BORDERS)
	{
		const TArray<dtNavLinkBuilder::Edge, TInlineAllocator<32>>& edges = linkBuilder.m_edges;
		const int nedges = linkBuilder.m_edges.Num();
		const int selectedEdge = linkBuilder.m_debugSelectedEdge;
		
		if (nedges)
		{
			dd->begin(DU_DRAW_LINES, 3.0f);
			for (int i = 0; i < nedges; ++i)
			{
				// Display edge index
				dtReal x = 0.5*(edges[i].sp[0]+edges[i].sq[0]);
				dtReal y = 0.5*(edges[i].sp[1]+edges[i].sq[1]);
				dtReal z = 0.5*(edges[i].sp[2]+edges[i].sq[2]);
				constexpr int bufferSize = 32;
				char buffer[bufferSize];
				snprintf(buffer, bufferSize, "%i", i);
				dd->text(x, y, z, buffer);
				
				unsigned int col = duRGBA(0,96,128,255);
				if (i == selectedEdge)
					continue;

				dd->vertex(edges[i].sp, col);
				dd->vertex(edges[i].sq, col);
			}
			dd->end();
			
			dd->begin(DU_DRAW_POINTS, 8.0f);
			for (int i = 0; i < nedges; ++i)
			{
				unsigned int col = duRGBA(0,96,128,255);
				if (i == selectedEdge)
					continue;
				
				dd->vertex(edges[i].sp, col);
				dd->vertex(edges[i].sq, col);
			}
			dd->end();
			
			if (selectedEdge >= 0 && selectedEdge < nedges)
			{
				unsigned int col = duRGBA(68,36,36,255);
				dd->begin(DU_DRAW_LINES, 3.0f);
				dd->vertex(edges[selectedEdge].sp, col);
				dd->vertex(edges[selectedEdge].sq, col);
				dd->end();
				dd->begin(DU_DRAW_POINTS, 8.0f);
				dd->vertex(edges[selectedEdge].sp, col);
				dd->vertex(edges[selectedEdge].sq, col);
				dd->end();
			}

			dd->begin(DU_DRAW_POINTS, 4.0f);
			for (int i = 0; i < nedges; ++i)
			{
				unsigned int col = duColor::lightGrey;
				dd->vertex(edges[i].sp, col);
				dd->vertex(edges[i].sq, col);
			}
			dd->end();
		}
	}
	
	if (drawFlags & DRAW_LINKS)
	{
		const bool drawFilteredLinks = drawFlags & DRAW_FILTERED_LINKS;
		constexpr int saturation = 50;
		unsigned int jumpConfig0Col0 = duLerpCol(duColor::blue, duColor::white, saturation);
		unsigned int jumpConfig0Col1 = duColor::blue;
		unsigned int jumpConfig1Col0 = duLerpCol(duColor::lightGrey, duColor::white, saturation);
		unsigned int jumpConfig1Col1 = duColor::lightGrey;

		auto selectColors = [&](dtNavLinkBuilder::JumpLinkFlag flag, const uint8 configIndex, unsigned int& col0, unsigned int& col1)
		{
			// Predefined "nice" colors for the first two configs. Random colors for the others.
			if (configIndex == 0)
			{
				col0 = jumpConfig0Col0;
				col1 = jumpConfig0Col1;
			}
			else if (configIndex == 1)
			{
				col0 = jumpConfig1Col0;
				col1 = jumpConfig1Col1;
			}
			else
			{
				col1 = duIntToCol(configIndex, 255);
				col0 = duLerpCol(col1, duColor::white, saturation);
			}

			if (flag == dtNavLinkBuilder::FILTERED)
			{
				col0 = duColor::grey;
				col1 = duColor::darkGrey;
			}
		};

		unsigned int col0 = duColor::black;
		unsigned int col1 = duColor::black;
		
		const int nlinks = linkBuilder.m_links.Num();
		if (nlinks)
		{
			dd->begin(DU_DRAW_QUADS);
			for (const dtNavLinkBuilder::JumpLink& link : linkBuilder.m_links)
			{
				if (!drawFilteredLinks && link.flags == dtNavLinkBuilder::FILTERED)
					continue;

				selectColors(link.flags, link.configIndex, col0, col1);
				
				for (int j = 0; j < link.nspine-1; ++j)
				{
					int u = (j*255)/link.nspine;
					unsigned int col = duTransCol(duLerpCol(col0,col1,u),128);
					
					dd->vertex(&link.spine1[j*3], col);
					dd->vertex(&link.spine1[(j+1)*3], col);
					dd->vertex(&link.spine0[(j+1)*3], col);
					dd->vertex(&link.spine0[j*3], col);
				}
			}
			dd->end();
			
			dd->begin(DU_DRAW_LINES, 3.0f);
			for (const dtNavLinkBuilder::JumpLink& link : linkBuilder.m_links)
			{
				if (!drawFilteredLinks && link.flags == dtNavLinkBuilder::FILTERED)
					continue;

				selectColors(link.flags, link.configIndex, col0, col1);
				
				for (int j = 0; j < link.nspine-1; ++j)
				{
					unsigned int col = duTransCol(duDarkenCol(col1),128);
					
					dd->vertex(&link.spine0[j*3], col);
					dd->vertex(&link.spine0[(j+1)*3], col);
					dd->vertex(&link.spine1[j*3], col);
					dd->vertex(&link.spine1[(j+1)*3], col);
				}

				dd->vertex(&link.spine0[0], duDarkenCol(col1));
				dd->vertex(&link.spine1[0], duDarkenCol(col1));

				dd->vertex(&link.spine0[(link.nspine-1)*3], duDarkenCol(col1));
				dd->vertex(&link.spine1[(link.nspine-1)*3], duDarkenCol(col1));
			}
			dd->end();

			dd->begin(DU_DRAW_POINTS, 8.0f);
			for (const dtNavLinkBuilder::JumpLink& link : linkBuilder.m_links)
			{
				if (link.flags == dtNavLinkBuilder::FILTERED)
					continue;

				selectColors(link.flags, link.configIndex, col0, col1);
				
				dd->vertex(&link.spine0[0], duDarkenCol(col1));
				dd->vertex(&link.spine1[0], duDarkenCol(col1));
				dd->vertex(&link.spine0[(link.nspine-1)*3], duDarkenCol(col1));
				dd->vertex(&link.spine1[(link.nspine-1)*3], duDarkenCol(col1));
			}
			dd->end();
			
			dd->begin(DU_DRAW_POINTS, 4.0f);
			for (const dtNavLinkBuilder::JumpLink& link : linkBuilder.m_links)
			{
				if (link.flags == dtNavLinkBuilder::FILTERED)
					continue;
				
				dd->vertex(&link.spine0[0], duColor::lightGrey);
				dd->vertex(&link.spine1[0], duColor::lightGrey);
				dd->vertex(&link.spine0[(link.nspine-1)*3], duColor::lightGrey);
				dd->vertex(&link.spine1[(link.nspine-1)*3], duColor::lightGrey);
			}
			dd->end();
		}
	}

	if (drawFlags & DRAW_SELECTED_EDGE)
	{
		if (es && es->configIndex != dtNavLinkBuilder::INVALID_CONFIG_INDEX)
		{
			dd->begin(DU_DRAW_LINES, 2.0f);

			// Axis
			constexpr float d = 20.f;
			dd->vertex(es->rigp, duColor::red);
			dd->vertex(es->rigp[0]+es->ax[0]*d, es->rigp[1]+es->ax[1]*d, es->rigp[2]+es->ax[2]*d, duColor::red);

			dd->vertex(es->rigp, duColor::green);
			dd->vertex(es->rigp[0]+es->ay[0]*d, es->rigp[1]+es->ay[1]*d, es->rigp[2]+es->ay[2]*d, duColor::green);

			dd->vertex(es->rigp, duColor::blue);
			dd->vertex(es->rigp[0]+es->az[0]*d, es->rigp[1]+es->az[1]*d, es->rigp[2]+es->az[2]*d, duColor::blue);
			
			dd->end();

			const dtNavLinkBuilder::GroundSegment& startSeg = es->start;
			const dtNavLinkBuilder::GroundSegment& endSeg = es->end;
			
			if (drawFlags & DRAW_TRAJECTORY)
			{
				const float r = es->groundRange;

				unsigned int col = duLerpCol(duRGBA(255,192,0,255), duColor::white, 64);
				unsigned int cola = duTransCol(col,192);
				unsigned int colb = duColor::white;
				
				// Start segment.
				dd->begin(DU_DRAW_LINES, 3.0f);
				dd->vertex(startSeg.p[0],startSeg.p[1],startSeg.p[2], col);
				dd->vertex(startSeg.q[0],startSeg.q[1],startSeg.q[2], col);
				dd->end();
				
				dd->begin(DU_DRAW_LINES, 1.0f);
				dd->vertex(startSeg.p[0],startSeg.p[1]-r,startSeg.p[2], colb);
				dd->vertex(startSeg.p[0],startSeg.p[1]+r,startSeg.p[2], colb);
				dd->vertex(startSeg.p[0],startSeg.p[1]+r,startSeg.p[2], colb);
				dd->vertex(startSeg.q[0],startSeg.q[1]+r,startSeg.q[2], colb);
				dd->vertex(startSeg.q[0],startSeg.q[1]+r,startSeg.q[2], colb);
				dd->vertex(startSeg.q[0],startSeg.q[1]-r,startSeg.q[2], colb);
				dd->vertex(startSeg.q[0],startSeg.q[1]-r,startSeg.q[2], colb);
				dd->vertex(startSeg.p[0],startSeg.p[1]-r,startSeg.p[2], colb);
				dd->end();
				
				// End segment.
				dd->begin(DU_DRAW_LINES, 3.0f);
				dd->vertex(endSeg.p[0],endSeg.p[1],endSeg.p[2], col);
				dd->vertex(endSeg.q[0],endSeg.q[1],endSeg.q[2], col);
				dd->end();
				
				dd->begin(DU_DRAW_LINES, 1.0f);
				dd->vertex(endSeg.p[0],endSeg.p[1]-r,endSeg.p[2], colb);
				dd->vertex(endSeg.p[0],endSeg.p[1]+r,endSeg.p[2], colb);
				dd->vertex(endSeg.p[0],endSeg.p[1]+r,endSeg.p[2], colb);
				dd->vertex(endSeg.q[0],endSeg.q[1]+r,endSeg.q[2], colb);
				dd->vertex(endSeg.q[0],endSeg.q[1]+r,endSeg.q[2], colb);
				dd->vertex(endSeg.q[0],endSeg.q[1]-r,endSeg.q[2], colb);
				dd->vertex(endSeg.q[0],endSeg.q[1]-r,endSeg.q[2], colb);
				dd->vertex(endSeg.p[0],endSeg.p[1]-r,endSeg.p[2], colb);
				dd->end();
		 
				dd->begin(DU_DRAW_LINES, 4.0f);
				UE::Detour::Private::drawTrajectory(dd, startSeg.p, endSeg.p, &es->trajectory, cola);
				UE::Detour::Private::drawTrajectory(dd, startSeg.q, endSeg.q, &es->trajectory, cola);
				dd->end();


				dd->begin(DU_DRAW_POINTS, 8.0f);
				dd->vertex(startSeg.p[0],startSeg.p[1],startSeg.p[2], duDarkenCol(col));
				dd->vertex(startSeg.q[0],startSeg.q[1],startSeg.q[2], duDarkenCol(col));
				dd->vertex(endSeg.p[0],endSeg.p[1],endSeg.p[2], duDarkenCol(col));
				dd->vertex(endSeg.q[0],endSeg.q[1],endSeg.q[2], duDarkenCol(col));
				dd->end();
				
				unsigned int colm = duColor::white;
				dd->begin(DU_DRAW_POINTS, 3.0f);
				dd->vertex(startSeg.p[0],startSeg.p[1],startSeg.p[2], colm);
				dd->vertex(startSeg.q[0],startSeg.q[1],startSeg.q[2], colm);
				dd->vertex(endSeg.p[0],endSeg.p[1],endSeg.p[2], colm);
				dd->vertex(endSeg.q[0],endSeg.q[1],endSeg.q[2], colm);
				dd->end();
			}
			
			if (drawFlags & DRAW_LAND_SAMPLES)
			{
				duDrawGroundSegment(dd, startSeg);
				duDrawGroundSegment(dd, endSeg);
			}

			const bool drawCollisionSlices = drawFlags & DRAW_COLLISION_SLICES;
			const bool drawCollisionSamples = drawFlags & DRAW_COLLISION_SAMPLES;
			if (drawCollisionSlices || drawCollisionSamples)
			{
				if (startSeg.ngsamples && startSeg.ngsamples == endSeg.ngsamples)
				{
					const int nsamples = startSeg.ngsamples;
					for (int i = 0; i < nsamples; ++i)
					{
						const dtNavLinkBuilder::GroundSample* ssmp = &startSeg.gsamples[i];
						const dtNavLinkBuilder::GroundSample* esmp = &endSeg.gsamples[i];
						if ((ssmp->flags & dtNavLinkBuilder::HAS_GROUND) == 0 || (esmp->flags & dtNavLinkBuilder::HAS_GROUND) == 0)
							continue;
						
						const float u = (float)i/(float)(nsamples-1);
						dtReal spt[3], ept[3];
						dtVlerp(spt, startSeg.p, startSeg.q, u);
						dtVlerp(ept, endSeg.p, endSeg.q, u);

						if (drawCollisionSlices)
						{
							if (ssmp->flags & dtNavLinkBuilder::UNRESTRICTED)
								UE::Detour::Private::drawTrajectorySlice(dd, spt, ept, &es->trajectory, es->az, duColor::green);
							else
								UE::Detour::Private::drawTrajectorySlice(dd, spt, ept, &es->trajectory, es->az, duColor::orangeRed);
						}

						if (drawCollisionSamples)
						{
							duDebugDrawTrajectorySamples(dd, linkBuilder, spt, ept, &es->trajectory, es->az);
						}
					}
				}
			}
		}
	}
}

void duDrawGroundSegment(duDebugDraw* dd, const dtNavLinkBuilder::GroundSegment& segment)
{
	for (int i = 0; i < segment.ngsamples; ++i)
	{
		constexpr float pointSize = 4.f;

		const dtNavLinkBuilder::GroundSample* s = &segment.gsamples[i];
		
		const float u = static_cast<float>(i) / static_cast<float>(segment.ngsamples - 1);
		dtReal spt[3];
		dtVlerp(spt, segment.p, segment.q, u);

		if (s->flags & dtNavLinkBuilder::HAS_GROUND)
		{
			dd->begin(DU_DRAW_POINTS, pointSize);
			dd->vertex(spt, duColor::lightGreen);
			dd->vertex(spt[0], s->height, spt[2], duColor::green);
			dd->end();

			constexpr float lineWidth = 2.f;
			dd->begin(DU_DRAW_LINES, lineWidth);
			dd->vertex(spt, duColor::lightGreen);
			dd->vertex(spt[0], s->height, spt[2], duColor::lightGreen);
			dd->end();
		}
		else
		{
			dd->begin(DU_DRAW_POINTS, pointSize);
			dd->vertex(spt, duColor::orangeRed);
			dd->end();
		}
	}
}
//@UE END
