// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DebugUtils/RecastDebugDraw.h"
#define _USE_MATH_DEFINES
#include "DebugUtils/DebugDraw.h"
#include "Recast/Recast.h"

void duDebugDrawTriMesh(duDebugDraw* dd, const duReal* verts, int /*nverts*/,
						const int* tris, const duReal* normals, int ntris,
						const unsigned char* flags, const duReal texScale)
{
	if (!dd) return;
	if (!verts) return;
	if (!tris) return;
	if (!normals) return;

	duReal uva[2];
	duReal uvb[2];
	duReal uvc[2];

	const unsigned int unwalkable = duRGBA(192,128,0,255);

	dd->texture(true);

	dd->begin(DU_DRAW_TRIS);
	for (int i = 0; i < ntris*3; i += 3)
	{
		const duReal* norm = &normals[i];
		unsigned int color;
		unsigned char a = (unsigned char)(220*(2+norm[0]+norm[1])/4);
		if (flags && !flags[i/3])
			color = duLerpCol(duRGBA(a,a,a,255), unwalkable, 64);
		else
			color = duRGBA(a,a,a,255);

		const duReal* va = &verts[tris[i+0]*3];
		const duReal* vb = &verts[tris[i+1]*3];
		const duReal* vc = &verts[tris[i+2]*3];
		
		int ax = 0, ay = 0;
		if (rcAbs(norm[1]) > rcAbs(norm[ax]))
			ax = 1;
		if (rcAbs(norm[2]) > rcAbs(norm[ax]))
			ax = 2;
		ax = (1<<ax)&3; // +1 mod 3
		ay = (1<<ax)&3; // +1 mod 3

		uva[0] = va[ax]*texScale;
		uva[1] = va[ay]*texScale;
		uvb[0] = vb[ax]*texScale;
		uvb[1] = vb[ay]*texScale;
		uvc[0] = vc[ax]*texScale;
		uvc[1] = vc[ay]*texScale;
		
		dd->vertex(va, color, uva);
		dd->vertex(vb, color, uvb);
		dd->vertex(vc, color, uvc);
	}
	dd->end();
	dd->texture(false);
}

void duDebugDrawTriMeshSlope(duDebugDraw* dd, const duReal* verts, int /*nverts*/,
							 const int* tris, const duReal* normals, int ntris,
							 const duReal walkableSlopeAngle, const duReal texScale)
{
	if (!dd) return;
	if (!verts) return;
	if (!tris) return;
	if (!normals) return;
	
	const duReal walkableThr = duCos(walkableSlopeAngle/180.0f*DU_PI);
	
	duReal uva[2];
	duReal uvb[2];
	duReal uvc[2];
	
	dd->texture(true);

	const unsigned int unwalkable = duRGBA(192,128,0,255);
	
	dd->begin(DU_DRAW_TRIS);
	for (int i = 0; i < ntris*3; i += 3)
	{
		const duReal* norm = &normals[i];
		unsigned int color;
		unsigned char a = (unsigned char)(220*(2+norm[0]+norm[1])/4);
		if (norm[1] < walkableThr)
			color = duLerpCol(duRGBA(a,a,a,255), unwalkable, 64);
		else
			color = duRGBA(a,a,a,255);
		
		const duReal* va = &verts[tris[i+0]*3];
		const duReal* vb = &verts[tris[i+1]*3];
		const duReal* vc = &verts[tris[i+2]*3];
		
		int ax = 0, ay = 0;
		if (duAbs(norm[1]) > duAbs(norm[ax]))
			ax = 1;
		if (duAbs(norm[2]) > duAbs(norm[ax]))
			ax = 2;
		ax = (1<<ax)&3; // +1 mod 3
		ay = (1<<ax)&3; // +1 mod 3
		
		uva[0] = va[ax]*texScale;
		uva[1] = va[ay]*texScale;
		uvb[0] = vb[ax]*texScale;
		uvb[1] = vb[ay]*texScale;
		uvc[0] = vc[ax]*texScale;
		uvc[1] = vc[ay]*texScale;
		
		dd->vertex(va, color, uva);
		dd->vertex(vb, color, uvb);
		dd->vertex(vc, color, uvc);
	}
	dd->end();

	dd->texture(false);
}

void duDebugDrawHeightfieldSolid(duDebugDraw* dd, const rcHeightfield& hf)
{
	if (!dd) return;

	const duReal* orig = hf.bmin;
	const duReal cs = hf.cs;
	const duReal ch = hf.ch;
	
	const int w = hf.width;
	const int h = hf.height;
		
	unsigned int fcol[6];
	duCalcBoxColors(fcol, duRGBA(255,255,255,255), duRGBA(255,255,255,255));
	
	dd->begin(DU_DRAW_QUADS);
	
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			duReal fx = orig[0] + x*cs;
			duReal fz = orig[2] + y*cs;
			const rcSpan* s = hf.spans[x + y*w];
			while (s)
			{
				duAppendBox(dd, fx, orig[1]+s->data.smin*ch, fz, fx+cs, orig[1] + s->data.smax*ch, fz+cs, fcol);
				s = s->next;
			}
		}
	}
	dd->end();
}

void duDebugDrawHeightfieldWalkable(duDebugDraw* dd, const rcHeightfield& hf)
{
	if (!dd) return;

	const duReal* orig = hf.bmin;
	const duReal cs = hf.cs;
	const duReal ch = hf.ch;
	
	const int w = hf.width;
	const int h = hf.height;
	
	unsigned int fcol[6];
	duCalcBoxColors(fcol, duRGBA(255,255,255,255), duRGBA(217,217,217,255));

	dd->begin(DU_DRAW_QUADS);
	
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			duReal fx = orig[0] + x*cs;
			duReal fz = orig[2] + y*cs;
			const rcSpan* s = hf.spans[x + y*w];
			while (s)
			{
				if (s->data.area == RC_WALKABLE_AREA)
					fcol[0] = duRGBA(64,128,160,255);
				else if (s->data.area == RC_NULL_AREA)
					fcol[0] = duRGBA(64,64,64,255);
				else
					fcol[0] = duMultCol(duIntToCol(s->data.area, 255), 200);
				
				duAppendBox(dd, fx, orig[1]+s->data.smin*ch, fz, fx+cs, orig[1] + s->data.smax*ch, fz+cs, fcol);
				s = s->next;
			}
		}
	}
	
	dd->end();
}

void duDebugDrawHeightfieldBounds(duDebugDraw* dd, const rcHeightfield& hf)
{
	if (!dd) return;

	const duReal hfMaxSpawnHeight = hf.bmin[1] + RC_SPAN_MAX_HEIGHT * hf.ch;

	const unsigned int hfBoundCol = duRGBA(160,160,160,255);

	dd->begin(DU_DRAW_LINES, 2.f);

	duAppendBoxWire(dd, hf.bmin[0], hf.bmin[1], hf.bmin[2], hf.bmax[0], hf.bmax[1], hf.bmax[2], hfBoundCol);

	if (hfMaxSpawnHeight < hf.bmax[1]) 
	{
		const unsigned int maxSpanBoundCol = duRGBA(255,20,20,255); 
		dd->vertex(hf.bmin[0], hfMaxSpawnHeight, hf.bmin[2], maxSpanBoundCol);
		dd->vertex(hf.bmax[0], hfMaxSpawnHeight, hf.bmin[2], maxSpanBoundCol);
		dd->vertex(hf.bmax[0], hfMaxSpawnHeight, hf.bmin[2], maxSpanBoundCol);
		dd->vertex(hf.bmax[0], hfMaxSpawnHeight, hf.bmax[2], maxSpanBoundCol);
		dd->vertex(hf.bmax[0], hfMaxSpawnHeight, hf.bmax[2], maxSpanBoundCol);
		dd->vertex(hf.bmin[0], hfMaxSpawnHeight, hf.bmax[2], maxSpanBoundCol);
		dd->vertex(hf.bmin[0], hfMaxSpawnHeight, hf.bmax[2], maxSpanBoundCol);
		dd->vertex(hf.bmin[0], hfMaxSpawnHeight, hf.bmin[2], maxSpanBoundCol);
	}

	dd->end();
}

void duDebugDrawCompactHeightfieldSolid(duDebugDraw* dd, const rcCompactHeightfield& chf)
{
	if (!dd) return;

	const duReal cs = chf.cs;
	const duReal ch = chf.ch;

	dd->begin(DU_DRAW_QUADS);
	
	for (int y = 0; y < chf.height; ++y)
	{
		for (int x = 0; x < chf.width; ++x)
		{
			const duReal fx = chf.bmin[0] + x*cs;
			const duReal fz = chf.bmin[2] + y*cs;
			const rcCompactCell& c = chf.cells[x+y*chf.width];

			for (unsigned i = c.index, ni = c.index+c.count; i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];

				unsigned int color;
				if (chf.areas[i] == RC_WALKABLE_AREA)
					color = duRGBA(0,192,255,64);
				else if (chf.areas[i] == RC_NULL_AREA)
					color = duRGBA(0,0,0,64);
				else
					color = duIntToCol(chf.areas[i], 255);
				
				const duReal fy = chf.bmin[1] + (s.y+1)*ch;
				dd->vertex(fx, fy, fz, color);
				dd->vertex(fx, fy, fz+cs, color);
				dd->vertex(fx+cs, fy, fz+cs, color);
				dd->vertex(fx+cs, fy, fz, color);
			}
		}
	}
	dd->end();
}

void duDebugDrawCompactHeightfieldRegions(duDebugDraw* dd, const rcCompactHeightfield& chf)
{
	if (!dd) return;

	const duReal cs = chf.cs;
	const duReal ch = chf.ch;

	dd->begin(DU_DRAW_QUADS);

	for (int y = 0; y < chf.height; ++y)
	{
		for (int x = 0; x < chf.width; ++x)
		{
			const duReal fx = chf.bmin[0] + x*cs;
			const duReal fz = chf.bmin[2] + y*cs;
			const rcCompactCell& c = chf.cells[x+y*chf.width];
			
			for (unsigned i = c.index, ni = c.index+c.count; i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				const duReal fy = chf.bmin[1] + (s.y)*ch;
				unsigned int color;
				if (s.reg)
					color = duIntToCol(s.reg, 192);
				else
					color = duRGBA(0,0,0,64);

				dd->vertex(fx, fy, fz, color);
				dd->vertex(fx, fy, fz+cs, color);
				dd->vertex(fx+cs, fy, fz+cs, color);
				dd->vertex(fx+cs, fy, fz, color);
			}
		}
	}
	
	dd->end();
}


void duDebugDrawCompactHeightfieldDistance(duDebugDraw* dd, const rcCompactHeightfield& chf)
{
	if (!dd) return;
	if (!chf.dist) return;
		
	const duReal cs = chf.cs;
	const duReal ch = chf.ch;
			
	duReal maxd = chf.maxDistance;
	if (maxd < 1.0f) maxd = 1;
	const duReal dscale = 255.0f / maxd;
	
	dd->begin(DU_DRAW_QUADS);
	
	for (int y = 0; y < chf.height; ++y)
	{
		for (int x = 0; x < chf.width; ++x)
		{
			const duReal fx = chf.bmin[0] + x*cs;
			const duReal fz = chf.bmin[2] + y*cs;
			const rcCompactCell& c = chf.cells[x+y*chf.width];
			
			for (unsigned i = c.index, ni = c.index+c.count; i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				const duReal fy = chf.bmin[1] + (s.y+1)*ch;
				const unsigned char cd = (unsigned char)(chf.dist[i] * dscale);
				const unsigned int color = duRGBA(cd,cd,cd,255);
				dd->vertex(fx, fy, fz, color);
				dd->vertex(fx, fy, fz+cs, color);
				dd->vertex(fx+cs, fy, fz+cs, color);
				dd->vertex(fx+cs, fy, fz, color);
			}
		}
	}
	dd->end();
}

static void drawLayerPortals(duDebugDraw* dd, const rcHeightfieldLayer* layer)
{
	const duReal cs = layer->cs;
	const duReal ch = layer->ch;
	const int w = layer->width;
	const int h = layer->height;
	
	unsigned int pcol = duRGBA(255,255,255,255);
	
	const int segs[4*4] = {0,0,0,1, 0,1,1,1, 1,1,1,0, 1,0,0,0};
	
	// Layer portals
	dd->begin(DU_DRAW_LINES, 2.0f);
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int idx = x+y*w;
			const int lh = (int)layer->heights[idx];
			if (lh == 0xffff) continue;
			
			for (int dir = 0; dir < 4; ++dir)
			{
				if (layer->cons[idx] & (1<<(dir+4)))
				{
					const int* seg = &segs[dir*4];
					const duReal ax = layer->bmin[0] + (x+seg[0])*cs;
					const duReal ay = layer->bmin[1] + (lh+2)*ch;
					const duReal az = layer->bmin[2] + (y+seg[1])*cs;
					const duReal bx = layer->bmin[0] + (x+seg[2])*cs;
					const duReal by = layer->bmin[1] + (lh+2)*ch;
					const duReal bz = layer->bmin[2] + (y+seg[3])*cs;
					dd->vertex(ax, ay, az, pcol);
					dd->vertex(bx, by, bz, pcol);
				}
			}
		}
	}
	dd->end();
}

void duDebugDrawHeightfieldLayer(duDebugDraw* dd, const struct rcHeightfieldLayer& layer, const int idx)
{
	const duReal cs = layer.cs;
	const duReal ch = layer.ch;
	const int w = layer.width;
	const int h = layer.height;
	
	unsigned int color = duIntToCol(idx+1, 255);
	
	// Layer bounds
	duReal bmin[3], bmax[3];
	bmin[0] = layer.bmin[0] + layer.minx*cs;
	bmin[1] = layer.bmin[1];
	bmin[2] = layer.bmin[2] + layer.miny*cs;
	bmax[0] = layer.bmin[0] + (layer.maxx+1)*cs;
	bmax[1] = layer.bmax[1];
	bmax[2] = layer.bmin[2] + (layer.maxy+1)*cs;
	duDebugDrawBoxWire(dd, bmin[0],bmin[1],bmin[2], bmax[0],bmax[1],bmax[2], duTransCol(color,128), 2.0f);
	
	// Layer height
	dd->begin(DU_DRAW_QUADS);
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int lidx = x+y*w;
			const int lh = (int)layer.heights[lidx];
			if (lh == 0xffff) continue;
			const unsigned char area = layer.areas[lidx];
			
			unsigned int col;
			if (area == RC_WALKABLE_AREA)
				col = duLerpCol(color, duRGBA(0,192,255,64), 32);
			else if (area == RC_NULL_AREA)
				col = duLerpCol(color, duRGBA(0,0,0,64), 32);
			else
				col = duLerpCol(color, duIntToCol(area, 255), 32);
			
			const duReal fx = layer.bmin[0] + x*cs;
			const duReal fy = layer.bmin[1] + (lh+1)*ch;
			const duReal fz = layer.bmin[2] + y*cs;
			
			dd->vertex(fx, fy, fz, col);
			dd->vertex(fx, fy, fz+cs, col);
			dd->vertex(fx+cs, fy, fz+cs, col);
			dd->vertex(fx+cs, fy, fz, col);
		}
	}
	dd->end();
	
	// Portals
	drawLayerPortals(dd, &layer);
}

void duDebugDrawHeightfieldLayers(duDebugDraw* dd, const struct rcHeightfieldLayerSet& lset)
{
	if (!dd) return;
	for (int i = 0; i < lset.nlayers; ++i)
		duDebugDrawHeightfieldLayer(dd, lset.layers[i], i);
}

/*
void duDebugDrawLayerContours(duDebugDraw* dd, const struct rcLayerContourSet& lcset)
{
	if (!dd) return;
	
	const float* orig = lcset.bmin;
	const float cs = lcset.cs;
	const float ch = lcset.ch;
	
	const unsigned char a = 255;// (unsigned char)(alpha*255.0f);
	
	const int offs[2*4] = {-1,0, 0,1, 1,0, 0,-1};

	dd->begin(DU_DRAW_LINES, 2.0f);
	
	for (int i = 0; i < lcset.nconts; ++i)
	{
		const rcLayerContour& c = lcset.conts[i];
		unsigned int color = 0;

		color = duIntToCol(i, a);

		for (int j = 0; j < c.nverts; ++j)
		{
			const int k = (j+1) % c.nverts;
			const unsigned char* va = &c.verts[j*4];
			const unsigned char* vb = &c.verts[k*4];
			const float ax = orig[0] + va[0]*cs;
			const float ay = orig[1] + (va[1]+1+(i&1))*ch;
			const float az = orig[2] + va[2]*cs;
			const float bx = orig[0] + vb[0]*cs;
			const float by = orig[1] + (vb[1]+1+(i&1))*ch;
			const float bz = orig[2] + vb[2]*cs;
			unsigned int col = color;
			if ((va[3] & 0xf) != 0xf)
			{
				col = duRGBA(255,255,255,128);
				int d = va[3] & 0xf;
				
				const float cx = (ax+bx)*0.5f;
				const float cy = (ay+by)*0.5f;
				const float cz = (az+bz)*0.5f;
				
				const float dx = cx + offs[d*2+0]*2*cs;
				const float dy = cy;
				const float dz = cz + offs[d*2+1]*2*cs;
				
				dd->vertex(cx,cy,cz,duRGBA(255,0,0,255));
				dd->vertex(dx,dy,dz,duRGBA(255,0,0,255));
			}
			
			duAppendArrow(dd, ax,ay,az, bx,by,bz, 0.0f, cs*0.5f, col);
		}
	}
	dd->end();
	
	dd->begin(DU_DRAW_POINTS, 4.0f);	
	
	for (int i = 0; i < lcset.nconts; ++i)
	{
		const rcLayerContour& c = lcset.conts[i];
		unsigned int color = 0;
		
		for (int j = 0; j < c.nverts; ++j)
		{
			const unsigned char* va = &c.verts[j*4];

			color = duDarkenCol(duIntToCol(i, a));
			if (va[3] & 0x80)
				color = duRGBA(255,0,0,255);

			float fx = orig[0] + va[0]*cs;
			float fy = orig[1] + (va[1]+1+(i&1))*ch;
			float fz = orig[2] + va[2]*cs;
			dd->vertex(fx,fy,fz, color);
		}
	}
	dd->end();
}

void duDebugDrawLayerPolyMesh(duDebugDraw* dd, const struct rcLayerPolyMesh& lmesh)
{
	if (!dd) return;
	
	const int nvp = lmesh.nvp;
	const float cs = lmesh.cs;
	const float ch = lmesh.ch;
	const float* orig = lmesh.bmin;
	
	const int offs[2*4] = {-1,0, 0,1, 1,0, 0,-1};

	dd->begin(DU_DRAW_TRIS);
	
	for (int i = 0; i < lmesh.npolys; ++i)
	{
		const unsigned short* p = &lmesh.polys[i*nvp*2];
		
		unsigned int color;
		if (lmesh.areas[i] == RC_WALKABLE_AREA)
			color = duRGBA(0,192,255,64);
		else if (lmesh.areas[i] == RC_NULL_AREA)
			color = duRGBA(0,0,0,64);
		else
			color = duIntToCol(lmesh.areas[i], 255);
		
		unsigned short vi[3];
		for (int j = 2; j < nvp; ++j)
		{
			if (p[j] == RC_MESH_NULL_IDX) break;
			vi[0] = p[0];
			vi[1] = p[j-1];
			vi[2] = p[j];
			for (int k = 0; k < 3; ++k)
			{
				const unsigned short* v = &lmesh.verts[vi[k]*3];
				const float x = orig[0] + v[0]*cs;
				const float y = orig[1] + (v[1]+1)*ch;
				const float z = orig[2] + v[2]*cs;
				dd->vertex(x,y,z, color);
			}
		}
	}
	dd->end();
	
	// Draw neighbours edges
	const unsigned int coln = duRGBA(0,48,64,32);
	dd->begin(DU_DRAW_LINES, 1.5f);
	for (int i = 0; i < lmesh.npolys; ++i)
	{
		const unsigned short* p = &lmesh.polys[i*nvp*2];
		for (int j = 0; j < nvp; ++j)
		{
			if (p[j] == RC_MESH_NULL_IDX) break;
			if (p[nvp+j] & 0x8000) continue;
			const int nj = (j+1 >= nvp || p[j+1] == RC_MESH_NULL_IDX) ? 0 : j+1; 
			int vi[2] = {p[j], p[nj]};
			
			for (int k = 0; k < 2; ++k)
			{
				const unsigned short* v = &lmesh.verts[vi[k]*3];
				const float x = orig[0] + v[0]*cs;
				const float y = orig[1] + (v[1]+1)*ch + 0.1f;
				const float z = orig[2] + v[2]*cs;
				dd->vertex(x, y, z, coln);
			}
		}
	}
	dd->end();
	
	// Draw boundary edges
	const unsigned int colb = duRGBA(0,48,64,220);
	dd->begin(DU_DRAW_LINES, 2.5f);
	for (int i = 0; i < lmesh.npolys; ++i)
	{
		const unsigned short* p = &lmesh.polys[i*nvp*2];
		for (int j = 0; j < nvp; ++j)
		{
			if (p[j] == RC_MESH_NULL_IDX) break;
			if ((p[nvp+j] & 0x8000) == 0) continue;
			const int nj = (j+1 >= nvp || p[j+1] == RC_MESH_NULL_IDX) ? 0 : j+1; 
			int vi[2] = {p[j], p[nj]};
			
			unsigned int col = colb;
			if ((p[nvp+j] & 0xf) != 0xf)
			{
				const unsigned short* va = &lmesh.verts[vi[0]*3];
				const unsigned short* vb = &lmesh.verts[vi[1]*3];

				const float ax = orig[0] + va[0]*cs;
				const float ay = orig[1] + (va[1]+1+(i&1))*ch;
				const float az = orig[2] + va[2]*cs;
				const float bx = orig[0] + vb[0]*cs;
				const float by = orig[1] + (vb[1]+1+(i&1))*ch;
				const float bz = orig[2] + vb[2]*cs;
				
				const float cx = (ax+bx)*0.5f;
				const float cy = (ay+by)*0.5f;
				const float cz = (az+bz)*0.5f;
				
				int d = p[nvp+j] & 0xf;
				
				const float dx = cx + offs[d*2+0]*2*cs;
				const float dy = cy;
				const float dz = cz + offs[d*2+1]*2*cs;
				
				dd->vertex(cx,cy,cz,duRGBA(255,0,0,255));
				dd->vertex(dx,dy,dz,duRGBA(255,0,0,255));
				
				col = duRGBA(255,255,255,128);
			}
							 
			for (int k = 0; k < 2; ++k)
			{
				const unsigned short* v = &lmesh.verts[vi[k]*3];
				const float x = orig[0] + v[0]*cs;
				const float y = orig[1] + (v[1]+1)*ch + 0.1f;
				const float z = orig[2] + v[2]*cs;
				dd->vertex(x, y, z, col);
			}
		}
	}
	dd->end();
	
	dd->begin(DU_DRAW_POINTS, 3.0f);
	const unsigned int colv = duRGBA(0,0,0,220);
	for (int i = 0; i < lmesh.nverts; ++i)
	{
		const unsigned short* v = &lmesh.verts[i*3];
		const float x = orig[0] + v[0]*cs;
		const float y = orig[1] + (v[1]+1)*ch + 0.1f;
		const float z = orig[2] + v[2]*cs;
		dd->vertex(x,y,z, colv);
	}
	dd->end();
}
*/

namespace RecastDrawFunc
{
	static void getContourCenter(const rcContour* cont, const duReal* orig, duReal cs, duReal ch, duReal* center)
	{
		center[0] = 0;
		center[1] = 0;
		center[2] = 0;
		if (!cont->nverts)
			return;
		for (int i = 0; i < cont->nverts; ++i)
		{
			const int* v = &cont->verts[i * 4];
			center[0] += (duReal)v[0];
			center[1] += (duReal)v[1];
			center[2] += (duReal)v[2];
		}
		const duReal s = duReal(1.) / cont->nverts;
		center[0] *= s * cs;
		center[1] *= s * ch;
		center[2] *= s * cs;
		center[0] += orig[0];
		center[1] += orig[1] + 4 * ch;
		center[2] += orig[2];
	}

	static const rcContour* findContourFromSet(const rcContourSet& cset, unsigned short reg)
	{
		for (int i = 0; i < cset.nconts; ++i)
		{
			if (cset.conts[i].reg == reg)
				return &cset.conts[i];
		}
		return 0;
	}
}

void duDebugDrawRegionConnections(duDebugDraw* dd, const rcContourSet& cset, const float alpha)
{
	if (!dd) return;
	
	const duReal* orig = cset.bmin;
	const duReal cs = cset.cs;
	const duReal ch = cset.ch;
	
	// Draw centers
	duReal pos[3], pos2[3];

	unsigned int color = duRGBA(0,0,0,196);

	dd->begin(DU_DRAW_LINES, 2.0f);

	for (int i = 0; i < cset.nconts; ++i)
	{
		const rcContour* cont = &cset.conts[i];
		RecastDrawFunc::getContourCenter(cont, orig, cs, ch, pos);
		for (int j = 0; j < cont->nverts; ++j)
		{
			const int* v = &cont->verts[j*4];
			if (v[3] == 0 || (unsigned short)v[3] < cont->reg) continue;
			const rcContour* cont2 = RecastDrawFunc::findContourFromSet(cset, (unsigned short)v[3]);
			if (cont2)
			{
				RecastDrawFunc::getContourCenter(cont2, orig, cs, ch, pos2);
				duAppendArc(dd, pos[0],pos[1],pos[2], pos2[0],pos2[1],pos2[2], 0.25f, 0.6f, 0.6f, color);
			}
		}
	}
	
	dd->end();

	unsigned char a = (unsigned char)(alpha * 255.0f);

	dd->begin(DU_DRAW_POINTS, 7.0f);

	for (int i = 0; i < cset.nconts; ++i)
	{
		const rcContour* cont = &cset.conts[i];
		unsigned int col = duDarkenCol(duIntToCol(cont->reg,a));
		RecastDrawFunc::getContourCenter(cont, orig, cs, ch, pos);
		dd->vertex(pos, col);
	}
	dd->end();
}

void duDebugDrawRawContours(duDebugDraw* dd, const rcContourSet& cset, const float alpha)
{
	if (!dd) return;

	const duReal* orig = cset.bmin;
	const duReal cs = cset.cs;
	const duReal ch = cset.ch;
	
	const unsigned char a = (unsigned char)(alpha*255.0f);
	
	dd->begin(DU_DRAW_LINES, 2.0f);
			
	for (int i = 0; i < cset.nconts; ++i)
	{
		const rcContour& c = cset.conts[i];
		unsigned int color = duIntToCol(c.reg, a);

		for (int j = 0; j < c.nrverts; ++j)
		{
			const int* v = &c.rverts[j*4];
			duReal fx = orig[0] + v[0]*cs;
			duReal fy = orig[1] + (v[1]+1+(i&1))*ch;
			duReal fz = orig[2] + v[2]*cs;
			dd->vertex(fx,fy,fz,color);
			if (j > 0)
				dd->vertex(fx,fy,fz,color);
		}
		// Loop last segment.
		const int* v = &c.rverts[0];
		duReal fx = orig[0] + v[0]*cs;
		duReal fy = orig[1] + (v[1]+1+(i&1))*ch;
		duReal fz = orig[2] + v[2]*cs;
		dd->vertex(fx,fy,fz,color);
	}
	dd->end();

	dd->begin(DU_DRAW_POINTS, 2.0f);	

	for (int i = 0; i < cset.nconts; ++i)
	{
		const rcContour& c = cset.conts[i];
		unsigned int color = duDarkenCol(duIntToCol(c.reg, a));
		
		for (int j = 0; j < c.nrverts; ++j)
		{
			const int* v = &c.rverts[j*4];
			duReal off = 0;
			unsigned int colv = color;
			if (v[3] & RC_BORDER_VERTEX)
			{
				colv = duRGBA(255,255,255,a);
				off = ch*2;
			}
			
			duReal fx = orig[0] + v[0]*cs;
			duReal fy = orig[1] + (v[1]+1+(i&1))*ch + off;
			duReal fz = orig[2] + v[2]*cs;
			dd->vertex(fx,fy,fz, colv);
		}
	}
	dd->end();
}

void duDebugDrawContours(duDebugDraw* dd, const rcContourSet& cset, const float alpha)
{
	if (!dd) return;

	const duReal* orig = cset.bmin;
	const duReal cs = cset.cs;
	const duReal ch = cset.ch;
	
	const unsigned char a = (unsigned char)(alpha*255.0f);
	
	dd->begin(DU_DRAW_LINES, 2.5f);
	
	for (int i = 0; i < cset.nconts; ++i)
	{
		const rcContour& c = cset.conts[i];
		if (!c.nverts)
			continue;
		const unsigned int color = duIntToCol(c.reg, a);
		const unsigned int bcolor = duLerpCol(color,duRGBA(255,255,255,a),128);
		for (int j = 0, k = c.nverts-1; j < c.nverts; k=j++)
		{
			const int* va = &c.verts[k*4];
			const int* vb = &c.verts[j*4];
			unsigned int col = (va[3] & RC_AREA_BORDER) ? bcolor : color; 
			duReal fx,fy,fz;
			fx = orig[0] + va[0]*cs;
			fy = orig[1] + (va[1]+1+(i&1))*ch;
			fz = orig[2] + va[2]*cs;
			dd->vertex(fx,fy,fz, col);
			fx = orig[0] + vb[0]*cs;
			fy = orig[1] + (vb[1]+1+(i&1))*ch;
			fz = orig[2] + vb[2]*cs;
			dd->vertex(fx,fy,fz, col);
		}
	}
	dd->end();

	dd->begin(DU_DRAW_POINTS, 3.0f);
	
	for (int i = 0; i < cset.nconts; ++i)
	{
		const rcContour& c = cset.conts[i];
		unsigned int color = duDarkenCol(duIntToCol(c.reg, a));
		for (int j = 0; j < c.nverts; ++j)
		{
			const int* v = &c.verts[j*4];
			duReal off = 0;
			unsigned int colv = color;
			if (v[3] & RC_BORDER_VERTEX)
			{
				colv = duRGBA(255,255,255,a);
				off = ch*2;
			}

			duReal fx = orig[0] + v[0]*cs;
			duReal fy = orig[1] + (v[1]+1+(i&1))*ch + off;
			duReal fz = orig[2] + v[2]*cs;
			dd->vertex(fx,fy,fz, colv);
		}
	}
	dd->end();
}

void duDebugDrawPolyMesh(duDebugDraw* dd, const struct rcPolyMesh& mesh)
{
	if (!dd) return;

	const int nvp = mesh.nvp;
	const duReal cs = mesh.cs;
	const duReal ch = mesh.ch;
	const duReal* orig = mesh.bmin;
	
	dd->begin(DU_DRAW_TRIS);
	
	for (int i = 0; i < mesh.npolys; ++i)
	{
		const unsigned short* p = &mesh.polys[i*nvp*2];
		
		unsigned int color;
		if (mesh.areas[i] == RC_WALKABLE_AREA)
			color = duRGBA(0,192,255,64);
		else if (mesh.areas[i] == RC_NULL_AREA)
			color = duRGBA(0,0,0,64);
		else
			color = duIntToCol(mesh.areas[i], 255);
		
		unsigned short vi[3];
		for (int j = 2; j < nvp; ++j)
		{
			if (p[j] == RC_MESH_NULL_IDX) break;
			vi[0] = p[0];
			vi[1] = p[j-1];
			vi[2] = p[j];
			for (int k = 0; k < 3; ++k)
			{
				const unsigned short* v = &mesh.verts[vi[k]*3];
				const duReal x = orig[0] + v[0]*cs;
				const duReal y = orig[1] + (v[1]+1)*ch;
				const duReal z = orig[2] + v[2]*cs;
				dd->vertex(x,y,z, color);
			}
		}
	}
	dd->end();

	// Draw neighbours edges
	const unsigned int coln = duRGBA(0,48,64,32);
	dd->begin(DU_DRAW_LINES, 1.5f);
	for (int i = 0; i < mesh.npolys; ++i)
	{
		const unsigned short* p = &mesh.polys[i*nvp*2];
		for (int j = 0; j < nvp; ++j)
		{
			if (p[j] == RC_MESH_NULL_IDX) break;
			if (p[nvp+j] & 0x8000) continue;
			const int nj = (j+1 >= nvp || p[j+1] == RC_MESH_NULL_IDX) ? 0 : j+1; 
			const int vi[2] = {p[j], p[nj]};
			
			for (int k = 0; k < 2; ++k)
			{
				const unsigned short* v = &mesh.verts[vi[k]*3];
				const duReal x = orig[0] + v[0]*cs;
				const duReal y = orig[1] + (v[1]+1)*ch + 0.1f;
				const duReal z = orig[2] + v[2]*cs;
				dd->vertex(x, y, z, coln);
			}
		}
	}
	dd->end();
	
	// Draw boundary edges
	const unsigned int colb = duRGBA(0,48,64,220);
	dd->begin(DU_DRAW_LINES, 2.5f);
	for (int i = 0; i < mesh.npolys; ++i)
	{
		const unsigned short* p = &mesh.polys[i*nvp*2];
		for (int j = 0; j < nvp; ++j)
		{
			if (p[j] == RC_MESH_NULL_IDX) break;
			if ((p[nvp+j] & 0x8000) == 0) continue;
			const int nj = (j+1 >= nvp || p[j+1] == RC_MESH_NULL_IDX) ? 0 : j+1; 
			const int vi[2] = {p[j], p[nj]};
			
			unsigned int col = colb;
			if ((p[nvp+j] & 0xf) != 0xf)
				col = duRGBA(255,255,255,128);
			for (int k = 0; k < 2; ++k)
			{
				const unsigned short* v = &mesh.verts[vi[k]*3];
				const duReal x = orig[0] + v[0]*cs;
				const duReal y = orig[1] + (v[1]+1)*ch + 0.1f;
				const duReal z = orig[2] + v[2]*cs;
				dd->vertex(x, y, z, col);
			}
		}
	}
	dd->end();
	
	dd->begin(DU_DRAW_POINTS, 3.0f);
	const unsigned int colv = duRGBA(0,0,0,220);
	for (int i = 0; i < mesh.nverts; ++i)
	{
		const unsigned short* v = &mesh.verts[i*3];
		const duReal x = orig[0] + v[0]*cs;
		const duReal y = orig[1] + (v[1]+1)*ch + 0.1f;
		const duReal z = orig[2] + v[2]*cs;
		dd->vertex(x,y,z, colv);
	}
	dd->end();
}

void duDebugDrawPolyMeshDetail(duDebugDraw* dd, const struct rcPolyMeshDetail& dmesh)
{
	if (!dd) return;

	dd->begin(DU_DRAW_TRIS);
	
	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const unsigned int btris = m[2];
		const int ntris = (int)m[3];
		const duReal* verts = &dmesh.verts[bverts*3];
		const unsigned char* tris = &dmesh.tris[btris*4];

		unsigned int color = duIntToCol(i, 192);

		for (int j = 0; j < ntris; ++j)
		{
			dd->vertex(&verts[tris[j*4+0]*3], color);
			dd->vertex(&verts[tris[j*4+1]*3], color);
			dd->vertex(&verts[tris[j*4+2]*3], color);
		}
	}
	dd->end();

	// Internal edges.
	dd->begin(DU_DRAW_LINES, 1.0f);
	const unsigned int coli = duRGBA(0,0,0,64);
	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const unsigned int btris = m[2];
		const int ntris = (int)m[3];
		const duReal* verts = &dmesh.verts[bverts*3];
		const unsigned char* tris = &dmesh.tris[btris*4];
		
		for (int j = 0; j < ntris; ++j)
		{
			const unsigned char* t = &tris[j*4];
			for (int k = 0, kp = 2; k < 3; kp=k++)
			{
				unsigned char ef = (t[3] >> (kp*2)) & 0x3;
				if (ef == 0)
				{
					// Internal edge
					if (t[kp] < t[k])
					{
						dd->vertex(&verts[t[kp]*3], coli);
						dd->vertex(&verts[t[k]*3], coli);
					}
				}
			}
		}
	}
	dd->end();
	
	// External edges.
	dd->begin(DU_DRAW_LINES, 2.0f);
	const unsigned int cole = duRGBA(0,0,0,64);
	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const unsigned int btris = m[2];
		const int ntris = (int)m[3];
		const duReal* verts = &dmesh.verts[bverts*3];
		const unsigned char* tris = &dmesh.tris[btris*4];
		
		for (int j = 0; j < ntris; ++j)
		{
			const unsigned char* t = &tris[j*4];
			for (int k = 0, kp = 2; k < 3; kp=k++)
			{
				unsigned char ef = (t[3] >> (kp*2)) & 0x3;
				if (ef != 0)
				{
					// Ext edge
					dd->vertex(&verts[t[kp]*3], cole);
					dd->vertex(&verts[t[k]*3], cole);
				}
			}
		}
	}
	dd->end();
	
	dd->begin(DU_DRAW_POINTS, 3.0f);
	const unsigned int colv = duRGBA(0,0,0,64);
	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const int nverts = (int)m[1];
		const duReal* verts = &dmesh.verts[bverts*3];
		for (int j = 0; j < nverts; ++j)
			dd->vertex(&verts[j*3], colv);
	}
	dd->end();
}

void duDrawCylinderArea(duDebugDraw* dd, const duReal* pos, const duReal radius, const duReal height, const unsigned int areaRGBAColor)
{
	duDebugDrawCylinderWire(dd, pos[0] - radius, pos[1], pos[2] - radius, pos[0] + radius, pos[1] + height, pos[2] + radius, duRGBA(areaRGBAColor), 4.f);
}

void duDrawBoxArea(duDebugDraw* dd, const duReal* pos, const duReal* extent, const unsigned int areaRGBAColor)
{
	duDebugDrawBoxWire(dd, pos[0] - extent[0], pos[1] - extent[1], pos[2] - extent[2], pos[0] + extent[0], pos[1] + extent[1], pos[2] + extent[2], duRGBA(areaRGBAColor), 4.f);
}

void duDrawConvexArea(duDebugDraw* dd, const duReal* verts, const int nverts, const duReal hmin, const duReal hmax, const unsigned int areaRGBAColor)
{
	dd->begin(DU_DRAW_LINES, 4.0f);

	const unsigned int areaColor = duRGBA(areaRGBAColor);
	for (int i = 0; i < nverts; ++i)
	{
		const duReal* vert = &verts[i*3];

		// vertical line
		dd->vertex(vert[0], hmin, vert[2], areaColor);
		dd->vertex(vert[0], hmax, vert[2], areaColor);

		// horizontal lines
		int ni = (i+1)%nverts;
		const duReal* nextVert = &verts[ni*3];
		dd->vertex(vert[0], hmin, vert[2], areaColor);
		dd->vertex(nextVert[0], hmin, nextVert[2], areaColor);
		dd->vertex(vert[0], hmax, vert[2], areaColor);
		dd->vertex(nextVert[0], hmax, nextVert[2], areaColor);
	}
	dd->end();
}
