// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DebugUtils/RecastDump.h"
#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAlloc.h"
#include "DebugUtils/DebugDrawLargeWorldCoordinates.h"

duFileIO::~duFileIO()
{
	// Empty
}
	
static void ioprintf(duFileIO* io, const char* format, ...)
{
	char line[256];
	va_list ap;
	va_start(ap, format);
	const int n = FCStringAnsi::GetVarArgs(line, UE_ARRAY_COUNT(line), format, ap);
	va_end(ap);
	if (n > 0)
		io->write(line, sizeof(char)*n);
}

bool duDumpPolyMeshToObj(rcPolyMesh& pmesh, duFileIO* io)
{
	if (!io)
	{
		printf("duDumpPolyMeshToObj: input IO is null.\n"); 
		return false;
	}
	if (!io->isWriting())
	{
		printf("duDumpPolyMeshToObj: input IO not writing.\n"); 
		return false;
	}
	
	const int nvp = pmesh.nvp;
	const duReal cs = pmesh.cs;
	const duReal ch = pmesh.ch;
	const duReal* orig = pmesh.bmin;
	
	ioprintf(io, "# Recast Navmesh\n");
	ioprintf(io, "o NavMesh\n");

	ioprintf(io, "\n");
	
	for (int i = 0; i < pmesh.nverts; ++i)
	{
		const unsigned short* v = &pmesh.verts[i*3];
		const duReal x = orig[0] + v[0]*cs;
		const duReal y = orig[1] + (v[1]+1)*ch + 0.1f;
		const duReal z = orig[2] + v[2]*cs;
		ioprintf(io, "v %f %f %f\n", x,y,z);
	}

	ioprintf(io, "\n");

	for (int i = 0; i < pmesh.npolys; ++i)
	{
		const unsigned short* p = &pmesh.polys[i*nvp*2];
		for (int j = 2; j < nvp; ++j)
		{
			if (p[j] == RC_MESH_NULL_IDX) break;
			ioprintf(io, "f %d %d %d\n", p[0]+1, p[j-1]+1, p[j]+1); 
		}
	}
	
	return true;
}

bool duDumpPolyMeshDetailToObj(rcPolyMeshDetail& dmesh, duFileIO* io)
{
	if (!io)
	{
		printf("duDumpPolyMeshDetailToObj: input IO is null.\n"); 
		return false;
	}
	if (!io->isWriting())
	{
		printf("duDumpPolyMeshDetailToObj: input IO not writing.\n"); 
		return false;
	}
	
	ioprintf(io, "# Recast Navmesh\n");
	ioprintf(io, "o NavMesh\n");
	
	ioprintf(io, "\n");

	for (int i = 0; i < dmesh.nverts; ++i)
	{
		const duReal* v = &dmesh.verts[i*3];
		ioprintf(io, "v %f %f %f\n", v[0],v[1],v[2]);
	}
	
	ioprintf(io, "\n");
	
	for (int i = 0; i < dmesh.nmeshes; ++i)
	{
		const unsigned int* m = &dmesh.meshes[i*4];
		const unsigned int bverts = m[0];
		const unsigned int btris = m[2];
		const unsigned int ntris = m[3];
		const unsigned char* tris = &dmesh.tris[btris*4];
		for (unsigned int j = 0; j < ntris; ++j)
		{
			ioprintf(io, "f %d %d %d\n",
					(int)(bverts+tris[j*4+0])+1,
					(int)(bverts+tris[j*4+1])+1,
					(int)(bverts+tris[j*4+2])+1);
		}
	}
	
	return true;
}

static const int CSET_MAGIC = ('c' << 24) | ('s' << 16) | ('e' << 8) | 't';
static const int CSET_VERSION = 2;

bool duDumpContourSet(struct rcContourSet& cset, duFileIO* io)
{
	if (!io)
	{
		printf("duDumpContourSet: input IO is null.\n"); 
		return false;
	}
	if (!io->isWriting())
	{
		printf("duDumpContourSet: input IO not writing.\n"); 
		return false;
	}
	
	io->write(&CSET_MAGIC, sizeof(CSET_MAGIC));
	io->write(&CSET_VERSION, sizeof(CSET_VERSION));

	io->write(&cset.nconts, sizeof(cset.nconts));
	
	io->write(cset.bmin, sizeof(cset.bmin));
	io->write(cset.bmax, sizeof(cset.bmax));
	
	io->write(&cset.cs, sizeof(cset.cs));
	io->write(&cset.ch, sizeof(cset.ch));

	io->write(&cset.width, sizeof(cset.width));
	io->write(&cset.height, sizeof(cset.height));
	io->write(&cset.borderSize, sizeof(cset.borderSize));

	for (int i = 0; i < cset.nconts; ++i)
	{
		const rcContour& cont = cset.conts[i];
		io->write(&cont.nverts, sizeof(cont.nverts));
		io->write(&cont.nrverts, sizeof(cont.nrverts));
		io->write(&cont.reg, sizeof(cont.reg));
		io->write(&cont.area, sizeof(cont.area));
		io->write(cont.verts, sizeof(int)*4*cont.nverts);
		io->write(cont.rverts, sizeof(int)*4*cont.nrverts);
	}

	return true;
}

bool duReadContourSet(struct rcContourSet& cset, duFileIO* io)
{
	if (!io)
	{
		printf("duReadContourSet: input IO is null.\n"); 
		return false;
	}
	if (!io->isReading())
	{
		printf("duReadContourSet: input IO not reading.\n"); 
		return false;
	}
	
	int magic = 0;
	int version = 0;
	
	io->read(&magic, sizeof(magic));
	io->read(&version, sizeof(version));
	
	if (magic != CSET_MAGIC)
	{
		printf("duReadContourSet: Bad voodoo.\n");
		return false;
	}
	if (version != CSET_VERSION)
	{
		printf("duReadContourSet: Bad version.\n");
		return false;
	}
	
	io->read(&cset.nconts, sizeof(cset.nconts));

	cset.conts = (rcContour*)rcAlloc(sizeof(rcContour)*cset.nconts, RC_ALLOC_PERM);
	if (!cset.conts)
	{
		printf("duReadContourSet: Could not alloc contours (%d)\n", cset.nconts);
		return false;
	}
	memset(cset.conts, 0, sizeof(rcContour)*cset.nconts);
	
	io->read(cset.bmin, sizeof(cset.bmin));
	io->read(cset.bmax, sizeof(cset.bmax));
	
	io->read(&cset.cs, sizeof(cset.cs));
	io->read(&cset.ch, sizeof(cset.ch));
	
	io->read(&cset.width, sizeof(cset.width));
	io->read(&cset.height, sizeof(cset.height));
	io->read(&cset.borderSize, sizeof(cset.borderSize));
	
	for (int i = 0; i < cset.nconts; ++i)
	{
		rcContour& cont = cset.conts[i];
		io->read(&cont.nverts, sizeof(cont.nverts));
		io->read(&cont.nrverts, sizeof(cont.nrverts));
		io->read(&cont.reg, sizeof(cont.reg));
		io->read(&cont.area, sizeof(cont.area));

		cont.verts = (int*)rcAlloc(sizeof(int)*4*cont.nverts, RC_ALLOC_PERM);
		if (!cont.verts)
		{
			printf("duReadContourSet: Could not alloc contour verts (%d)\n", cont.nverts);
			return false;
		}
		cont.rverts = (int*)rcAlloc(sizeof(int)*4*cont.nrverts, RC_ALLOC_PERM);
		if (!cont.rverts)
		{
			printf("duReadContourSet: Could not alloc contour rverts (%d)\n", cont.nrverts);
			return false;
		}
		
		io->read(cont.verts, sizeof(int)*4*cont.nverts);
		io->read(cont.rverts, sizeof(int)*4*cont.nrverts);
	}
	
	return true;
}
	

static const int CHF_MAGIC = ('r' << 24) | ('c' << 16) | ('h' << 8) | 'f';
static const int CHF_VERSION = 3;

bool duDumpCompactHeightfield(struct rcCompactHeightfield& chf, duFileIO* io)
{
	if (!io)
	{
		printf("duDumpCompactHeightfield: input IO is null.\n"); 
		return false;
	}
	if (!io->isWriting())
	{
		printf("duDumpCompactHeightfield: input IO not writing.\n"); 
		return false;
	}
	
	io->write(&CHF_MAGIC, sizeof(CHF_MAGIC));
	io->write(&CHF_VERSION, sizeof(CHF_VERSION));
	
	io->write(&chf.width, sizeof(chf.width));
	io->write(&chf.height, sizeof(chf.height));
	io->write(&chf.spanCount, sizeof(chf.spanCount));

	io->write(&chf.walkableHeight, sizeof(chf.walkableHeight));
	io->write(&chf.walkableClimb, sizeof(chf.walkableClimb));
	io->write(&chf.borderSize, sizeof(chf.borderSize));

	io->write(&chf.maxDistance, sizeof(chf.maxDistance));
	io->write(&chf.maxRegions, sizeof(chf.maxRegions));

	io->write(chf.bmin, sizeof(chf.bmin));
	io->write(chf.bmax, sizeof(chf.bmax));

	io->write(&chf.cs, sizeof(chf.cs));
	io->write(&chf.ch, sizeof(chf.ch));

	int tmp = 0;
	if (chf.cells) tmp |= 1;
	if (chf.spans) tmp |= 2;
	if (chf.dist) tmp |= 4;
	if (chf.areas) tmp |= 8;

	io->write(&tmp, sizeof(tmp));

	if (chf.cells)
		io->write(chf.cells, sizeof(rcCompactCell)*chf.width*chf.height);
	if (chf.spans)
		io->write(chf.spans, sizeof(rcCompactSpan)*chf.spanCount);
	if (chf.dist)
		io->write(chf.dist, sizeof(unsigned short)*chf.spanCount);
	if (chf.areas)
		io->write(chf.areas, sizeof(unsigned char)*chf.spanCount);

	return true;
}

bool duReadCompactHeightfield(struct rcCompactHeightfield& chf, duFileIO* io)
{
	if (!io)
	{
		printf("duReadCompactHeightfield: input IO is null.\n"); 
		return false;
	}
	if (!io->isReading())
	{
		printf("duReadCompactHeightfield: input IO not reading.\n"); 
		return false;
	}

	int magic = 0;
	int version = 0;
	
	io->read(&magic, sizeof(magic));
	io->read(&version, sizeof(version));
	
	if (magic != CHF_MAGIC)
	{
		printf("duReadCompactHeightfield: Bad voodoo.\n");
		return false;
	}
	if (version != CHF_VERSION)
	{
		printf("duReadCompactHeightfield: Bad version.\n");
		return false;
	}
	
	io->read(&chf.width, sizeof(chf.width));
	io->read(&chf.height, sizeof(chf.height));
	io->read(&chf.spanCount, sizeof(chf.spanCount));
	
	io->read(&chf.walkableHeight, sizeof(chf.walkableHeight));
	io->read(&chf.walkableClimb, sizeof(chf.walkableClimb));
	io->write(&chf.borderSize, sizeof(chf.borderSize));

	io->read(&chf.maxDistance, sizeof(chf.maxDistance));
	io->read(&chf.maxRegions, sizeof(chf.maxRegions));
	
	io->read(chf.bmin, sizeof(chf.bmin));
	io->read(chf.bmax, sizeof(chf.bmax));
	
	io->read(&chf.cs, sizeof(chf.cs));
	io->read(&chf.ch, sizeof(chf.ch));
	
	int tmp = 0;
	io->read(&tmp, sizeof(tmp));
	
	if (tmp & 1)
	{
		chf.cells = (rcCompactCell*)rcAlloc(sizeof(rcCompactCell)*chf.width*chf.height, RC_ALLOC_PERM);
		if (!chf.cells)
		{
			printf("duReadCompactHeightfield: Could not alloc cells (%d)\n", chf.width*chf.height);
			return false;
		}
		io->read(chf.cells, sizeof(rcCompactCell)*chf.width*chf.height);
	}
	if (tmp & 2)
	{
		chf.spans = (rcCompactSpan*)rcAlloc(sizeof(rcCompactSpan)*chf.spanCount, RC_ALLOC_PERM);
		if (!chf.spans)
		{
			printf("duReadCompactHeightfield: Could not alloc spans (%d)\n", chf.spanCount);
			return false;
		}
		io->read(chf.spans, sizeof(rcCompactSpan)*chf.spanCount);
	}
	if (tmp & 4)
	{
		chf.dist = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount, RC_ALLOC_PERM);
		if (!chf.dist)
		{
			printf("duReadCompactHeightfield: Could not alloc dist (%d)\n", chf.spanCount);
			return false;
		}
		io->read(chf.dist, sizeof(unsigned short)*chf.spanCount);
	}
	if (tmp & 8)
	{
		chf.areas = (unsigned char*)rcAlloc(sizeof(unsigned char)*chf.spanCount, RC_ALLOC_PERM);
		if (!chf.areas)
		{
			printf("duReadCompactHeightfield: Could not alloc areas (%d)\n", chf.spanCount);
			return false;
		}
		io->read(chf.areas, sizeof(unsigned char)*chf.spanCount);
	}
	
	return true;
}


static void logLine(rcContext& ctx, rcTimerLabel label, const char* name, const float pc)
{
	const int t = ctx.getAccumulatedTime(label);
	if (t < 0) return;
	ctx.log(RC_LOG_PROGRESS, "%s:\t%.2fms\t(%.1f%%)", name, t/1000.0f, t*pc);
}

void duLogBuildTimes(rcContext& ctx, const int totalTimeUsec)
{
	const float pc = 100.0f / totalTimeUsec;
 
	ctx.log(RC_LOG_PROGRESS, "Build Times");
	logLine(ctx, RC_TIMER_RASTERIZE_TRIANGLES,		"- Rasterize", pc);
	logLine(ctx, RC_TIMER_BUILD_COMPACTHEIGHTFIELD,	"- Build Compact", pc);
	logLine(ctx, RC_TIMER_FILTER_BORDER,				"- Filter Border", pc);
	logLine(ctx, RC_TIMER_FILTER_WALKABLE,			"- Filter Walkable", pc);
	logLine(ctx, RC_TIMER_ERODE_AREA,				"- Erode Area", pc);
	logLine(ctx, RC_TIMER_MEDIAN_AREA,				"- Median Area", pc);
	logLine(ctx, RC_TIMER_MARK_BOX_AREA,				"- Mark Box Area", pc);
	logLine(ctx, RC_TIMER_MARK_CONVEXPOLY_AREA,		"- Mark Convex Area", pc);
	logLine(ctx, RC_TIMER_MARK_CYLINDER_AREA,		"- Mark Cylinder Area", pc);
	logLine(ctx, RC_TIMER_BUILD_DISTANCEFIELD,		"- Build Distance Field", pc);
	logLine(ctx, RC_TIMER_BUILD_DISTANCEFIELD_DIST,	"    - Distance", pc);
	logLine(ctx, RC_TIMER_BUILD_DISTANCEFIELD_BLUR,	"    - Blur", pc);
	logLine(ctx, RC_TIMER_BUILD_REGIONS,				"- Build Regions", pc);
	logLine(ctx, RC_TIMER_BUILD_REGIONS_WATERSHED,	"    - Watershed", pc);
	logLine(ctx, RC_TIMER_BUILD_REGIONS_EXPAND,		"      - Expand", pc);
	logLine(ctx, RC_TIMER_BUILD_REGIONS_FLOOD,		"      - Find Basins", pc);
	logLine(ctx, RC_TIMER_BUILD_REGIONS_FILTER,		"    - Filter", pc);
	logLine(ctx, RC_TIMER_BUILD_LAYERS,				"- Build Layers", pc);
	logLine(ctx, RC_TIMER_BUILD_CONTOURS,			"- Build Contours", pc);
	logLine(ctx, RC_TIMER_BUILD_CONTOURS_TRACE,		"    - Trace", pc);
	logLine(ctx, RC_TIMER_BUILD_CONTOURS_SIMPLIFY,	"    - Simplify", pc);
	logLine(ctx, RC_TIMER_BUILD_CLUSTERS,			"- Build Clusters", pc);
	logLine(ctx, RC_TIMER_BUILD_POLYMESH,			"- Build Polymesh", pc);
	logLine(ctx, RC_TIMER_BUILD_POLYMESHDETAIL,		"- Build Polymesh Detail", pc);
	logLine(ctx, RC_TIMER_MERGE_POLYMESH,			"- Merge Polymeshes", pc);
	logLine(ctx, RC_TIMER_MERGE_POLYMESHDETAIL,		"- Merge Polymesh Details", pc);
	ctx.log(RC_LOG_PROGRESS, "=== TOTAL:\t%.2fms", totalTimeUsec/1000.0f);
}

