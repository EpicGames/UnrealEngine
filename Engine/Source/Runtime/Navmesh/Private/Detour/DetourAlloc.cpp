// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "Detour/DetourAlloc.h"

static void *dtAllocDefault(int size, dtAllocHint)
{
	return malloc(size);
}

static void dtFreeDefault(void *ptr, dtAllocHint) //UE
{
	free(ptr);
}

static dtAllocFunc* sAllocFunc = dtAllocDefault;
static dtFreeFunc* sFreeFunc = dtFreeDefault;

void dtAllocSetCustom(dtAllocFunc *allocFunc, dtFreeFunc *freeFunc)
{
	sAllocFunc = allocFunc ? allocFunc : dtAllocDefault;
	sFreeFunc = freeFunc ? freeFunc : dtFreeDefault;
}

void* dtAlloc(int size, dtAllocHint hint)
{
	return sAllocFunc(size, hint);
}

//@UE BEGIN Added dtAllocHint.
void dtFree(void* ptr, dtAllocHint hint)
{
	if (ptr)
		sFreeFunc(ptr, hint);
}
//@UE END

void dtMemCpy(void* dst, void* src, int size)
{
	memcpy(dst, src, size);
}

/// @class dtIntArray
///
/// While it is possible to pre-allocate a specific array size during 
/// construction or by using the #resize method, certain methods will 
/// automatically resize the array as needed.
///
/// @warning The array memory is not initialized to zero when the size is 
/// manually set during construction or when using #resize.

/// @par
///
/// Using this method ensures the array is at least large enough to hold
/// the specified number of elements.  This can improve performance by
/// avoiding auto-resizing during use.
void dtIntArray::resize(int n)
{
	if (n > m_cap)
	{
		if (!m_cap) m_cap = n;
		while (m_cap < n) m_cap *= 2;
		int* newData = (int*)dtAlloc(m_cap*sizeof(int), DT_ALLOC_TEMP);
		if (m_size && newData) memcpy(newData, m_data, m_size*sizeof(int));
		dtFree(m_data, DT_ALLOC_TEMP);
		m_data = newData;
	}
	m_size = n;
}

void dtIntArray::copy(const dtIntArray& src)
{
	if (src.size() > 0)
	{
		resize(src.size());
		memcpy(m_data, src.getData(), m_size);
	}
	else
	{
		resize(0);
	}
}
