// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "Detour/DetourCommon.h"
#include "Detour/DetourAssert.h"

//////////////////////////////////////////////////////////////////////////////////////////

void dtClosestPtPointTriangle(dtReal* closest, const dtReal* p,
							  const dtReal* a, const dtReal* b, const dtReal* c)
{
	// Check if P in vertex region outside A
	dtReal ab[3], ac[3], ap[3];
	dtVsub(ab, b, a);
	dtVsub(ac, c, a);
	dtVsub(ap, p, a);
	dtReal d1 = dtVdot(ab, ap);
	dtReal d2 = dtVdot(ac, ap);
	if (d1 <= 0.0f && d2 <= 0.0f)
	{
		// barycentric coordinates (1,0,0)
		dtVcopy(closest, a);
		return;
	}
	
	// Check if P in vertex region outside B
	dtReal bp[3];
	dtVsub(bp, p, b);
	dtReal d3 = dtVdot(ab, bp);
	dtReal d4 = dtVdot(ac, bp);
	if (d3 >= 0.0f && d4 <= d3)
	{
		// barycentric coordinates (0,1,0)
		dtVcopy(closest, b);
		return;
	}
	
	// Check if P in edge region of AB, if so return projection of P onto AB
	dtReal vc = d1*d4 - d3*d2;
	if (vc <= 0.0f && d1 >= 0.0f && d3 <= 0.0f)
	{
		// barycentric coordinates (1-v,v,0)
		dtReal v = d1 / (d1 - d3);
		closest[0] = a[0] + v * ab[0];
		closest[1] = a[1] + v * ab[1];
		closest[2] = a[2] + v * ab[2];
		return;
	}
	
	// Check if P in vertex region outside C
	dtReal cp[3];
	dtVsub(cp, p, c);
	dtReal d5 = dtVdot(ab, cp);
	dtReal d6 = dtVdot(ac, cp);
	if (d6 >= 0.0f && d5 <= d6)
	{
		// barycentric coordinates (0,0,1)
		dtVcopy(closest, c);
		return;
	}
	
	// Check if P in edge region of AC, if so return projection of P onto AC
	dtReal vb = d5*d2 - d1*d6;
	if (vb <= 0.0f && d2 >= 0.0f && d6 <= 0.0f)
	{
		// barycentric coordinates (1-w,0,w)
		dtReal w = d2 / (d2 - d6);
		closest[0] = a[0] + w * ac[0];
		closest[1] = a[1] + w * ac[1];
		closest[2] = a[2] + w * ac[2];
		return;
	}
	
	// Check if P in edge region of BC, if so return projection of P onto BC
	dtReal va = d3*d6 - d5*d4;
	if (va <= 0.0f && (d4 - d3) >= 0.0f && (d5 - d6) >= 0.0f)
	{
		// barycentric coordinates (0,1-w,w)
		dtReal w = (d4 - d3) / ((d4 - d3) + (d5 - d6));
		closest[0] = b[0] + w * (c[0] - b[0]);
		closest[1] = b[1] + w * (c[1] - b[1]);
		closest[2] = b[2] + w * (c[2] - b[2]);
		return;
	}
	
	// P inside face region. Compute Q through its barycentric coordinates (u,v,w)
	dtReal denom = 1.0f / (va + vb + vc);
	dtReal v = vb * denom;
	dtReal w = vc * denom;
	closest[0] = a[0] + ab[0] * v + ac[0] * w;
	closest[1] = a[1] + ab[1] * v + ac[1] * w;
	closest[2] = a[2] + ab[2] * v + ac[2] * w;
}

bool dtIntersectSegmentPoly2D(const dtReal* p0, const dtReal* p1,
							  const dtReal* verts, int nverts,
							  dtReal& tmin, dtReal& tmax,
							  int& segMin, int& segMax)
{
	static const dtReal EPS = 0.00000001f;

	tmin = 0;
	tmax = 1;
	segMin = -1;
	segMax = -1;
	
	dtReal dir[3];
	dtVsub(dir, p1, p0);
	
	for (int i = 0, j = nverts-1; i < nverts; j=i++)
	{
		dtReal edge[3], diff[3];
		dtVsub(edge, &verts[i*3], &verts[j*3]);
		dtVsub(diff, p0, &verts[j*3]);
		const dtReal n = dtVperp2D(edge, diff);
		const dtReal d = dtVperp2D(dir, edge);
		if (dtAbs(d) < EPS)
		{
			// S is nearly parallel to this edge
			if (n < 0)
				return false;
			else
				continue;
		}
		const dtReal t = n / d;
		if (d < 0)
		{
			// segment S is entering across this edge
			if (t > tmin)
			{
				tmin = t;
				segMin = j;
				// S enters after leaving polygon
				if (tmin > tmax)
					return false;
			}
		}
		else
		{
			// segment S is leaving across this edge
			if (t < tmax)
			{
				tmax = t;
				segMax = j;
				// S leaves before entering polygon
				if (tmax < tmin)
					return false;
			}
		}
	}
	
	return true;
}

dtReal dtDistancePtSegSqr2D(const dtReal* pt, const dtReal* p, const dtReal* q, dtReal& t)
{
	dtReal pqx = q[0] - p[0];
	dtReal pqz = q[2] - p[2];
	dtReal dx = pt[0] - p[0];
	dtReal dz = pt[2] - p[2];
	dtReal d = pqx*pqx + pqz*pqz;

	t = pqx * dx + pqz * dz;
	if (d > 0) t /= d;
	if (t < 0) t = 0;
	else if (t > 1) t = 1;

	dx = p[0] + t*pqx - pt[0];
	dz = p[2] + t*pqz - pt[2];
	return dx*dx + dz*dz;
}

dtReal dtDistancePtSegSqr(const dtReal* pt, const dtReal* p, const dtReal* q)
{
	dtReal seg[3], toPt[3], closest[3];
	dtVsub(seg, q, p);
	dtVsub(toPt, pt, p);

	const dtReal d1 = dtVdot(toPt, seg);
	const dtReal d2 = dtVdot(seg, seg);
	if (d1 <= 0)
	{
		dtVcopy(closest, p);
	}
	else if (d2 <= d1)
	{
		dtVcopy(closest, q);
	}
	else
	{
		dtVmad(closest, p, seg, d1 / d2);
	}

	dtVsub(toPt, closest, pt);
	return dtVlenSqr(toPt);
}

void dtCalcPolyCenter(dtReal* tc, const unsigned short* idx, int nidx, const dtReal* verts)
{
	tc[0] = 0.0f;
	tc[1] = 0.0f;
	tc[2] = 0.0f;
	for (int j = 0; j < nidx; ++j)
	{
		const dtReal* v = &verts[idx[j]*3];
		tc[0] += v[0];
		tc[1] += v[1];
		tc[2] += v[2];
	}
	const dtReal s = dtReal(1.) / nidx;
	tc[0] *= s;
	tc[1] *= s;
	tc[2] *= s;
}

bool dtClosestHeightPointTriangle(const dtReal* p, const dtReal* a, const dtReal* b, const dtReal* c, dtReal& h)
{
	dtReal vC[3], vB[3], vP[3];
	dtVsub(vC, c, a);
	dtVsub(vB, b, a);
	dtVsub(vP, p, a);

	// Compute scaled barycentric coordinates
	dtReal denom = vC[0] * vB[2] - vC[2] * vB[0];

	// The (sloppy) epsilon is needed to allow to get height of points which
	// are interpolated along the edges of the triangles.
	constexpr dtReal EPS = 1.0e-6;
	if (fabs(denom) < EPS)
		return false;

	const dtReal invDenom = 1.0 / denom;
	const dtReal u = (vB[2] * vP[0] - vB[0] * vP[2]) * invDenom;
	const dtReal v = (vC[0] * vP[2] - vC[2] * vP[0]) * invDenom;

	// If point lies inside the triangle, return interpolated ycoord.
	if (u >= -EPS && v >= -EPS && (u+v) <= 1.0+EPS)
	{
		h = a[1] + (vC[1]*u + vB[1]*v);
		return true;
	}
	
	return false;
}

/// @par
///
/// All points are projected onto the xz-plane, so the y-values are ignored.
bool dtPointInPolygon(const dtReal* pt, const dtReal* verts, const int nverts)
{
	// TODO: Replace pnpoly with triArea2D tests?
	int i, j;
	bool c = false;
	for (i = 0, j = nverts-1; i < nverts; j = i++)
	{
		const dtReal* vi = &verts[i*3];
		const dtReal* vj = &verts[j*3];
		if (((vi[2] > pt[2]) != (vj[2] > pt[2])) &&
			(pt[0] < (vj[0]-vi[0]) * (pt[2]-vi[2]) / (vj[2]-vi[2]) + vi[0]) )
			c = !c;
	}
	return c;
}

bool dtDistancePtPolyEdgesSqr(const dtReal* pt, const dtReal* verts, const int nverts,
							  dtReal* ed, dtReal* et)
{
	// TODO: Replace pnpoly with triArea2D tests?
	int i, j;
	bool c = false;
	for (i = 0, j = nverts-1; i < nverts; j = i++)
	{
		const dtReal* vi = &verts[i*3];
		const dtReal* vj = &verts[j*3];
		if (((vi[2] > pt[2]) != (vj[2] > pt[2])) &&
			(pt[0] < (vj[0]-vi[0]) * (pt[2]-vi[2]) / (vj[2]-vi[2]) + vi[0]) )
			c = !c;
		ed[j] = dtDistancePtSegSqr2D(pt, vj, vi, et[j]);
	}
	return c;
}

static void projectPoly(const dtReal* axis, const dtReal* poly, const int npoly,
						dtReal& rmin, dtReal& rmax)
{
	rmin = rmax = dtVdot2D(axis, &poly[0]);
	for (int i = 1; i < npoly; ++i)
	{
		const dtReal d = dtVdot2D(axis, &poly[i*3]);
		rmin = dtMin(rmin, d);
		rmax = dtMax(rmax, d);
	}
}

inline bool overlapRange(const dtReal amin, const dtReal amax,
						 const dtReal bmin, const dtReal bmax,
						 const dtReal eps)
{
	return ((amin+eps) > bmax || (amax-eps) < bmin) ? false : true;
}

/// @par
///
/// All vertices are projected onto the xz-plane, so the y-values are ignored.
bool dtOverlapPolyPoly2D(const dtReal* polya, const int npolya,
						 const dtReal* polyb, const int npolyb)
{
	const dtReal eps = 1e-2f;
	
	for (int i = 0, j = npolya-1; i < npolya; j=i++)
	{
		const dtReal* va = &polya[j*3];
		const dtReal* vb = &polya[i*3];
		const dtReal n[3] = { vb[2]-va[2], 0, -(vb[0]-va[0]) };
		dtReal amin,amax,bmin,bmax;
		projectPoly(n, polya, npolya, amin,amax);
		projectPoly(n, polyb, npolyb, bmin,bmax);
		if (!overlapRange(amin,amax, bmin,bmax, eps))
		{
			// Found separating axis
			return false;
		}
	}
	for (int i = 0, j = npolyb-1; i < npolyb; j=i++)
	{
		const dtReal* va = &polyb[j*3];
		const dtReal* vb = &polyb[i*3];
		const dtReal n[3] = { vb[2]-va[2], 0, -(vb[0]-va[0]) };
		dtReal amin,amax,bmin,bmax;
		projectPoly(n, polya, npolya, amin,amax);
		projectPoly(n, polyb, npolyb, bmin,bmax);
		if (!overlapRange(amin,amax, bmin,bmax, eps))
		{
			// Found separating axis
			return false;
		}
	}
	return true;
}

// Returns a random point in a convex polygon.
// Adapted from Graphics Gems article.
void dtRandomPointInConvexPoly(const dtReal* pts, const int npts, dtReal* areas,
							   const dtReal s, const dtReal t, dtReal* out)
{
	dtAssert(npts > 2);

	// Calc triangle araes
	dtReal areasum = 0.0f;
	for (int i = 2; i < npts; i++) {
		areas[i] = dtTriArea2D(&pts[0], &pts[(i-1)*3], &pts[i*3]);
		areasum += dtMax((dtReal)0.001f, areas[i]);
	}
	// Find sub triangle weighted by area.
	const dtReal thr = s*areasum;
	dtReal acc = 0.0f;
	dtReal u = 1.0f;
	int tri = npts - 1;
	for (int i = 2; i < npts; i++) {
		const dtReal dacc = areas[i];
		if (thr >= acc && thr < (acc+dacc))
		{
			u = (thr - acc) / dacc;
			tri = i;
			break;
		}
		acc += dacc;
	}

	dtAssert(tri >= 2);
	
	dtReal v = dtSqrt(t);
	
	const dtReal a = 1 - v;
	const dtReal b = (1 - u) * v;
	const dtReal c = u * v;
	const dtReal* pa = &pts[0];
	const dtReal* pb = &pts[(tri-1)*3];
	const dtReal* pc = &pts[tri*3];
	
	out[0] = a*pa[0] + b*pb[0] + c*pc[0];
	out[1] = a*pa[1] + b*pb[1] + c*pc[1];
	out[2] = a*pa[2] + b*pb[2] + c*pc[2];
}

// @UE BEGIN
dtRotation dtSelectRotation(dtReal rotationDeg)
{
	rotationDeg = dtfMod(rotationDeg, dtReal(360.));
	if (rotationDeg < 0)
		rotationDeg += 360.f;

	// Snap to 90 degrees increment
	dtRotation rot = DT_ROTATE_0;
	if (rotationDeg > 45.f && rotationDeg <= 135.f)
		rot = DT_ROTATE_90;
	else if (rotationDeg > 135.f && rotationDeg <= 225.f)
		rot = DT_ROTATE_180;
	else if (rotationDeg > 225.f && rotationDeg <= 315.f)
		rot = DT_ROTATE_270;

	return rot;
}

void dtVRot90(dtReal* dest, const dtReal* v, const dtRotation rot)
{
	dest[1] = v[1];

	switch (rot)
	{
	case DT_ROTATE_90:
		dest[0] = -v[2];
		dest[2] = v[0];
		break;
	case DT_ROTATE_180:
		dest[0] = -v[0];
		dest[2] = -v[2];
		break;
	case DT_ROTATE_270:
		dest[0] = v[2];
		dest[2] = -v[0];
		break;
	default:
		// DT_ROTATE_0
		dest[0] = v[0];
		dest[2] = v[2];
		break;
	}
}

void dtVRot90(unsigned short* dest, const unsigned short* v, const dtRotation rot)
{
	dest[1] = v[1];

	switch (rot)
	{
	case DT_ROTATE_90:
		dest[0] = -v[2];
		dest[2] = v[0];
		break;
	case DT_ROTATE_180:
		dest[0] = -v[0];
		dest[2] = -v[2];
		break;
	case DT_ROTATE_270:
		dest[0] = v[2];
		dest[2] = -v[0];
		break;
	default:
		// DT_ROTATE_0
		dest[0] = v[0];
		dest[2] = v[2];
		break;
	}
}

void dtRotate90(dtReal* dest, const dtReal* v, const dtReal* center, const dtRotation rot)
{
	dtReal localPos[3];
	dtVsub(localPos, v, center);
	dtReal newLocalPos[3];
	dtVRot90(newLocalPos, localPos, rot);
	dtVadd(dest, center, newLocalPos);
}

void dtRotate90(unsigned short* dest, const unsigned short* v, const unsigned short* center, const dtRotation rot)
{
	unsigned short localPos[3];
	localPos[0] = v[0] - center[0];
	localPos[2] = v[2] - center[2];

	unsigned short newLocalPos[3];
	dtVRot90(newLocalPos, localPos, rot);

	dest[0] = center[0] + newLocalPos[0];
	dest[1] = v[1];
	dest[2] = center[2] + newLocalPos[2];
}
// @UE END

inline dtReal vperpXZ(const dtReal* a, const dtReal* b) { return a[0]*b[2] - a[2]*b[0]; }

bool dtIntersectSegSeg2D(const dtReal* ap, const dtReal* aq,
						 const dtReal* bp, const dtReal* bq,
						 dtReal& s, dtReal& t)
{
	dtReal u[3], v[3], w[3];
	dtVsub(u,aq,ap);
	dtVsub(v,bq,bp);
	dtVsub(w,ap,bp);
	dtReal d = vperpXZ(u,v);
	if (dtAbs(d) < 1e-6f) return false;
	s = vperpXZ(v,w) / d;
	t = vperpXZ(u,w) / d;
	return (s >= 0.0f && s <= 1.0f);
}

