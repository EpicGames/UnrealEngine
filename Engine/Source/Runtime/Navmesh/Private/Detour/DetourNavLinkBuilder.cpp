// Copyright Epic Games, Inc. All Rights Reserved.
#include "Detour/DetourNavLinkBuilder.h"

#include "Detour/DetourCommon.h"
#include "DetourTileCache/DetourTileCacheBuilder.h"
#include "Recast/Recast.h"

//@UE BEGIN
namespace UE::Detour::NavLink::Private
{
	static void insertSort(unsigned char* a, const int n)
	{
		int j;
		for (int i = 1; i < n; i++)
		{
			const unsigned char value = a[i];
			for (j = i - 1; j >= 0 && a[j] > value; j--)
			{
				a[j+1] = a[j];
			}
			a[j+1] = value;
		}
	}
	
	static float getHeight(const float x, const float* pts, const int npts)
	{
		if (x <= pts[0])
			return pts[1];

		if (x >= pts[(npts-1)*2])
			return pts[(npts-1)*2+1];
	
		for (int i = 1; i < npts; ++i)
		{
			const float* q = &pts[i*2];
			if (x <= q[0])
			{
				const float* p = &pts[(i-1)*2];
				const float u = (x-p[0]) / (q[0]-p[0]);
				return dtLerp(p[1], q[1], u);
			}
		}
	
		return pts[(npts-1)*2+1];
	}

	inline bool overlapRange(const float amin, const float amax, const float bmin, const float bmax)
	{
		return (amin <= bmax && amax >= bmin);
	}

	inline void trans2d(dtReal* dst, const dtReal* ax, const dtReal* ay, const float* pt)
	{
		dst[0] = ax[0]*pt[0] + ay[0]*pt[1];
		dst[1] = ax[1]*pt[0] + ay[1]*pt[1];
		dst[2] = ax[2]*pt[0] + ay[2]*pt[1];
	}
}

dtNavLinkBuilder::GroundSegment::~GroundSegment()
{
}

bool dtNavLinkBuilder::findEdges(rcContext& ctx, const rcConfig& cfg, const dtLinkBuilderConfig& builderConfig,
                                 const dtTileCacheContourSet& lcset, const dtReal* orig,
                                 const rcHeightfield* solidHF, const rcCompactHeightfield* compactHF)
{
	dtAssert(m_solid == nullptr && m_chf == nullptr && m_edges.IsEmpty() && m_links.IsEmpty());
	m_linkBuilderConfig = builderConfig;

	m_cs = cfg.cs;
	m_csSquared = dtSqr(cfg.cs);
	m_ch = cfg.ch;
	m_invCs = 1.0/cfg.cs;
	m_solid = solidHF;
	m_chf = compactHF;

	dtAssert(m_cs == m_chf->cs && m_ch == m_chf->ch);

	// Build edges.
	int edgeCount = 0;
	for (int i = 0; i < lcset.nconts; ++i)
	{
		edgeCount += lcset.conts[i].nverts;
	}

	if (edgeCount == 0)
	{
		ctx.log(RC_LOG_ERROR, "fillEdges: No edges!");
		return false;
	}

	m_edges.Reserve(edgeCount);

	const dtReal cs = cfg.cs;
	const dtReal ch = cfg.ch;
	
	for (int i = 0; i < lcset.nconts; ++i)
	{
		const dtTileCacheContour& c = lcset.conts[i];
		if (!c.nverts)
			continue;

		for (int j = 0, k = c.nverts-1; j < c.nverts; k=j++)
		{
			const unsigned short* va = &c.verts[k*4];
			const unsigned short* vb = &c.verts[j*4];

			if ((va[3] & 0xf) != 0xf)	// A direction is set, so it's a portal edge.
				continue;

			// Check k-j for matching contour
			bool matchFound = false;
			for (int ii = 0; ii < lcset.nconts; ++ii)
			{
				if (i == ii)
					continue;
				
				const dtTileCacheContour& otherCont = lcset.conts[ii];
				if (otherCont.nverts < 3)
					continue;
			
				for (int jj = 0, kk = otherCont.nverts-1; jj < otherCont.nverts; kk=jj++)
				{
					const unsigned short* otherVa = &otherCont.verts[kk*4];
					const unsigned short* otherVb = &otherCont.verts[jj*4];
			
					if( (dtVisEqual(va, otherVa) && dtVisEqual(vb, otherVb)) ||
						(dtVisEqual(va, otherVb) && dtVisEqual(vb, otherVa))    )
					{
						// Same edge, skip it.
						matchFound = true;
						break;
					}
				}
			
				if (matchFound)
				{
					break;
				}
			}
			
			if (!matchFound)
			{
				// Add edge
				Edge& e = m_edges.Emplace_GetRef();

				e.sp[0] = orig[0] + vb[0]*cs;
				e.sp[1] = orig[1] + (vb[1]+2)*ch;
				e.sp[2] = orig[2] + vb[2]*cs;

				e.sq[0] = orig[0] + va[0]*cs;
				e.sq[1] = orig[1] + (va[1]+2)*ch;
				e.sq[2] = orig[2] + va[2]*cs;
			}
		}
	}

	return true;
}

void dtNavLinkBuilder::addEdgeLinks(const dtLinkBuilderConfig& builderConfig, const EdgeSampler* es, const int edgeIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(dtNavLinkBuilder::addEdgeLinks);
	
	using namespace UE::Detour::NavLink::Private;
	
	if (es->start.ngsamples != es->end.ngsamples)
	{
		return;
	}
	
	const int nsamples = es->start.ngsamples;

	// Filter small holes.
	constexpr int RAD = 2;
	GroundSampleFlag kernel[RAD*2+1];

	TArray<GroundSampleFlag, TInlineAllocator<64>> groundSampleFlags;
	groundSampleFlags.Reserve(nsamples);
	
	for (int i = 0; i < nsamples; ++i)
	{
		const int a = dtMax(0, i-RAD);
		const int b = dtMin(nsamples-1, i+RAD);
		int nkernel = 0;
		for (int j = a; j <= b; ++j)
		{
			kernel[nkernel++] = (GroundSampleFlag)(es->start.gsamples[i].flags & UNRESTRICTED);
		}
		insertSort((unsigned char*)kernel, nkernel);
		groundSampleFlags.Add(kernel[(nkernel+1)/2]);
	}

	const dtReal edgeLength = dtVdist(es->rigp, es->rigq);
	const dtReal distanceBetweenSamples = edgeLength / (es->start.ngsamples-1);
	
	// Build segments
	int start = -1;
	for (int i = 0; i <= nsamples; ++i)
	{
		const bool valid = i < nsamples && groundSampleFlags[i] != UNSET;
		if (start == -1)
		{
			if (valid)
				start = i;
		}
		else
		{
			if (!valid)
			{
				dtReal freeWidth = ((i-start)-1)*distanceBetweenSamples;

				// Trajectories just before the start or after the end of the edge are expected to be unrestricted.
				if (start == 0 || i == (nsamples-1))
				{
					freeWidth += builderConfig.agentRadius;
				}

				if (freeWidth >= builderConfig.agentRadius)
				{
					const float u0 = (float)start/(float)(nsamples-1);
					const float u1 = (float)(i-1)/(float)(nsamples-1);

					dtReal sp[3], sq[3], ep[3], eq[3];

					dtVlerp(sp, es->start.p,es->start.q, u0);
					dtVlerp(sq, es->start.p,es->start.q, u1);
					dtVlerp(ep, es->end.p,es->end.q, u0);
					dtVlerp(eq, es->end.p,es->end.q, u1);
					sp[1] = es->start.gsamples[start].height;
					sq[1] = es->start.gsamples[i-1].height;
					ep[1] = es->end.gsamples[start].height;
					eq[1] = es->end.gsamples[i-1].height;

					JumpLink& link = m_links.Emplace_GetRef();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					link.debugSourceEdge = (short)edgeIndex;
#endif
					link.configIndex = es->configIndex;
					link.flags = VALID;
					link.nspine = es->trajectory.nspine;

					const float startx = es->trajectory.spine[0];
					const float endx = es->trajectory.spine[(es->trajectory.nspine-1)*2];
					const float deltax = endx - startx;
					
					const float starty = es->trajectory.spine[1];
					const float endy = es->trajectory.spine[(es->trajectory.nspine-1)*2+1];
					
					// Build link->spine0
					for (int j = 0; j < es->trajectory.nspine; ++j)
					{
						const float* spt = &es->trajectory.spine[j*2];
						const float u = (spt[0] - startx)/deltax;
						const float dy = spt[1] - dtLerp(starty, endy, u) + m_linkBuilderConfig.agentClimb;
						dtReal* p = &link.spine0[j*3];
						dtVlerp(p, sp, ep, u);
						dtVmad(p, p, es->ay, dy);
					}

					// Build link->spine1
					for (int j = 0; j < es->trajectory.nspine; ++j)
					{
						const float* spt = &es->trajectory.spine[j*2];
						const float u = (spt[0] - startx)/deltax;
						const float dy = spt[1] - dtLerp(starty, endy, u) + m_linkBuilderConfig.agentClimb;
						dtReal* p = &link.spine1[j*3];
						dtVlerp(p, sq, eq, u);
						dtVmad(p, p, es->ay, dy);
					}
				}
				
				start = -1;
			}
		}
	}
}

void dtNavLinkBuilder::filterOverlappingLinks(const float edgeDistanceThreshold)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(dtNavLinkBuilder::filterOverlappingLinks);
	
	using namespace UE::Detour::NavLink::Private;
	
	// Filter out links which overlap
	const float thresholdSquared = dtSqr(edgeDistanceThreshold);
	
	for (int i = 0; i < m_links.Num()-1; ++i)
	{
		JumpLink& li = m_links[i];
		if (li.flags == FILTERED)
			continue;
		
		const dtReal* spi = &li.spine0[0];
		const dtReal* sqi = &li.spine1[0];
		const dtReal* epi = &li.spine0[(li.nspine-1)*3];
		const dtReal* eqi = &li.spine1[(li.nspine-1)*3];
		
		for (int j = i+1; j < m_links.Num(); ++j)
		{
			JumpLink& lj = m_links[j];
			if (lj.flags == FILTERED)
				continue;
			
			const dtReal* spj = &lj.spine0[0];
			const dtReal* sqj = &lj.spine1[0];
			const dtReal* epj = &lj.spine0[(lj.nspine-1)*3];
			const dtReal* eqj = &lj.spine1[(lj.nspine-1)*3];

			const dtReal d0 = dtDistancePtSegSqr(spj, epi, eqi);
			const dtReal d1 = dtDistancePtSegSqr(sqj, epi, eqi);
			const dtReal d2 = dtDistancePtSegSqr(epj, spi, sqi);
			const dtReal d3 = dtDistancePtSegSqr(eqj, spi, sqi);
			
			if (d0 < thresholdSquared && d1 < thresholdSquared && d2 < thresholdSquared && d3 < thresholdSquared)
			{
				// If the links are similar but use different areas, they need to be kept.
				const dtNavLinkBuilderJumpConfig& liConfig = m_linkBuilderConfig.jumpConfigs[li.configIndex];
				const dtNavLinkBuilderJumpConfig& ljConfig = m_linkBuilderConfig.jumpConfigs[lj.configIndex];
				if (liConfig.upDirArea != ljConfig.upDirArea ||
					liConfig.downDirArea != ljConfig.downDirArea)
				{
					continue;
				}
				
				// Remove one of the link, keeping the wider one
				if (dtVdistSqr(spi,sqi) > dtVdistSqr(spj,sqj))
				{
					lj.flags = FILTERED;
				}
				else
				{
					li.flags = FILTERED;
					break;
				}
			}
		}
	}
}

void dtNavLinkBuilder::buildForAllEdges(rcContext& ctx, const dtLinkBuilderConfig& builderConfig, const dtNavLinkBuilderJumpConfig& jumpConfig)
{
	for (int i = 0; i < m_edges.Num(); ++i)
	{
		EdgeSampler sampler;
		const bool success = sampleEdge(jumpConfig, m_edges[i].sp, m_edges[i].sq, &sampler);
		if (success)
		{
			addEdgeLinks(builderConfig, &sampler, i);
		}
	}

	ctx.log(RC_LOG_PROGRESS, "   %i links added.", m_links.Num());

	filterOverlappingLinks(jumpConfig.filterDistanceThreshold);
}

void dtNavLinkBuilder::debugBuildEdge(const dtLinkBuilderConfig& builderConfig, int edgeIndex, EdgeSampler& sampler, const dtNavLinkBuilderJumpConfig& jumpConfig)
{
	if (edgeIndex >= m_edges.Num())
	{
		return;
	}
	
	m_debugSelectedEdge = edgeIndex;

	const bool success = sampleEdge(jumpConfig, m_edges[edgeIndex].sp, m_edges[edgeIndex].sq, &sampler);
	if (success)
	{
		addEdgeLinks(builderConfig, &sampler, edgeIndex);
	}

	filterOverlappingLinks(jumpConfig.filterDistanceThreshold);
}

bool dtNavLinkBuilder::getCompactHeightfieldHeight(const dtReal* pt, const dtReal hrange, dtReal* height) const
{
	const int chfWidth = m_chf->width;
	const int chfHeight = m_chf->height;

	const dtReal range = m_cs;
	const int ix0 = dtClamp((int)dtFloor((pt[0]-range - m_chf->bmin[0])*m_invCs), 0, chfWidth-1);
	const int iz0 = dtClamp((int)dtFloor((pt[2]-range - m_chf->bmin[2])*m_invCs), 0, chfHeight-1);
	const int ix1 = dtClamp((int)dtFloor((pt[0]+range - m_chf->bmin[0])*m_invCs), 0, chfWidth-1);
	const int iz1 = dtClamp((int)dtFloor((pt[2]+range - m_chf->bmin[2])*m_invCs), 0, chfHeight-1);
	
	dtReal bestDist = DT_REAL_MAX;
	dtReal bestHeight = DT_REAL_MAX;
	bool found = false;
	
	for (int z = iz0; z <= iz1; ++z)
	{
		for (int x = ix0; x <= ix1; ++x)
		{
			const rcCompactCell& c = m_chf->cells[x+z*chfWidth];
			for (unsigned int i = c.index, ni = c.index+c.count; i < ni; ++i)
			{
				if (m_chf->areas[i] == RC_NULL_AREA)
				{
					continue;
				}
				
				const dtReal y = m_chf->bmin[1] + m_chf->spans[i].y * m_ch;
				const dtReal dist = abs(y - pt[1]);
				if (dist < hrange && dist < bestDist)
				{
					bestDist = dist;
					bestHeight = y;
					found = true;
				}
			}
		}
	}
	
	if (found)
	{
		*height = bestHeight;
	}
	else
	{
		*height = pt[1];
	}
	
	return found;
}

// Compare ymin, ymax range with the solid height field spans to see if it collides.
// Returns true if there is a collision.
bool dtNavLinkBuilder::checkHeightfieldCollision(const dtReal x, const dtReal ymin, const dtReal ymax, const dtReal z) const
{
	using namespace UE::Detour::NavLink::Private;
	
	const int w = m_solid->width;
	const int h = m_solid->height;
	const rcReal* orig = m_solid->bmin;
	const int ix = (int)dtFloor((x - orig[0])*m_invCs);
	const int iz = (int)dtFloor((z - orig[2])*m_invCs);
	
	if (ix < 0 || iz < 0 || ix > w || iz > h)
	{
		return false;
	}
	
	const rcSpan* s = m_solid->spans[ix + iz*w];
	if (!s)
	{
		return false;
	}
	
	while (s)
	{
		const float symin = orig[1] + s->data.smin*m_ch;
		const float symax = orig[1] + s->data.smax*m_ch;
		if (overlapRange(ymin, ymax, symin, symax))
			return true;
		
		s = s->next;
	}

	return false;
}

// Returns true if none of the samples ymin, ymax collide with the heightfield.
bool dtNavLinkBuilder::isTrajectoryClear(const dtReal* pa, const dtReal* pb, const Trajectory2D* trajectory, const dtReal* trajectoryDir) const
{
	dtReal start[3];
	dtReal end[3];
	dtVcopy(start, pa);
	dtVcopy(end, pb);
	
	// Offset start and end points to account for the agent radius.
	dtVmad(start, pa, trajectoryDir, -trajectory->radiusOverflow);
	dtVmad(end, pb, trajectoryDir,  trajectory->radiusOverflow);

	const int nsamples = trajectory->samples.Num();
	const float invLastSample = 1.f / (nsamples-1);
	for (int i = 0; i < nsamples; ++i)
	{
		dtReal p[3];
		const TrajectorySample& s = trajectory->samples[i];
		const float u = (float)i * invLastSample;
		dtVlerp(p, start, end, u);

		if (checkHeightfieldCollision(p[0], p[1] + s.ymin, p[1] + s.ymax, p[2]))
		{
			return false;
		}
	}
	
	return true;	
}

// Add ground samples and set height on them.
void dtNavLinkBuilder::sampleGroundSegment(GroundSegment* seg, const int nsamples, const float groundRange) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(dtNavLinkBuilder::sampleGroundSegment);
	
	dtReal delta[3];
	dtVsub(delta, seg->p, seg->q);
	
	seg->ngsamples = nsamples;
	seg->npass = 0;

	const float invLastIndex = 1.f/(nsamples-1);
	for (int i = 0; i < nsamples; ++i)
	{
		const float u = (float)i*invLastIndex;
		dtReal pt[3];

		GroundSample& s = seg->gsamples.Emplace_GetRef();
		dtVlerp(pt, seg->p, seg->q, u);
		s.flags = dtNavLinkBuilder::UNSET;
		if (!getCompactHeightfieldHeight(pt, groundRange, &s.height))
		{
			continue;
		}
		
		s.flags = static_cast<GroundSampleFlag>((unsigned char)s.flags | (unsigned char)HAS_GROUND);
		seg->npass++;
	}
}

void dtNavLinkBuilder::updateTrajectorySamples(EdgeSampler* es) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(dtNavLinkBuilder::updateTrajectorySamples);
	
	if (es->start.ngsamples != es->end.ngsamples)
		return;
	
	const int nsamples = es->start.ngsamples;
	
	for (int i = 0; i < nsamples; ++i)
	{
		GroundSample& ssmp = es->start.gsamples[i];
		GroundSample& esmp = es->end.gsamples[i];

		// If there is no ground, the ground height will not be set.
		if ((ssmp.flags & HAS_GROUND) == 0 || (esmp.flags & HAS_GROUND) == 0)
			continue;

		// When we sampled ground segments, in sampleEdges, we added least 2 samples.
		check(nsamples >= 2);																
		const dtReal u = (dtReal)i/(dtReal)(nsamples-1);
		dtReal spt[3], ept[3];
		dtVlerp(spt, es->start.p, es->start.q, u);
		dtVlerp(ept, es->end.p, es->end.q, u);
	
		// Offset start and end points to account for the agent radius.
		dtVmad(spt, spt, es->az, -es->trajectory.radiusOverflow);
		dtVmad(ept, ept, es->az,  es->trajectory.radiusOverflow);

		const int nTrajectorySamples = es->trajectory.samples.Num();
		// When we initialize trajectory samples (initTrajectorySamples), we add at least 2 trajectory samples.
		check(nTrajectorySamples >= 2);												
		const float invLastTrajSample = 1.f / (nTrajectorySamples-1);
		for (int trajIndex = 0; trajIndex < nTrajectorySamples; ++trajIndex)
		{
			dtReal p[3];
			TrajectorySample& s = es->trajectory.samples[trajIndex];
			const float trajU = (float)trajIndex * invLastTrajSample;
			dtVlerp(p, spt, ept, trajU);

			if (s.floorStart)
			{
				s.ymin = (ssmp.height + m_linkBuilderConfig.agentClimb) - p[1];		// -p[1] to stay relative to p[1]

				// Update ymax if ymin is now higher than ymax.
				s.ymax = dtMax(s.ymin, s.ymax);
			}
			else if (s.floorEnd)
			{
				s.ymin = (esmp.height + m_linkBuilderConfig.agentClimb) - p[1];		// -p[1] to stay relative to p[1]
				
				// Update ymax if ymin is now higher than ymax.
				s.ymax = dtMax(s.ymin, s.ymax);
			}
		}
	}
}

void dtNavLinkBuilder::sampleAction(EdgeSampler* es) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(dtNavLinkBuilder::sampleAction);
	
	if (es->start.ngsamples != es->end.ngsamples)
		return;
	
	const int nsamples = es->start.ngsamples;
	
	for (int i = 0; i < nsamples; ++i)
	{
		GroundSample& ssmp = es->start.gsamples[i];
		GroundSample& esmp = es->end.gsamples[i];
		
		if ((ssmp.flags & HAS_GROUND) == 0 || (esmp.flags & HAS_GROUND) == 0)
			continue;

		const dtReal u = (dtReal)i/(dtReal)(nsamples-1);
		dtReal spt[3], ept[3];
		dtVlerp(spt, es->start.p, es->start.q, u);
		dtVlerp(ept, es->end.p, es->end.q, u);
		
		if (!isTrajectoryClear(spt, ept, &es->trajectory, es->az))
			continue;

		ssmp.flags = static_cast<GroundSampleFlag>((unsigned char)ssmp.flags | (unsigned char)UNRESTRICTED);
	}
}

void dtNavLinkBuilder::initTrajectorySamples(const float groundRange, Trajectory2D* trajectory) const
{
	using namespace UE::Detour::NavLink::Private;
	
	const float agentRadius = (float)m_linkBuilderConfig.agentRadius;
	trajectory->radiusOverflow = agentRadius;

	// Spine points [x,y]. y is up and x is in the direction of the trajectory, relative to the edge. 
	float pa[2] = { trajectory->spine[0], trajectory->spine[1] };
	float pb[2] = { trajectory->spine[(trajectory->nspine-1)*2], trajectory->spine[(trajectory->nspine-1)*2+1] };

	// Finding samples along the spine accounting for the agent size,
	// so we need to look a bit before and after the desired trajectory.
	pa[0] -= agentRadius;
	pb[0] += agentRadius;
	
	const float dx = pb[0] - pa[0];
	const int nsamples  = dtMax(2, (int)ceilf(dx*m_invCs));
	trajectory->samples.Reserve(nsamples);

	const float dxSample = dx/nsamples;
	const float roundedAgentRadius = dxSample > 0.f ? ceilf(agentRadius/dxSample)*dxSample : 0.f; 
	
	const float* spine = trajectory->spine;
	unsigned char nspine = trajectory->nspine;

	const unsigned short lastSampleIndex = nsamples-1;
	const float invLastIndex = 1.f/lastSampleIndex;
	for (int i = 0; i < nsamples; ++i)
	{
		const float u = (float)i * invLastIndex;
		const float xRef = dtLerp(pa[0], pb[0], u);
		const float yRef = dtLerp(pa[1], pb[1], u);

		// Sample the height on the spine at 3 locations to get an approximated min and max y.
		const float y0 = getHeight(xRef - agentRadius, spine, nspine);
		const float y1 = getHeight(xRef + agentRadius, spine, nspine);
		const float y2 = getHeight(xRef, spine, nspine);

		TrajectorySample& s = trajectory->samples.Emplace_GetRef();
		s.ymin = dtMin(dtMin(y0,y1), y2) + m_linkBuilderConfig.agentClimb - yRef;	
		s.ymax = dtMax(dtMax(y0,y1), y2) + m_linkBuilderConfig.agentHeight - yRef; 

		// Mark start samples that need to be floored. 
		if (xRef >= (spine[0]-roundedAgentRadius) && xRef <= spine[0]+roundedAgentRadius)
			s.floorStart = true;
		
		// More importantly, mark samples that need to be floored at the end since the ground could be far from trajectory end point.
		// We use the upper bound of the tolerance at the end segment (groundRange) to identify samples that need to be floored.
		// Min values below the upper bound need to be marked.
		const float endSplineHeight = pb[1];
		if (s.ymin + yRef < endSplineHeight + groundRange)
		{
			s.floorEnd = true;
		}		
	}
}

void dtNavLinkBuilder::initJumpRig(EdgeSampler* es, const dtReal* sp, const dtReal* sq, const dtNavLinkBuilderJumpConfig& config) const
{
	es->configIndex = config.ConfigIndex;
	
	// Set axes
	dtVsub(es->ax, sq, sp);
	dtVnormalize(es->ax);
	dtVset(es->az, es->ax[2], 0, -es->ax[0]);
	dtVnormalize(es->az);
	dtVset(es->ay, 0, 1, 0);
	
	// Set edge
	const dtReal edgeLengthSqr = dtVdistSqr(sp, sq);
	if (edgeLengthSqr > m_csSquared)
	{
		// Trim tips by cellSize to account for edges overlapping the rasterization borders.
		// This avoids getting the wrong height in getCompactHeightfieldHeight that need to lookup multiple cells.
		dtVmad(es->rigp, sp, es->ax, m_cs);
		dtVmad(es->rigq, sq, es->ax, -m_cs);
	}
	else
	{
		// If it's impossible because the edge is too short, just keep the original edge.
		dtVcopy(es->rigp, sp);
		dtVcopy(es->rigq, sq);
	}

	// Parabolic equation y(x) = ax^2 + (-d/l - al)x
	// Where 'a' is constant
	//       'l' is the jump length from the starting point
	//       'd' is the distance below the starting point

	const float jumpStartDist = config.jumpDistanceFromEdge;
	const float jumpLength = config.jumpLength;
	const float a = config.cachedParabolaConstant;
	const float downRatio = config.cachedDownRatio;		// -d/l
	
	// Build action sampling spine.
	es->trajectory.nspine = MAX_SPINE;
	for (int i = 0; i < MAX_SPINE; ++i)
	{
		float* pt = &es->trajectory.spine[i*2];			// pt: [xy] (x is toward jump end, y is up)
		const float u = (float)i/(float)(MAX_SPINE-1);
		pt[0] = -jumpStartDist + (u*jumpLength);

		// Parabolic equation y(x) = ax^2 + (-d/l - al)x
		//                    y(x) = x * (ax + (-d/l - al))
		pt[1] = (u*jumpLength) * (a*(u*jumpLength) + (downRatio - a*jumpLength));
	}

	es->groundRange = config.jumpEndsHeightTolerance;
}

bool dtNavLinkBuilder::sampleEdge(const dtNavLinkBuilderJumpConfig& jumpConfig, const dtReal* sp, const dtReal* sq, dtNavLinkBuilder::EdgeSampler* es) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(dtNavLinkBuilder::sampleEdge);
	
	using namespace UE::Detour::NavLink::Private;

	const float samplingSeparationFactor = jumpConfig.samplingSeparationFactor;
	initJumpRig(es, sp, sq, jumpConfig);

	initTrajectorySamples(es->groundRange, &es->trajectory);
	
	// Init start end segments.
	dtReal offset[3];
	trans2d(offset, es->az, es->ay, &es->trajectory.spine[0]);
	dtVadd(es->start.p, es->rigp, offset);
	dtVadd(es->start.q, es->rigq, offset);
	trans2d(offset, es->az, es->ay, &es->trajectory.spine[(es->trajectory.nspine-1)*2]);
	dtVadd(es->end.p, es->rigp, offset);
	dtVadd(es->end.q, es->rigq, offset);

	// Sample start and end ground segments.
	const float dist = sqrtf(dtVdistSqr(es->rigp, es->rigq));

	const dtReal distBetweenSamples = samplingSeparationFactor*m_cs;
	const int ngsamples = dtMax(2, (int)ceilf(dist/distBetweenSamples));

	sampleGroundSegment(&es->start, ngsamples, es->groundRange);
	sampleGroundSegment(&es->end, ngsamples, es->groundRange);

	// Now that we have ground heights, update the trajectory samples.
	updateTrajectorySamples(es);
	
	sampleAction(es);
	
	return true;
}
//@UE END
