// Copyright Epic Games, Inc. All Rights Reserved.
#include "Detour/DetourNavLinkBuilderConfig.h"

#include "Detour/DetourCommon.h"

void dtNavLinkBuilderJumpConfig::init()
{
	check(jumpHeight >= 0.f);
	
	// Parabolic equation y(x) = ax^2 + (-d/l - al)x
	// Where 'a' is constant
	//       'l' is the jump length from the starting point (jumpLength)
	//       'd' is the distance below the starting point (jumpMaxDepth)

	// Solving a for the jumpHeight (h) from the starting point gives:
	// a(h) = -(1/l^2) * (d + 2h + 2*srt(h(h+d)))
	cachedParabolaConstant = -(1/dtSqr(jumpLength)) * (jumpMaxDepth + 2*jumpHeight + 2*dtSqrt(jumpHeight * dtMax(0.f, jumpHeight+jumpMaxDepth)));

	// Caching this as well as it remains constant for all links
	cachedDownRatio = -jumpMaxDepth/jumpLength;
}