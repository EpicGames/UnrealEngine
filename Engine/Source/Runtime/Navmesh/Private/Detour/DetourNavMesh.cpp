// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "Detour/DetourNavMesh.h"
#include "Detour/DetourCommon.h"
#include "Detour/DetourAssert.h"

DEFINE_LOG_CATEGORY(LogDetour);

#if WITH_NAVMESH_SEGMENT_LINKS
DEFINE_LOG_CATEGORY(LogSegmentLink);
#endif

//@UE BEGIN Adding support for memory tracking.
static dtStatsPostAddTileFunc* sAddTileFunc = nullptr;
static dtStatsPreRemoveTileFunc* sRemoveTileFunc = nullptr;

void dtStatsSetCustom(dtStatsPostAddTileFunc* addFunc, dtStatsPreRemoveTileFunc* removeFunc)
{
	sAddTileFunc = addFunc;
	sRemoveTileFunc = removeFunc;
}

static void dtStatsPostAddTile(const dtMeshTile& tileAdd)
{
#if DT_STATS
	if (sAddTileFunc)
	{
		sAddTileFunc(tileAdd);
	}
#endif
}

static void dtStatsPreRemoveTile(const dtMeshTile& tileRemove)
{
#if DT_STATS
	if (sRemoveTileFunc)
	{
		sRemoveTileFunc(tileRemove);
	}
#endif
}
//@UE END Adding support for memory tracking.

enum ESlabOverlapFlag
{
	SLABOVERLAP_Cross = 1,
	SLABOVERLAP_Min = 2,
	SLABOVERLAP_Max = 4,
};

inline bool overlapSlabs(const dtReal* amin, const dtReal* amax,
						 const dtReal* bmin, const dtReal* bmax,
						 const dtReal px, const dtReal py, unsigned char* mode)
{
	// Check for horizontal overlap.
	// The segment is shrunken a little so that slabs which touch
	// at end points are not connected.
	//@UE BEGIN Changed to relative comparison to avoid losing floating point precision.
	const dtReal minx = dtMax(amin[0], bmin[0]);
	const dtReal maxx = dtMin(amax[0], bmax[0]);
	const dtReal diff = maxx - minx;
	if (diff < px)
	{
		*mode = 0; // No overlap
		return false;
	}
	//@UE END

	// Check vertical overlap.
	const dtReal ad = (amax[1]-amin[1]) / (amax[0]-amin[0]);
	const dtReal ak = amin[1] - ad*amin[0];
	const dtReal bd = (bmax[1]-bmin[1]) / (bmax[0]-bmin[0]);
	const dtReal bk = bmin[1] - bd*bmin[0];
	const dtReal aminy = ad*minx + ak;
	const dtReal amaxy = ad*maxx + ak;
	const dtReal bminy = bd*minx + bk;
	const dtReal bmaxy = bd*maxx + bk;
	const dtReal dmin = bminy - aminy;
	const dtReal dmax = bmaxy - amaxy;
		
	// Crossing segments always overlap.
	if (dmin*dmax < 0)
	{
		*mode = SLABOVERLAP_Cross;
		return true;
	}
		
	// Check for overlap at endpoints.
	const dtReal thr = dtSqr(py*2);
	if (dmin*dmin <= thr)
	{
		*mode |= SLABOVERLAP_Min;
	}

	if (dmax*dmax <= thr)
	{
		*mode |= SLABOVERLAP_Max;
	}
		
	return (*mode != 0);
}

static dtReal getSlabCoord(const dtReal* va, const int side)
{
	if (side == 0 || side == 4)
		return va[0];
	else if (side == 2 || side == 6)
		return va[2];
	return 0;
}

static void calcSlabEndPoints(const dtReal* va, const dtReal* vb, dtReal* bmin, dtReal* bmax, const int side)
{
	if (side == 0 || side == 4)
	{
		if (va[2] < vb[2])
		{
			bmin[0] = va[2];
			bmin[1] = va[1];
			bmax[0] = vb[2];
			bmax[1] = vb[1];
		}
		else
		{
			bmin[0] = vb[2];
			bmin[1] = vb[1];
			bmax[0] = va[2];
			bmax[1] = va[1];
		}
	}
	else if (side == 2 || side == 6)
	{
		if (va[0] < vb[0])
		{
			bmin[0] = va[0];
			bmin[1] = va[1];
			bmax[0] = vb[0];
			bmax[1] = vb[1];
		}
		else
		{
			bmin[0] = vb[0];
			bmin[1] = vb[1];
			bmax[0] = va[0];
			bmax[1] = va[1];
		}
	}
}

static dtReal getHeightFromDMesh(const dtMeshTile* tile, int polyIdx, dtReal* pos)
{
	if (tile == 0 || polyIdx < 0 || polyIdx >= tile->header->detailMeshCount)
		return 0.0f;

	const dtPolyDetail* pd = &tile->detailMeshes[polyIdx];
	const dtPoly* poly = &tile->polys[polyIdx];
	for (int j = 0; j < pd->triCount; ++j)
	{
		const unsigned char* t = &tile->detailTris[(pd->triBase+j)*4];
		dtReal vBuf[3][3]; //@UE
		const dtReal* v[3];
		for (int k = 0; k < 3; ++k)
		{
			if (t[k] < poly->vertCount)
				tile->verts[poly->verts[t[k]]].toWorld(tile->header->bmin, vBuf[k]); //@UE
			else
				tile->detailVerts[pd->vertBase+(t[k]-poly->vertCount)].toWorld(tile->header->bmin, vBuf[k]); //@UE

			v[k] = vBuf[k]; //@UE
		}
		dtReal h;
		if (dtClosestHeightPointTriangle(pos, v[0], v[1], v[2], h))
		{
			return h;
		}
	}

	return 0.0f;
}

inline int computeTileHash(int x, int y, const int mask)
{
	const unsigned int h1 = 0x8da6b343; // Large multiplicative constants;
	const unsigned int h2 = 0xd8163841; // here arbitrarily chosen primes
	unsigned int n = h1 * x + h2 * y;
	return (int)(n & mask);
}

//@UE BEGIN
enum ELinkAllocationType
{
	// allocated in linksFreeList from dtMeshTile (detour) first, then to dynamic array
	CREATE_LINK_PREALLOCATED,
	// offmesh links are added in a dynamic array (sharing the space with extra reallocated links) 
	CREATE_LINK_DYNAMIC_OFFMESH,
#if WITH_NAVMESH_CLUSTER_LINKS
	CREATE_LINK_DYNAMIC_CLUSTER,
#endif // WITH_NAVMESH_CLUSTER_LINKS
};

inline unsigned int allocLink(dtMeshTile* tile, char LinkAllocMode)
{
	unsigned int newLink = DT_NULL_LINK;

	if (LinkAllocMode == CREATE_LINK_PREALLOCATED)
	{
		// Preallocated links: prefer base pool, fall back to dynamic if exhausted
		if (tile->linksFreeList != DT_NULL_LINK)
		{
			newLink = tile->linksFreeList;
			tile->linksFreeList = tile->links[newLink].next;
		}
		else
		{
			// Base pool exhausted - fall back to secondary storage
			if (tile->dynamicFreeList == DT_NULL_LINK)
			{
				// No free secondary links, grow the array
				dtLink emptyLink;
				memset(&emptyLink, 0, sizeof(dtLink));
				emptyLink.next = DT_NULL_LINK;

				tile->dynamicFreeList = tile->dynamicLinks.size();
				tile->dynamicLinks.push(emptyLink);
			}

			newLink = tile->dynamicFreeList;
			tile->dynamicFreeList = tile->dynamicLinks[newLink].next;

			// Offset into secondary range
			newLink += tile->header->maxLinkCount;
		}
	}
	// Off-mesh links use secondary storage to preserve base pool for internal/external links
	else if (LinkAllocMode == CREATE_LINK_DYNAMIC_OFFMESH)
	{
		// Allocate from secondary storage only (never consume base pool)
		// This ensures internal/external links (PREALLOCATED) get base pool priority
		if (tile->dynamicFreeList == DT_NULL_LINK)
		{
			// No free secondary links, grow the array
			dtLink emptyLink;
			memset(&emptyLink, 0, sizeof(dtLink));
			emptyLink.next = DT_NULL_LINK;

			tile->dynamicFreeList = tile->dynamicLinks.size();
			tile->dynamicLinks.push(emptyLink);
		}

		newLink = tile->dynamicFreeList;
		tile->dynamicFreeList = tile->dynamicLinks[newLink].next;

		// Offset into secondary range
		newLink += tile->header->maxLinkCount;
	}
#if WITH_NAVMESH_CLUSTER_LINKS
	else if (LinkAllocMode == CREATE_LINK_DYNAMIC_CLUSTER)
	{
		if (tile->dynamicFreeListC == DT_NULL_LINK)
		{
			dtClusterLink emptyLink;
			memset(&emptyLink, 0, sizeof(dtClusterLink));
			emptyLink.next = DT_NULL_LINK;

			tile->dynamicFreeListC = tile->dynamicLinksC.size();
			tile->dynamicLinksC.push(emptyLink);		
		}

		newLink = tile->dynamicFreeListC;
		tile->dynamicFreeListC = tile->dynamicLinksC[newLink].next;

		newLink += DT_CLINK_FIRST;
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

	return newLink;
}

inline void freeLink(dtMeshTile* tile, unsigned int link)
{
	//@UE BEGIN: offmesh links were added in dynamic array
#if WITH_NAVMESH_CLUSTER_LINKS
	const unsigned int firstClusterLinkIdx = DT_CLINK_FIRST;
#else
	const unsigned int firstClusterLinkIdx = UINT_MAX;
#endif // WITH_NAVMESH_CLUSTER_LINKS

	if (link < (unsigned int)tile->header->maxLinkCount)
	{
		tile->links[link].next = tile->linksFreeList;
		tile->linksFreeList = link;
	}
	else if (link < firstClusterLinkIdx)
	{
		const unsigned int linkIdx = link - tile->header->maxLinkCount;
		tile->dynamicLinks[linkIdx].next = tile->dynamicFreeList;	//@UE
		tile->dynamicFreeList = linkIdx;							//@UE
	}
#if WITH_NAVMESH_CLUSTER_LINKS
	else
	{
		const unsigned int linkIdx = link - DT_CLINK_FIRST;
		tile->dynamicLinksC[linkIdx].next = tile->dynamicFreeListC;
		tile->dynamicFreeListC = linkIdx;
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END
}

dtNavMesh* dtAllocNavMesh()
{
	void* mem = dtAlloc(sizeof(dtNavMesh), DT_ALLOC_PERM_NAVMESH);
	if (!mem) return 0;
	return new(mem) dtNavMesh;
}

/// @par
///
/// This function will only free the memory for tiles with the #DT_TILE_FREE_DATA
/// flag set.
void dtFreeNavMesh(dtNavMesh* navmesh)
{
	if (!navmesh) return;
	navmesh->~dtNavMesh();
	dtFree(navmesh, DT_ALLOC_PERM_NAVMESH);
}

//@UE BEGIN
void dtFreeNavMeshTileRuntimeData(dtMeshTile* tile)
{
	tile->dynamicLinks.~dtChunkArray();	//@UE
#if WITH_NAVMESH_CLUSTER_LINKS
	tile->dynamicLinksC.~dtChunkArray();
#endif // WITH_NAVMESH_CLUSTER_LINKS
}

#if WITH_NAVMESH_SEGMENT_LINKS
//////////////////////////////////////////////////////////////////////////////////////////
// Segment type offmesh links

static const unsigned int DT_INVALID_SEGMENT = 0xffffffff;
static const int DT_MAX_OFFMESH_SEGMENT_POINTS = 32;

struct dtOffMeshSegmentIntersection
{
	dtMeshTile* tile;
	unsigned int poly;
	dtReal t;
};

struct dtOffMeshSegmentTileIntersection
{
	dtOffMeshSegmentIntersection points[DT_MAX_OFFMESH_SEGMENT_POINTS];
	int npoints;
};

struct dtOffMeshSegmentIntersectionLink
{
	// Links connect intersection points on different segments so the t values for both edge
	// segments are needed to retrieve the correct intersection points later.
	dtReal tA, tB;
	unsigned int polyA, polyB;
	dtMeshTile* tileA;
	dtMeshTile* tileB;
};

// This class stores pairs of intersection links.
// The pairs found will be sorted by relative t values and then converted into segment parts in order.
struct dtOffMeshSegmentIntersectionLinkPair
{
	dtOffMeshSegmentIntersectionLink links[2];
};

struct dtOffMeshSegmentPart
{
	// Separate t values for A and B edges.
	// Parts are formed from two unique segments requiring their own t values to retrieve the correct end points from the original segments.
	dtReal tA0, tA1, tB0, tB1;
	unsigned short vA0, vA1, vB0, vB1;
	unsigned int polyA, polyB;
	dtMeshTile* tileA;
	dtMeshTile* tileB;
};

struct dtOffMeshSegmentData
{
	dtOffMeshSegmentTileIntersection listA;
	dtOffMeshSegmentTileIntersection listB;
};

inline bool isIntersectionPointEqual(dtReal t0, dtReal t1)
{
	return dtAbs(t0 - t1) < 0.001f;
}

static bool isPolyIntersectingSegment(const dtMeshTile* tile, int polyIdx,
	const dtReal* spos, const dtReal* epos, dtReal& tmin, dtReal& tmax)
{
	dtPoly* poly = &tile->polys[polyIdx];

	// Convert segment to local space instead of converting all vertices to world.
	dtReal sposLocal[3], eposLocal[3]; //@UE
	sposLocal[0] = spos[0] - tile->header->bmin[0]; //@UE
	sposLocal[1] = spos[1] - tile->header->bmin[1]; //@UE
	sposLocal[2] = spos[2] - tile->header->bmin[2]; //@UE
	eposLocal[0] = epos[0] - tile->header->bmin[0]; //@UE
	eposLocal[1] = epos[1] - tile->header->bmin[1]; //@UE
	eposLocal[2] = epos[2] - tile->header->bmin[2]; //@UE

	dtReal verts[DT_VERTS_PER_POLYGON*3];
	for (int i = 0; i < poly->vertCount; i++)
	{
		const dtTileVert& v = tile->verts[poly->verts[i]]; //@UE
		verts[i*3 + 0] = (dtReal)v.x; //@UE
		verts[i*3 + 1] = (dtReal)v.y; //@UE
		verts[i*3 + 2] = (dtReal)v.z; //@UE
	}

	int smin, smax;
	return dtIntersectSegmentPoly2D(sposLocal, eposLocal, verts, poly->vertCount, tmin, tmax, smin, smax); //@UE
}

static void addSegmentIntersections(const dtOffMeshSegmentIntersection* isec, dtOffMeshSegmentIntersection* list, int& nlist)
{
	UE_LOGF(LogSegmentLink, Log, "         %s", __FUNCTION__);
	
	int minIdx = -1;
	int maxIdx = -1;
	for (int i = 0; i < nlist; i++)
	{
		if (isIntersectionPointEqual(list[i].t, isec[0].t))
			minIdx = i;
		
		if (isIntersectionPointEqual(list[i].t, isec[1].t))
			maxIdx = i;
	}

	// min = overwrite if exists or add new one
	if (minIdx < 0)
	{
		UE_LOGF(LogSegmentLink, Log, "            %s minIdx=%i, add seg intersection, tile (%i,%i,%i), poly %i, %f",
			__FUNCTION__, minIdx, isec[0].tile->header->x, isec[0].tile->header->y, isec[0].tile->header->layer, isec[0].poly, isec[0].t);
		list[nlist] = isec[0];
		nlist++;
	}
	else
	{
		UE_LOGF(LogSegmentLink, Log, "            %s override", __FUNCTION__);
		list[minIdx] = isec[0];
	}

	// max = skip if exists or add new one
	if (maxIdx < 0 && nlist < DT_MAX_OFFMESH_SEGMENT_POINTS)
	{
		UE_LOGF(LogSegmentLink, Log, "            %s maxIdx=%i, add seg intersection, tile (%i,%i,%i), poly %i, %f",
			__FUNCTION__, maxIdx, isec[1].tile->header->x, isec[1].tile->header->y, isec[1].tile->header->layer, isec[1].poly, isec[1].t);
		list[nlist] = isec[1];
		nlist++;
	}
}

int segmentIntersectionSorter(const void* i1, const void* i2)
{
	// lesser "t" goes first
	const dtOffMeshSegmentIntersection* a = (const dtOffMeshSegmentIntersection*)i1;
	const dtOffMeshSegmentIntersection* b = (const dtOffMeshSegmentIntersection*)i2;
	return (a->t < b->t) ? -1 : (a->t > b->t) ? 1 : 0;
}

static void gatherSegmentIntersections(dtMeshTile* tile,
	const dtReal* spos, const dtReal* epos,const dtReal radius, const dtReal walkableClimb,
	dtOffMeshSegmentTileIntersection& list)
{
	// get all polys intersecting with segment
	
	UE_LOGF(LogSegmentLink, Log, "      %s for (%i,%i,%i) segment: start (%0.2f, %0.2f, %0.2f) end (%0.2f, %0.2f, %0.2f)",
		__FUNCTION__, tile->header->x, tile->header->y, tile->header->layer,
		spos[0], spos[1], spos[2], epos[0], epos[1], epos[2]);

	// compute bounds including radius
	dtReal segBoundsMin[3], segBoundsMax[3], segRad[3] = { radius, walkableClimb, radius };
	dtVcopy(segBoundsMin, spos);
	dtVcopy(segBoundsMax, spos);
	dtVmin(segBoundsMin, epos);
	dtVmax(segBoundsMax, epos);
	dtVsub(segBoundsMin, segBoundsMin, segRad);
	dtVadd(segBoundsMax, segBoundsMax, segRad);

	if (!dtOverlapBounds(segBoundsMin, segBoundsMax, tile->header->bmin, tile->header->bmax))
		return;

	dtOffMeshSegmentIntersection intersec[2];
	intersec[0].tile = tile;
	intersec[1].tile = tile;

	dtReal bmin[3], bmax[3];
	for (int i = 0; i < tile->header->offMeshBase; i++)
	{
		dtPoly* poly = &tile->polys[i];

		//@UE BEGIN
		dtReal vtmp[3];
		tile->verts[poly->verts[0]].toWorld(tile->header->bmin, vtmp);
		dtVcopy(bmin, vtmp);
		dtVcopy(bmax, vtmp);
		for (int j = 1; j < poly->vertCount; j++)
		{
			tile->verts[poly->verts[j]].toWorld(tile->header->bmin, vtmp);
			dtVmin(bmin, vtmp);
			dtVmax(bmax, vtmp);
		}
		//@UE END

		// use simple AABB overlap test first
		if (dtOverlapBounds(segBoundsMin, segBoundsMax, bmin, bmax))
		{
			// mark intersection
			if (isPolyIntersectingSegment(tile, i, spos, epos, intersec[0].t, intersec[1].t))
			{
				intersec[0].poly = i;
				intersec[1].poly = i;
				addSegmentIntersections(intersec, list.points, list.npoints);

				if (list.npoints >= DT_MAX_OFFMESH_SEGMENT_POINTS)
					break;
			}
		}
	}
}

static dtOffMeshSegmentData* initSegmentIntersection(const dtNavMesh& navMesh, dtMeshTile* tile)
{
	const int segCount = tile->header->offMeshSegConCount;
	if (segCount <= 0)
		return nullptr;

	dtOffMeshSegmentData* segs = (dtOffMeshSegmentData*)dtAlloc(sizeof(dtOffMeshSegmentData)*segCount, DT_ALLOC_TEMP);
	if (segs == nullptr)
		return nullptr;

	UE_LOGF(LogSegmentLink, Log, "   %s for tile (%i,%i,%i)", __FUNCTION__, tile->header->x, tile->header->y, tile->header->layer);
	
	memset(segs, 0, sizeof(dtOffMeshSegmentData)*segCount);
	for (int i = 0; i < segCount; i++)
	{
		dtOffMeshSegmentConnection& con = tile->offMeshSeg[i];
		
		CA_SUPPRESS(6385);
		gatherSegmentIntersections(tile, con.startA, con.endA, con.rad, navMesh.getWalkableClimb(), segs[i].listA);
		gatherSegmentIntersections(tile, con.startB, con.endB, con.rad, navMesh.getWalkableClimb(), segs[i].listB);
	}

	return segs;
}

static void appendSegmentIntersection(const dtNavMesh& navMesh, dtOffMeshSegmentData* seg, const dtMeshTile* tile, dtMeshTile* nei)
{
	if (seg == nullptr)
		return;

	for (int i = 0; i < tile->header->offMeshSegConCount; i++)
	{
		dtOffMeshSegmentConnection& con = tile->offMeshSeg[i];

		gatherSegmentIntersections(nei, con.startA, con.endA, con.rad, navMesh.getWalkableClimb(), seg[i].listA);
		gatherSegmentIntersections(nei, con.startB, con.endB, con.rad, navMesh.getWalkableClimb(), seg[i].listB);
	}
}

int segmentIntersectionLinkPairSorter(const void* i1, const void* i2)
{
	// lesser combined "t" goes first
	const dtOffMeshSegmentIntersectionLinkPair* first = (const dtOffMeshSegmentIntersectionLinkPair*)i1;
	const dtOffMeshSegmentIntersectionLinkPair* second = (const dtOffMeshSegmentIntersectionLinkPair*)i2;

	const int tFirst = (first->links[1].tA - first->links[0].tA) + (first->links[1].tB - first->links[0].tB);
	const int tSecond = (second->links[1].tA - second->links[0].tA) + (second->links[1].tB - second->links[0].tB);

	return (tFirst < tSecond) ? -1 : (tFirst > tSecond) ? 1 : 0;
}

int segmentPartSorter(const void* i1, const void* i2)
{
	// higher combined relative edge length ((tA1-tA0) + (tB1-tB0)) goes first
	const dtOffMeshSegmentPart* a = (const dtOffMeshSegmentPart*)i1;
	const dtOffMeshSegmentPart* b = (const dtOffMeshSegmentPart*)i2;
	const dtReal lenA = (a->tA1 - a->tA0) + (a->tB1 - a->tB0);
	const dtReal lenB = (b->tA1 - b->tA0) + (b->tB1 - b->tB0);
	return (lenA > lenB) ? -1 : (lenA < lenB) ? 1 : 0;
}

// Advance the index into the specified list skipping over sections of the segment that do not intersect any polygons.
// 
// In this image, intersection points are defined with an X. The first two intersection points are owned by PolyA. The fact
// that both points are owned by the same poly signals that there is no polygon that shares the edge where the intersection occurs.
// If PolyA and PolyB shared the edge where the second intersection occurred, then PolyB would be the owner of the second
// intersection point and we would know that there is no gap.
// 
//          PolyA           Gap           PolyB
//      ##############               ##############
//      #            #               #            #
//      #            #               #            #
//      #            # ----Skip----> #            #
//  ----X------------X---------------X------------X---->
//      #            #               #            #
//      #            #               #            #
//      ##############               ##############
//
static void advanceListIndex(int& listIndex, const dtOffMeshSegmentTileIntersection& list)
{
	unsigned int lastPoly = list.points[listIndex].poly;
	const dtMeshTile* lastTile = list.points[listIndex].tile;
	if (++listIndex < list.npoints)
	{
		unsigned int nextPoly = list.points[listIndex].poly;
		const dtMeshTile* nextTile = list.points[listIndex].tile;
		// The next intersection point will be owned by a different poly if the segment continues directly into another polygon on the navmesh.
		// Otherwise, there is a gap and we should not link any part of a segment that intersects a gap.
		if (lastPoly == nextPoly && lastTile == nextTile)
		{
			// This there is a gap between the previous poly intersected and the next poly intersected by this segment.
			// Advance one more index to get to the next poly that is intersected.
			++listIndex;
		}
	}
}

// This function matches points in listA with points in listB that can form a pair of links that can later be converted into segment parts.
// Points in the longer list will be shared with other pairs at most once, but points in the shorter list might be shared as many times
// as it takes to create a link to every point in the shorter list.
// The number of link pairs found is returned.
static int findSegmentIntersectionLinkPairs(const dtOffMeshSegmentTileIntersection& listA, const dtOffMeshSegmentTileIntersection& listB, dtOffMeshSegmentIntersectionLinkPair* outLinkPairs)
{
	// Each points array needs at least two points to create at least one pair of segment links.
	if (listA.npoints < 2 || listB.npoints < 2)
	{
		return 0;
	}

	int nPairs = 0;
	int indexA = 0;
	int indexB = 0;

	// Determine aliases to A and B specific data based on the length of the lists involved.
	bool bListAisLonger = listA.npoints > listB.npoints;
	const dtOffMeshSegmentTileIntersection& longerList = bListAisLonger ? listA : listB;
	const dtOffMeshSegmentTileIntersection& shorterList = bListAisLonger ? listB : listA;
	int& longerListIndex = bListAisLonger ? indexA : indexB;
	int& shorterListIndex = bListAisLonger ? indexB : indexA;

	// Lambda function for convenience to create link pairs.
	auto createSegmentIntersectionLinkPair = [&outLinkPairs, &nPairs, &listA, &listB, &indexA, &indexB]()
	{
		dtOffMeshSegmentIntersectionLink& firstLink = outLinkPairs[nPairs].links[0];
		firstLink.tA = listA.points[indexA].t;
		firstLink.polyA = listA.points[indexA].poly;
		firstLink.tileA = listA.points[indexA].tile;
		firstLink.tB = listB.points[indexB].t;
		firstLink.polyB = listB.points[indexB].poly;
		firstLink.tileB = listB.points[indexB].tile;

		dtOffMeshSegmentIntersectionLink& secondLink = outLinkPairs[nPairs].links[1];
		secondLink.tA = listA.points[indexA + 1].t;
		secondLink.polyA = listA.points[indexA + 1].poly;
		secondLink.tileA = listA.points[indexA + 1].tile;
		secondLink.tB = listB.points[indexB + 1].t;
		secondLink.polyB = listB.points[indexB + 1].poly;
		secondLink.tileB = listB.points[indexB + 1].tile;

		nPairs++;
	};

	// Create the first pair of links which is guaranteed to connect the first two points in listA with the first two points in listB.
	createSegmentIntersectionLinkPair();

	// Advance to the next point in the longer list looking for the next pair of links.
	advanceListIndex(longerListIndex, longerList);

	// Iterate over the longer list since it is known that the pair of points used in the longer list will increment after each segment link pair is created.
	while (longerListIndex < longerList.npoints - 1)
	{
		// Calculate the number of points remaining in each list that have not been considered for pairing yet.
		const int numRemainingLongListPoints = longerList.npoints - longerListIndex - 2;
		const int numRemainingShortListPoints = shorterList.npoints - shorterListIndex - 2;

		// Determine if the index to the pair of points used in the shorter list should increment based on
		// which points in the shorter list have closer t values to the current points in the longer list.
		if (numRemainingShortListPoints > 0)
		{
			// Ensure every point in the shorter list also gets a connection to a point in the longer list regardless of t values.
			if (numRemainingShortListPoints > numRemainingLongListPoints)
			{
				advanceListIndex(shorterListIndex, shorterList);
			}
			else
			{
				int currentTDiff = dtAbs(longerList.points[longerListIndex].t - shorterList.points[shorterListIndex].t) +
					dtAbs(longerList.points[longerListIndex + 1].t - shorterList.points[shorterListIndex + 1].t);

				int nextTDiff = dtAbs(longerList.points[longerListIndex].t - shorterList.points[shorterListIndex + 1].t) +
					dtAbs(longerList.points[longerListIndex + 1].t - shorterList.points[shorterListIndex + 2].t);

				if (nextTDiff < currentTDiff)
				{
					// The next pair of points in the shorter list are more appropriate to link to the current points in the longer list.
					advanceListIndex(shorterListIndex, shorterList);
				}
			}
		}

		// Create the next pair of segment intersection links based on the currently selected points in both lists.
		createSegmentIntersectionLinkPair();

		// Advance to the next point in the longer list looking for the next pair of links.
		advanceListIndex(longerListIndex, longerList);
	}

	return nPairs;
}

static unsigned short findOrAddUniqueValue(dtReal value, dtReal* array, unsigned short& narray)
{
	for (unsigned short i = 0; i < narray; i++)
	{
		if (array[i] == value)
			return i;
	}

	const unsigned short pos = narray;
	array[pos] = value;
	narray++;
	return pos;
}

static void createSegmentParts(const dtMeshTile* tile, const dtOffMeshSegmentData& segData,
	dtOffMeshSegmentPart* parts, const int maxParts, int& nparts, unsigned short& nverts)
{
	if (segData.listA.npoints <= 0 && segData.listB.npoints <= 0)
		return;

	UE_LOGF(LogSegmentLink, Log, "   %s (%i,%i,%i)", __FUNCTION__, tile->header->x, tile->header->y, tile->header->layer);
	
	// Create dtOffMeshSegmentIntersectionLinkPair list.
	const int maxLinkPairs = DT_MAX_OFFMESH_SEGMENT_POINTS - 1;
	dtOffMeshSegmentIntersectionLinkPair linkPairs[maxLinkPairs];
	memset(linkPairs, 0, sizeof(dtOffMeshSegmentIntersectionLinkPair) * maxLinkPairs);

	int nlinkPairs = findSegmentIntersectionLinkPairs(segData.listA, segData.listB, linkPairs);

	// There needs to be at least one pair of links to create a segment part.
	if (nlinkPairs < 1)
	{
		return;
	}

	UE_LOGF(LogSegmentLink, Log, "   %s %i nlinkPairs", __FUNCTION__, nlinkPairs);

	qsort(linkPairs, nlinkPairs, sizeof(dtOffMeshSegmentIntersectionLinkPair), segmentIntersectionLinkPairSorter);

	// create segments
	memset(parts, 0, sizeof(dtOffMeshSegmentPart)*maxParts);
	nparts = 0;

	// Create segment parts for link pairs that belong to the current tile.
	for (int i = 0; i < nlinkPairs; i++)
	{
		const dtOffMeshSegmentIntersectionLinkPair& linkPair = linkPairs[i];
		
		if (linkPair.links[0].tileA == tile || linkPair.links[0].tileB == tile ||
			linkPair.links[1].tileA == tile || linkPair.links[1].tileB == tile)
		{
			dtOffMeshSegmentPart& part = parts[nparts];

			part.tA0 = linkPair.links[0].tA;
			part.tA1 = linkPair.links[1].tA;
			part.tB0 = linkPair.links[0].tB;
			part.tB1 = linkPair.links[1].tB;
			part.polyA = linkPair.links[0].polyA;
			part.polyB = linkPair.links[0].polyB;
			part.tileA = linkPair.links[0].tileA;
			part.tileB = linkPair.links[0].tileB;

			nparts++;
			
			UE_LOGF(LogSegmentLink, Log, "   %s seg part %i created tA: %f, %f, tb: %f, %f, tileA (%i,%i,%i), tileB (%i,%i,%i), tile=%p, tileA=%p, tileB=%p",
				__FUNCTION__, nparts,
				part.tA0, part.tA1,
				part.tB0, part.tB1,
				part.tileA->header->x, part.tileA->header->y, part.tileA->header->layer,
				part.tileB->header->x, part.tileB->header->y, part.tileB->header->layer,
				tile, part.tileA, part.tileB);
		}
	}

	// sort positions
	if (nparts > DT_MAX_OFFMESH_SEGMENT_PARTS)
	{
		qsort(parts, nparts, sizeof(dtOffMeshSegmentPart), segmentPartSorter);
		nparts = DT_MAX_OFFMESH_SEGMENT_PARTS;
	}

	// count unique verts
	dtReal uniquePos[DT_MAX_OFFMESH_SEGMENT_PARTS*2];
	unsigned short nPos = 0;
	for (int i = 0; i < nparts; i++)
	{
		// Use specific t values for the A segment part.
		parts[i].vA0 = findOrAddUniqueValue(parts[i].tA0, uniquePos, nPos);
		parts[i].vA1 = findOrAddUniqueValue(parts[i].tA1, uniquePos, nPos);
	}
	
	dtReal uniquePosB[DT_MAX_OFFMESH_SEGMENT_PARTS*2];
	unsigned short nPosB = 0;

	for (int i = 0; i < nparts; i++)
	{
		// Use specific t values for the B segment part.
		// B values can also be unique.
		parts[i].vB0 = findOrAddUniqueValue(parts[i].tB0, uniquePosB, nPosB) + nPos;
		parts[i].vB1 = findOrAddUniqueValue(parts[i].tB1, uniquePosB, nPosB) + nPos;
	}

	nverts = (nPos + nPosB) * 2;
	
	UE_LOGF(LogSegmentLink, Log, "   %s %i nparts created", __FUNCTION__, nparts);
}

static void createSegmentPolys(dtNavMesh* nav, dtMeshTile* tile, dtOffMeshSegmentConnection* con,
	dtOffMeshSegmentPart* parts, int nparts, unsigned short vertBase, int polyBase)
{
	dtReal lenA[3], lenB[3];
	dtVsub(lenA, con->endA, con->startA);
	dtVsub(lenB, con->endB, con->startB);

	unsigned char sideFwd = DT_LINK_FLAG_OFFMESH_CON | (con->getBiDirectional() ? DT_LINK_FLAG_OFFMESH_CON_BIDIR : 0);
	unsigned char sideBck = sideFwd | DT_LINK_FLAG_OFFMESH_CON_BACKTRACKER;
	con->firstPoly = (unsigned short)(polyBase - tile->header->offMeshSegPolyBase);
	con->npolys = (nparts > 0 && nparts < 256) ? (unsigned char)nparts : 0;

	for (int i = 0; i < nparts; i++)
	{
		dtOffMeshSegmentPart* it = &parts[i];

		//@UE BEGIN
		// add verts
		dtReal wp[3];

		dtVmad(wp, con->startA, lenA, it->tA0);
		tile->verts[vertBase+it->vA0].setFromWorld(tile->header->bmin, wp);

		dtVmad(wp, con->startA, lenA, it->tA1);
		tile->verts[vertBase+it->vA1].setFromWorld(tile->header->bmin, wp);

		dtVmad(wp, con->startB, lenB, it->tB0);
		tile->verts[vertBase+it->vB0].setFromWorld(tile->header->bmin, wp);

		dtVmad(wp, con->startB, lenB, it->tB1);
		tile->verts[vertBase+it->vB1].setFromWorld(tile->header->bmin, wp);
		//@UE END

		// add poly
		dtPoly* poly = &tile->polys[polyBase + i];
		poly->vertCount = 4;
		poly->verts[0] = vertBase + it->vA0;
		poly->verts[1] = vertBase + it->vA1;
		poly->verts[2] = vertBase + it->vB0;
		poly->verts[3] = vertBase + it->vB1;
		poly->firstLink = DT_NULL_LINK;

		// add links
		const unsigned char sideA = (tile == it->tileA) ? DT_CONNECTION_INTERNAL : 0;
		const unsigned char sideB = (tile == it->tileB) ? DT_CONNECTION_INTERNAL : 0;
		nav->linkOffMeshHelper(tile, polyBase + i, it->tileA, it->polyA, sideBck | sideA, 0);
		nav->linkOffMeshHelper(tile, polyBase + i, it->tileB, it->polyB, sideFwd | sideB, 1);
		nav->linkOffMeshHelper(it->tileA, it->polyA, tile, polyBase + i, sideFwd | sideA, 0xff);
		nav->linkOffMeshHelper(it->tileB, it->polyB, tile, polyBase + i, sideBck | sideB, 0xff);
	}
}

static void createSegmentLinks(dtNavMesh* nav, dtOffMeshSegmentData* seg, dtMeshTile* tile)
{
	if (seg == nullptr)
		return;

	UE_LOGF(LogSegmentLink, Log, "%s (%i,%i,%i)", __FUNCTION__, tile->header->x, tile->header->y, tile->header->layer);

	unsigned short vertBase = tile->header->offMeshSegVertBase;
	int polyBase = tile->header->offMeshSegPolyBase;
	for (int i = 0; i < tile->header->offMeshSegConCount; i++)
	{
		dtOffMeshSegmentConnection& con = tile->offMeshSeg[i];
		dtOffMeshSegmentData& segData = seg[i];

		qsort(segData.listA.points, segData.listA.npoints, sizeof(dtOffMeshSegmentIntersection), segmentIntersectionSorter);
		qsort(segData.listB.points, segData.listB.npoints, sizeof(dtOffMeshSegmentIntersection), segmentIntersectionSorter);

		const int maxParts = (DT_MAX_OFFMESH_SEGMENT_POINTS * 2) - 1;
		dtOffMeshSegmentPart parts[maxParts];
		int nparts = 0;
		unsigned short nverts = 0;
		createSegmentParts(tile, segData, parts, maxParts, nparts, nverts);

		createSegmentPolys(nav, tile, &con, parts, nparts, vertBase, polyBase);
		vertBase += nverts;

		// The polys array is allocated enough memory to contain the max off-mesh segment parts when it is created.
		// Therefore, polyBase must be incremented by the max off-mesh segment parts count for subsequent connections to access the correct data.
		// See dtCreateNavMeshData for polys array creation and initialization.
		polyBase += DT_MAX_OFFMESH_SEGMENT_PARTS;
	}
}
#endif // WITH_NAVMESH_SEGMENT_LINKS
//@UE END

//////////////////////////////////////////////////////////////////////////////////////////

/**
@class dtNavMesh

The navigation mesh consists of one or more tiles defining three primary types of structural data:

A polygon mesh which defines most of the navigation graph. (See rcPolyMesh for its structure.)
A detail mesh used for determining surface height on the polygon mesh. (See rcPolyMeshDetail for its structure.)
Off-mesh connections, which define custom point-to-point edges within the navigation graph.

The general build process is as follows:

-# Create rcPolyMesh and rcPolyMeshDetail data using the Recast build pipeline.
-# Optionally, create off-mesh connection data.
-# Combine the source data into a dtNavMeshCreateParams structure.
-# Create a tile data array using dtCreateNavMeshData().
-# Allocate at dtNavMesh object and initialize it. (For single tile navigation meshes,
   the tile data is loaded during this step.)
-# For multi-tile navigation meshes, load the tile data using dtNavMesh::addTile().

Notes:

- This class is usually used in conjunction with the dtNavMeshQuery class for pathfinding.
- Technically, all navigation meshes are tiled. A 'solo' mesh is simply a navigation mesh initialized 
  to have only a single tile.
- This class does not implement any asynchronous methods. So the ::dtStatus result of all methods will 
  always contain either a success or failure flag.

@see dtNavMeshQuery, dtCreateNavMeshData, dtNavMeshCreateParams, #dtAllocNavMesh, #dtFreeNavMesh
*/

dtNavMesh::dtNavMesh() :
	m_tileWidth(0),
	m_tileHeight(0),
	m_maxTiles(0),
	m_tileLutSize(0),
	m_tileLutMask(0),
	m_posLookup(0),
	m_nextFree(0),
	m_tiles(0),
	m_saltBits(0),
	m_tileBits(0),
	m_polyBits(0)
{
	memset(&m_params, 0, sizeof(dtNavMeshParams));
	memset(m_areaCostOrder, 0, sizeof(m_areaCostOrder));
	m_bDeferOffMeshLinking = false;
	m_orig[0] = 0;
	m_orig[1] = 0;
	m_orig[2] = 0;
}

dtNavMesh::~dtNavMesh()
{
	for (int i = 0; i < m_maxTiles; ++i)
	{
		if (m_tiles[i].flags & DT_TILE_FREE_DATA)
		{
			dtMeshTile& meshTile = m_tiles[i];

			dtStatsPreRemoveTile(meshTile);

			dtFree(meshTile.data, DT_ALLOC_PERM_TILE_DATA);
			meshTile.data = 0;
			meshTile.dataSize = 0;
		}

		// cleanup runtime data (not serialized by navmesh owners)
		dtFreeNavMeshTileRuntimeData(&m_tiles[i]);
	}
	dtFree(m_posLookup, DT_ALLOC_PERM_LOOKUP);
	dtFree(m_tiles, DT_ALLOC_PERM_TILES);
}
		
dtStatus dtNavMesh::init(const dtNavMeshParams* params)
{
	memcpy(&m_params, params, sizeof(dtNavMeshParams));
	dtVcopy(m_orig, params->orig);

	// @UE BEGIN
#if DO_CHECK	
	for (uint8 i = 0; i < DT_RESOLUTION_COUNT; i++)
	{
		check(m_params.resolutionParams[i].bvQuantFactor != 0);
	}
#endif // DO_CHECK	
	// @UE END
	
	m_tileWidth = params->tileWidth;
	m_tileHeight = params->tileHeight;
	
	// Init tiles
	m_maxTiles = params->maxTiles;
	m_tileLutSize = dtNextPow2(params->maxTiles/4);
	if (!m_tileLutSize) m_tileLutSize = 1;
	m_tileLutMask = m_tileLutSize-1;
	
	m_tiles = (dtMeshTile*)dtAlloc(sizeof(dtMeshTile)*m_maxTiles, DT_ALLOC_PERM_TILES);
	if (!m_tiles)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	m_posLookup = (dtMeshTile**)dtAlloc(sizeof(dtMeshTile*)*m_tileLutSize, DT_ALLOC_PERM_LOOKUP);
	if (!m_posLookup)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset((void*)m_tiles, 0, sizeof(dtMeshTile)*m_maxTiles);
	memset(m_posLookup, 0, sizeof(dtMeshTile*)*m_tileLutSize);
	m_nextFree = 0;
	for (int i = m_maxTiles-1; i >= 0; --i)
	{
		m_tiles[i].salt = DT_SALT_BASE;
		m_tiles[i].next = m_nextFree;
		m_nextFree = &m_tiles[i];
	}

	// Init ID generator values.
	m_tileBits = dtIlog2(dtNextPow2((unsigned int)params->maxTiles));
	m_polyBits = dtIlog2(dtNextPow2((unsigned int)params->maxPolys));
	// Only allow 31 salt bits, since the salt mask is calculated using 32bit uint and it will overflow.
#if USE_64BIT_ADDRESS
	m_saltBits = dtMin((unsigned int)31, 64 - m_tileBits - m_polyBits);
#else
	m_saltBits = dtMin((unsigned int)31, 32 - m_tileBits - m_polyBits);
#endif
	if (m_saltBits < DT_MIN_SALT_BITS)
		return DT_FAILURE | DT_INVALID_PARAM;
	
	return DT_SUCCESS;
}

dtStatus dtNavMesh::init(unsigned char* data, const int dataSize, const int flags)
{
	// Make sure the data is in right format.
	dtMeshHeader* header = (dtMeshHeader*)data;
	if (header->version != DT_NAVMESH_VERSION)
		return DT_FAILURE | DT_WRONG_VERSION;

	dtNavMeshParams params;
	dtVcopy(params.orig, header->bmin);
	params.tileWidth = header->bmax[0] - header->bmin[0];
	params.tileHeight = header->bmax[2] - header->bmin[2];
	params.maxTiles = 1;
	params.maxPolys = header->polyCount;
	
	dtStatus status = init(&params);
	if (dtStatusFailed(status))
		return status;

	return addTile(data, dataSize, flags, 0, 0);
}

/// @par
///
/// @note The parameters are created automatically when the single tile
/// initialization is performed.
const dtNavMeshParams* dtNavMesh::getParams() const
{
	return &m_params;
}

//////////////////////////////////////////////////////////////////////////////////////////
int dtNavMesh::findConnectingPolys(const dtReal* va, const dtReal* vb,
	const dtMeshTile* fromTile, int fromPolyIdx,
	const dtMeshTile* tile, int side,
	dtChunkArray<FConnectingPolyData>& cons) const
{
	if (!tile) return 0;

	dtReal amin[2], amax[2], apt[3];
	amin[0] = 0;
	amin[1] = 0;
	amax[0] = 0;
	amax[1] = 0;
	
	calcSlabEndPoints(va, vb, amin, amax, side);
	const dtReal apos = getSlabCoord(va, side);
	dtVcopy(apt, va);

	// Remove links pointing to 'side' and compact the links array. 
	dtReal bmin[2], bmax[2], bpt[3];
	bmin[0] = 0;
	bmin[1] = 0;
	bmax[0] = 0;
	bmax[1] = 0;
	
	unsigned short m = DT_EXT_LINK | (unsigned short)side;
	int n = 0;

	dtPolyRef base = getPolyRefBase(tile);

	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		dtPoly* poly = &tile->polys[i];
		const int nv = poly->vertCount;
		for (int j = 0; j < nv; ++j)
		{
			// Skip edges which do not point to the right side.
			if (poly->neis[j] != m) continue;

			dtReal vc[3]; //@UE
			tile->verts[poly->verts[j]].toWorld(tile->header->bmin, vc); //@UE
			const dtReal bpos = getSlabCoord(vc, side);

			// Segments are not close enough.
			if (dtAbs(apos - bpos) > 0.01f)
				continue;

			dtReal vd[3]; //@UE
			tile->verts[poly->verts[(j + 1) % nv]].toWorld(tile->header->bmin, vd); //@UE

			// Check if the segments touch.
			calcSlabEndPoints(vc, vd, bmin, bmax, side);

			unsigned char overlapMode = 0;
			if (!overlapSlabs(amin, amax, bmin, bmax, 0.01f, getWalkableClimb(), &overlapMode)) continue;

			// if overlapping with only one side, verify height difference using detailed mesh
			if (overlapMode == SLABOVERLAP_Max || overlapMode == SLABOVERLAP_Min)
			{
				dtVcopy(bpt, vc);
				const int coordIdx = (side == 0 || side == 4) ? 2 : 0;
				apt[coordIdx] = (overlapMode == SLABOVERLAP_Min) ? dtMax(amin[0], bmin[0]) : dtMin(amax[0], bmax[0]);
				bpt[coordIdx] = apt[coordIdx];

				const dtReal aH = getHeightFromDMesh(fromTile, fromPolyIdx, apt);
				const dtReal bH = getHeightFromDMesh(tile, i, bpt);
				const dtReal heightDiff = dtAbs(aH - bH);
				if (heightDiff > getWalkableClimb())
					continue;
			}

			// Add return value.
			FConnectingPolyData NewPolyData;
			NewPolyData.min = dtMax(amin[0], bmin[0]);
			NewPolyData.max = dtMin(amax[0], bmax[0]);
			NewPolyData.ref = base | (dtPolyRef)i;
			cons.push(NewPolyData);
			n++;
			break;
		}
	}

	return n;
}

void dtNavMesh::unconnectExtLinks(dtMeshTile* tile, dtMeshTile* target)
{
	if (!tile || !target) return;

	const unsigned int targetNum = decodePolyIdTile(getTileRef(target));

	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		dtPoly* poly = &tile->polys[i];
		unsigned int j = poly->firstLink;
		unsigned int pj = DT_NULL_LINK;
		while (j != DT_NULL_LINK)
		{
			dtLink& testLink = getLink(tile, j);
//@UE BEGIN
			if ((testLink.side & DT_CONNECTION_INTERNAL) == 0 &&
//@UE END
				decodePolyIdTile(testLink.ref) == targetNum)
			{
				// Remove link.
				unsigned int nj = testLink.next;
				if (pj == DT_NULL_LINK)
				{
					poly->firstLink = nj;
				}
				else
				{
					dtLink& prevLink = getLink(tile, pj);
					prevLink.next = nj;
				}
				freeLink(tile, j);
				j = nj;
			}
			else
			{
				// Advance
				pj = j;
				j = testLink.next;
			}
		}
	}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	unconnectClusterLinks(tile, target);
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END
}

void dtNavMesh::connectExtLinks(dtMeshTile* tile, dtMeshTile* target, int side, bool updateCLinks)
{
	if (!tile) return;
	
	dtChunkArray<FConnectingPolyData> cons(16);

	// Connect border links.
	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		dtPoly* poly = &tile->polys[i];

		// Create new links.
		const int nv = poly->vertCount;
		for (int j = 0; j < nv; ++j)
		{
			// Skip non-portal edges.
			if ((poly->neis[j] & DT_EXT_LINK) == 0)
				continue;
			
			const int dir = (int)(poly->neis[j] & 0xff);
			if (side != -1 && dir != side)
				continue;
			
			// Create new links
			dtReal va[3], vb[3]; //@UE
			tile->verts[poly->verts[j]].toWorld(tile->header->bmin, va); //@UE
			tile->verts[poly->verts[(j+1) % nv]].toWorld(tile->header->bmin, vb); //@UE

			// reset array before using
			cons.resize(0);
			findConnectingPolys(va,vb, tile, i, target, dtOppositeTile(dir), cons);

			for (int k = 0; k < cons.size(); ++k)
			{
				const FConnectingPolyData& NeiData = cons[k];
				unsigned int idx = allocLink(tile, CREATE_LINK_PREALLOCATED);
				if (idx != DT_NULL_LINK)
				{
					dtLink& link = getLink(tile, idx);	//@UE
					link.ref = NeiData.ref;				//@UE
					link.edge = (unsigned char)j;		//@UE
					link.side = (unsigned char)dir;		//@UE

					link.next = poly->firstLink;		//@UE
					poly->firstLink = idx;

					// Compress portal limits to a byte value.
					if (dir == 0 || dir == 4)
					{
						dtReal tmin = (NeiData.min-va[2]) / (vb[2]-va[2]);
						dtReal tmax = (NeiData.max-va[2]) / (vb[2]-va[2]);
						if (tmin > tmax)
							dtSwap(tmin,tmax);
						link.bmin = (unsigned char)(dtClamp(tmin, 0.0f, 1.0f)*255.0f);	//@UE
						link.bmax = (unsigned char)(dtClamp(tmax, 0.0f, 1.0f)*255.0f);	//@UE
					}
					else if (dir == 2 || dir == 6)
					{
						dtReal tmin = (NeiData.min-va[0]) / (vb[0]-va[0]);
						dtReal tmax = (NeiData.max-va[0]) / (vb[0]-va[0]);
						if (tmin > tmax)
							dtSwap(tmin,tmax);
						link.bmin = (unsigned char)(dtClamp(tmin, 0.0f, 1.0f)*255.0f);	//@UE
						link.bmax = (unsigned char)(dtClamp(tmax, 0.0f, 1.0f)*255.0f);	//@UE
					}
				}

				//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
				if (updateCLinks)
				{
					unsigned int targetIdx = decodePolyIdPoly(NeiData.ref);
					if (tile->polyClusters && target->polyClusters &&
						i < tile->header->offMeshBase &&
						targetIdx < (unsigned int)target->header->offMeshBase)
					{
						connectClusterLink(tile, tile->polyClusters[i], target, target->polyClusters[targetIdx], DT_CLINK_VALID_FWD);
						connectClusterLink(target, target->polyClusters[targetIdx], tile, tile->polyClusters[i], DT_CLINK_VALID_BCK);
					}
				}
#endif // WITH_NAVMESH_CLUSTER_LINKS
				//@UE END
			}
		}
	}
}

//@UE BEGIN
void dtNavMesh::linkOffMeshHelper(dtMeshTile* tile0, unsigned int polyIdx0, const dtMeshTile* tile1, unsigned int polyIdx1, unsigned char side, unsigned char edge)
{
	dtPoly* poly0 = &tile0->polys[polyIdx0];

	const unsigned int idx = allocLink(tile0, CREATE_LINK_DYNAMIC_OFFMESH);
	dtLink& link = getLink(tile0, idx);

	link.ref = getPolyRefBase(tile1) | (dtPolyRef)polyIdx1;
	link.edge = edge;
	link.side = side;
	link.bmin = link.bmax = 0;
	link.next = poly0->firstLink;
	poly0->firstLink = idx;
}
//@UE END

void dtNavMesh::connectExtOffMeshLinks(dtMeshTile* tile, dtMeshTile* target, int side, bool updateCLinks)
{
	if (!tile) return;
	
	// Connect off-mesh links.
	// We are interested on links which land from target tile to this tile.
	//@UE BEGIN
	const unsigned char oppositeSide = (side == -1) ? DT_CONNECTION_INTERNAL : (unsigned char)dtOppositeTile(side);

	for (int i = 0; i < target->header->offMeshConCount; ++i)
	{
		dtOffMeshConnection* targetCon = &target->offMeshCons[i];
		if (targetCon->side != oppositeSide)
			continue;

		const unsigned char biDirFlag = targetCon->getBiDirectional() ? DT_LINK_FLAG_OFFMESH_CON_BIDIR : 0;

		dtPoly* targetPoly = &target->polys[targetCon->poly];
		// Skip off-mesh connections which start location could not be connected at all.
		if (targetPoly->firstLink == DT_NULL_LINK)
			continue;
		
		const dtLink& targetLink = getLink(target, targetPoly->firstLink);
		const dtPolyRef targetLandPoly = targetLink.ref;
		const dtReal ext[3] = { targetCon->rad, targetCon->height, targetCon->rad };

		// Find polygon to connect to.
		const dtReal* p = &targetCon->pos[3];
		dtReal nearestPt[3];
		dtPolyRef ref = 0;

		// [UE] try finding cheapest, but it that's outside requested radius, fallback to nearest one
		// findNearestPoly may return too optimistic results, further check to make sure. 
		if (targetCon->getSnapToCheapestArea())
		{
			ref = findCheapestNearPolyInTile(tile, p, ext, nearestPt);
			if (!ref || (ref == targetLandPoly) || (dtSqr(nearestPt[0] - p[0]) + dtSqr(nearestPt[2] - p[2]) > dtSqr(targetCon->rad)))
			{
				ref = 0;
			}
		}

		if (!ref)
		{
			ref = findNearestPolyInTile(tile, p, ext, nearestPt, true);
			if (!ref || (ref == targetLandPoly) || (dtSqr(nearestPt[0] - p[0]) + dtSqr(nearestPt[2] - p[2]) > dtSqr(targetCon->rad)))
			{
				ref = 0;
			}
		}

		// Avoid linking back into the same ground poly
		if (!ref || (targetLandPoly == ref))
			continue;

		// For reversed links, the landing vertex is verts[0] instead of verts[1].
		const bool bReversed = targetCon->getIsReversed();
		const int landingVertIdx = bReversed ? 0 : 1;
		target->verts[targetPoly->verts[landingVertIdx]].setFromWorld(target->header->bmin, nearestPt); //@UE

		// Link off-mesh connection to landing poly.
		// For reversed links, this is the backtracker direction.
		unsigned char fwdLinkSide = oppositeSide | DT_LINK_FLAG_OFFMESH_CON | biDirFlag;
		if (tile != target)
		{
			fwdLinkSide &= ~DT_CONNECTION_INTERNAL;
		}
		if (bReversed && biDirFlag == 0)
		{
			fwdLinkSide |= DT_LINK_FLAG_OFFMESH_CON_BACKTRACKER;
		}

		// Link off-mesh connection to target poly.
		const unsigned int landPolyIdx = decodePolyIdPoly(ref);
		const unsigned char landingEdge = bReversed ? 0 : 1;

		linkOffMeshHelper(target, targetCon->poly, tile, landPolyIdx, fwdLinkSide, landingEdge);

		// Link landing poly back to off-mesh connection.
		// For non-reversed unidirectional links, this is the backtracker direction.
		unsigned char bckLinkSide = (unsigned char)(side == -1 ? DT_CONNECTION_INTERNAL : side) | DT_LINK_FLAG_OFFMESH_CON | biDirFlag;
		if (tile != target)
		{
			bckLinkSide &= ~DT_CONNECTION_INTERNAL;
		}
		if (!bReversed && biDirFlag == 0)
		{
			bckLinkSide |= DT_LINK_FLAG_OFFMESH_CON_BACKTRACKER;
		}

		linkOffMeshHelper(tile, landPolyIdx, target, targetCon->poly, bckLinkSide, 0xff);

#if WITH_NAVMESH_CLUSTER_LINKS
		if (updateCLinks)
		{
			unsigned int targetPolyIdx = decodePolyIdPoly(targetLandPoly);
			unsigned int thisPolyIdx = landPolyIdx;
			if (thisPolyIdx < (unsigned int)tile->header->offMeshBase &&
				targetPolyIdx < (unsigned int)target->header->offMeshBase &&
				tile->polyClusters && target->polyClusters)
			{
				unsigned int targetClusterIdx = target->polyClusters[targetPolyIdx];
				unsigned int thisClusterIdx = tile->polyClusters[thisPolyIdx];
				const bool bUniqueCheck = true;

				const unsigned char baseFwd = bReversed ? DT_CLINK_VALID_BCK : DT_CLINK_VALID_FWD;
				const unsigned char baseBck = bReversed ? DT_CLINK_VALID_FWD : DT_CLINK_VALID_BCK;
				const unsigned char flagsFwd = baseFwd | (biDirFlag ? baseBck : 0);
				const unsigned char flagsBck = baseBck | (biDirFlag ? baseFwd : 0);

				connectClusterLink(target, targetClusterIdx, tile, thisClusterIdx, flagsFwd, bUniqueCheck);
				connectClusterLink(tile, thisClusterIdx, target, targetClusterIdx, flagsBck, bUniqueCheck);
			}
		}
#endif // WITH_NAVMESH_CLUSTER_LINKS
	}
	//@UE END
}

void dtNavMesh::connectIntLinks(dtMeshTile* tile)
{
	if (!tile) return;

	dtPolyRef base = getPolyRefBase(tile);

	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		dtPoly* poly = &tile->polys[i];
		poly->firstLink = DT_NULL_LINK;

		if (poly->getType() != DT_POLYTYPE_GROUND)
			continue;
			
		// Build edge links backwards so that the links will be
		// in the linked list from lowest index to highest.
		for (int j = poly->vertCount-1; j >= 0; --j)
		{
			// Skip hard and non-internal edges.
			if (poly->neis[j] == 0 || (poly->neis[j] & DT_EXT_LINK)) continue;

			unsigned int idx = allocLink(tile, CREATE_LINK_PREALLOCATED);
			if (idx != DT_NULL_LINK)
			{
				dtLink& link = getLink(tile, idx);				//@UE
				link.ref = base | (dtPolyRef)(poly->neis[j]-1); //@UE
				link.edge = (unsigned char)j;					//@UE

				link.side = DT_CONNECTION_INTERNAL;				//@UE

				link.bmin = link.bmax = 0;						//@UE
				// Add to linked list.
				link.next = poly->firstLink;					//@UE
				poly->firstLink = idx;
			}
		}
	}
}

void dtNavMesh::baseOffMeshLinks(dtMeshTile* tile)
{
	if (!tile) return;
	
	// Base off-mesh connection start points.
	for (int i = 0; i < tile->header->offMeshConCount; ++i)
	{
		dtOffMeshConnection* con = &tile->offMeshCons[i];
		dtPoly* poly = &tile->polys[con->poly];
	
		const dtReal ext[3] = { con->rad, con->height, con->rad };
		
		// Find polygon to connect to.
		const dtReal* p = &con->pos[0]; // First vertex
		dtReal nearestPt[3];
		dtPolyRef ref = 0;

		// [UE] try finding cheapest, but it that's outside requested radius, fallback to nearest one
		// findNearestPoly may return too optimistic results, further check to make sure. 
		if (con->getSnapToCheapestArea())
		{
			ref = findCheapestNearPolyInTile(tile, p, ext, nearestPt);
			if (!ref || (dtSqr(nearestPt[0] - p[0]) + dtSqr(nearestPt[2] - p[2]) > dtSqr(con->rad)))
			{
				ref = 0;
			}
		}

		if (!ref)
		{
			ref = findNearestPolyInTile(tile, p, ext, nearestPt, true);
			if (!ref || (dtSqr(nearestPt[0] - p[0]) + dtSqr(nearestPt[2] - p[2]) > dtSqr(con->rad)))
			{
				ref = 0;
			}
		}

		if (!ref) continue;

		// Make sure the location is on current mesh.
		const bool bReversed = con->getIsReversed();
		const int baseVertIdx = bReversed ? 1 : 0;
		tile->verts[poly->verts[baseVertIdx]].setFromWorld(tile->header->bmin, nearestPt); //@UE

		unsigned char sideFwd = DT_CONNECTION_INTERNAL | DT_LINK_FLAG_OFFMESH_CON | (con->getBiDirectional() ? DT_LINK_FLAG_OFFMESH_CON_BIDIR : 0);
		unsigned char sideBck = sideFwd | DT_LINK_FLAG_OFFMESH_CON_BACKTRACKER;

		// For reversed links, swap forward/backward so that the normal traversal
		// direction goes from pos[3] to pos[0] (i.e. B0 to A0).
		const unsigned char sideToLocal = bReversed ? sideFwd : sideBck;
		const unsigned char sideFromLocal = bReversed ? sideBck : sideFwd;
		const int baseEdge = bReversed ? 1 : 0;

		// Link off-mesh connection to local ground poly.
		linkOffMeshHelper(tile, con->poly, tile, decodePolyIdPoly(ref), sideToLocal, baseEdge);

		// Link local ground poly back to off-mesh connection.
		linkOffMeshHelper(tile, decodePolyIdPoly(ref), tile, con->poly, sideFromLocal, 0xff);
	}
}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
void dtNavMesh::connectClusterLink(dtMeshTile* tile0, unsigned int clusterIdx0, dtMeshTile* tile1, unsigned int clusterIdx1, unsigned char flags, bool bCheckExisting)
{
	if (tile0 == tile1 && clusterIdx0 == clusterIdx1)
		return;

	dtCluster& cluster0 = tile0->clusters[clusterIdx0];
	dtClusterRef cluster1Ref = getClusterRefBase(tile1) | (dtClusterRef)clusterIdx1;

	dtClusterLink* link = 0;

	// check if already connected
	if (bCheckExisting)
	{
		unsigned int i = cluster0.firstLink;
		while (i != DT_NULL_LINK)
		{
			dtClusterLink& testLink = getClusterLink(tile0, i);
			if (testLink.ref == cluster1Ref)
			{
				link = &testLink;
				break;
			}

			i = testLink.next;
		}
	}

	if (!link)
	{
		// add new link
		unsigned int linkIdx = allocLink(tile0, CREATE_LINK_DYNAMIC_CLUSTER);
		dtClusterLink& testLink = getClusterLink(tile0, linkIdx);

		testLink.ref = cluster1Ref;
		testLink.next = cluster0.firstLink;
		cluster0.firstLink = linkIdx;

		link = &testLink;
	}

	// assign cost and side properties
	link->flags = link->flags | flags;
}

void dtNavMesh::unconnectClusterLinks(dtMeshTile* tile0, dtMeshTile* tile1)
{
	unsigned int tile1Num = decodeClusterIdTile(getTileRef(tile1));
	unsigned int nclusters = (unsigned int)tile0->header->clusterCount;
	for (unsigned int i = 0; i < nclusters; i++)
	{
		dtCluster& cluster = tile0->clusters[i];

		unsigned int pj = DT_NULL_LINK;
		unsigned int j = cluster.firstLink;
		while (j != DT_NULL_LINK)
		{
			dtClusterLink& link = getClusterLink(tile0, j);
			unsigned int linkTileNum = decodeClusterIdTile(link.ref);
			if (linkTileNum == tile1Num)
			{
				unsigned int nj = link.next;
				if (pj == DT_NULL_LINK)
				{
					cluster.firstLink = nj;
				}
				else
				{
					dtClusterLink& prevLink = getClusterLink(tile0, pj);
					prevLink.next = nj;
				}
				freeLink(tile0, j);
				j = nj;
			}
			else
			{
				pj = j;
				j = link.next;
			}
		}
	}
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

void dtNavMesh::closestPointOnPolyInTile(const dtMeshTile* tile, unsigned int ip,
										 const dtReal* pos, dtReal* closest) const
{
	const dtPoly* poly = &tile->polys[ip];
	// Off-mesh connections don't have detail polygons.
	if (poly->getType() == DT_POLYTYPE_OFFMESH_POINT)
	{
		//@UE BEGIN
		dtReal v0[3], v1[3];
		tile->verts[poly->verts[0]].toWorld(tile->header->bmin, v0);
		tile->verts[poly->verts[1]].toWorld(tile->header->bmin, v1);
		const dtReal d0 = dtVdist(pos, v0);
		const dtReal d1 = dtVdist(pos, v1);

		dtReal u;
		const dtReal sum = d0 + d1;
		if (sum != 0)
			u = d0 / sum;
		else
			u = 0;
		//@UE END

		dtVlerp(closest, v0, v1, u);
		return;
	}

	// Clamp point to be inside the polygon.
	dtReal verts[DT_VERTS_PER_POLYGON*3];
	dtReal edged[DT_VERTS_PER_POLYGON];
	dtReal edget[DT_VERTS_PER_POLYGON];
	const int nv = poly->vertCount;
	for (int i = 0; i < nv; ++i)
	{ //@UE
		tile->verts[poly->verts[i]].toWorld(tile->header->bmin, &verts[i*3]); //@UE
	} //@UE
	
	dtVcopy(closest, pos);
	if (nv == 0)
	{
		return;
	}
	
	if (!dtDistancePtPolyEdgesSqr(pos, verts, nv, edged, edget))
	{
		// Point is outside the polygon, dtClamp to nearest edge.
		dtReal dmin = edged[0];
		int imin = 0;
		for (int i = 1; i < nv; ++i)
		{
			if (edged[i] < dmin)
			{
				dmin = edged[i];
				imin = i;
			}
		}
		CA_ASSUME(imin < nv);
		const dtReal* va = &verts[imin*3];
		const dtReal* vb = &verts[((imin+1)%nv)*3];
		dtVlerp(closest, va, vb, edget[imin]);
	}
	
	// Find height at the location.
	if (poly->getType() == DT_POLYTYPE_GROUND)
	{
		const dtPolyDetail* pd = &tile->detailMeshes[ip];

		//@UE BEGIN
		for (int j = 0; j < pd->triCount; ++j)
		{
			const unsigned char* t = &tile->detailTris[(pd->triBase+j)*4];
			dtReal vBuf[3][3];
			const dtReal* v[3];
			for (int k = 0; k < 3; ++k)
			{
				CA_SUPPRESS(6385);
				if (t[k] < poly->vertCount)
					tile->verts[poly->verts[t[k]]].toWorld(tile->header->bmin, vBuf[k]);
				else
					tile->detailVerts[pd->vertBase+(t[k]-poly->vertCount)].toWorld(tile->header->bmin, vBuf[k]);
				v[k] = vBuf[k];
			}
			dtReal h;
			if (dtClosestHeightPointTriangle(closest, v[0], v[1], v[2], h))
			{
				closest[1] = h;
				break;
			}
		}
		//@UE END
	}
	else
	{
		dtReal h;
		if (dtClosestHeightPointTriangle(closest, &verts[0], &verts[6], &verts[3], h))
		{
			closest[1] = h;
		}
		else if (dtClosestHeightPointTriangle(closest, &verts[3], &verts[6], &verts[9], h))
		{
			closest[1] = h;
		}
	}
}

dtPolyRef dtNavMesh::findNearestPolyInTile(const dtMeshTile* tile,
										   const dtReal* center, const dtReal* extents,
										   dtReal* nearestPt, bool bExcludeUnwalkable) const
{
	dtAssert(nearestPt);

	dtReal bmin[3], bmax[3];
	dtVsub(bmin, center, extents);
	dtVadd(bmax, center, extents);
	
	// Get nearby polygons from proximity grid.
	dtPolyRef polys[128];
	int polyCount = queryPolygonsInTile(tile, bmin, bmax, polys, 128, bExcludeUnwalkable);
	
	// Find nearest polygon amongst the nearby polygons.
	dtPolyRef nearest = 0;
	dtReal nearestDistanceSqr = DT_REAL_MAX;
	dtVcopy(nearestPt, center);
	for (int i = 0; i < polyCount; ++i)
	{
		dtPolyRef ref = polys[i];
		dtReal closestPtPoly[3];
		closestPointOnPolyInTile(tile, decodePolyIdPoly(ref), center, closestPtPoly);
		dtReal d = dtVdistSqr(center, closestPtPoly);
		if (d < nearestDistanceSqr)
		{
			dtVcopy(nearestPt, closestPtPoly);
			nearestDistanceSqr = d;
			nearest = ref;
		}
	}

	// Verify if the point is actually within requested height, caller is performing 2D check anyway (radius)
	if (dtAbs(nearestPt[1]-center[1]) > extents[1])
	{
		nearest = 0;
	}
	
	return nearest;
}

dtPolyRef dtNavMesh::findCheapestNearPolyInTile(const dtMeshTile* tile, const dtReal* center,
												const dtReal* extents, dtReal* nearestPt) const
{
	dtAssert(nearestPt);

	dtReal bmin[3], bmax[3];
	dtVsub(bmin, center, extents);
	dtVadd(bmax, center, extents);

	// Get nearby polygons from proximity grid.
	dtPolyRef polys[128];
	constexpr bool bExcludeUnwalkable = true;
	int polyCount = queryPolygonsInTile(tile, bmin, bmax, polys, 128, bExcludeUnwalkable);

	// Find nearest polygon amongst the nearby polygons.
	dtPolyRef nearest = 0;
	dtReal nearestDistanceSqr = DT_REAL_MAX;
	unsigned char cheapestAreaCostOrder = 0xff;
	for (int i = 0; i < polyCount; ++i)
	{
		const dtPolyRef ref = polys[i];
		
		const int polyIdx = decodePolyIdPoly(ref);
		dtPoly* poly = &tile->polys[polyIdx];
		const unsigned char polyAreaCostOrder = m_areaCostOrder[poly->getArea()];
		if (polyAreaCostOrder < cheapestAreaCostOrder)
		{
			cheapestAreaCostOrder = polyAreaCostOrder;
			nearestDistanceSqr = DT_REAL_MAX;
			nearest = 0;
		}

		if (polyAreaCostOrder == cheapestAreaCostOrder)
		{
			dtReal closestPtPoly[3];
			closestPointOnPolyInTile(tile, polyIdx, center, closestPtPoly);
			const dtReal d = dtVdistSqr(center, closestPtPoly);
			if (d < nearestDistanceSqr)
			{
				dtVcopy(nearestPt, closestPtPoly);
				nearestDistanceSqr = d;
				nearest = ref;
			}
		}
	}

	// Verify if the point is actually within requested height, caller is performing 2D check anyway (radius).
	// Using nearest != 0 indicate if nearestPt has been set.
	if (nearest != 0 && dtAbs(nearestPt[1] - center[1]) > extents[1])
	{
		nearest = 0;
	}

	return nearest;
}

int dtNavMesh::queryPolygonsInTile(const dtMeshTile* tile, const dtReal* qmin, const dtReal* qmax,
									dtPolyRef* polys, const int maxPolys, bool bExcludeUnwalkable) const
{
	if (tile->bvTree)
	{
		const dtBVNode* node = &tile->bvTree[0];
		const dtBVNode* end = &tile->bvTree[tile->header->bvNodeCount];
		const dtReal* tbmin = tile->header->bmin;
		const dtReal* tbmax = tile->header->bmax;
		
		// Calculate quantized box
		unsigned short bmin[3], bmax[3];
		// dtClamp query box to world box.
		dtReal minx = dtClamp(qmin[0], tbmin[0], tbmax[0]) - tbmin[0];
		dtReal miny = dtClamp(qmin[1], tbmin[1], tbmax[1]) - tbmin[1];
		dtReal minz = dtClamp(qmin[2], tbmin[2], tbmax[2]) - tbmin[2];
		dtReal maxx = dtClamp(qmax[0], tbmin[0], tbmax[0]) - tbmin[0];
		dtReal maxy = dtClamp(qmax[1], tbmin[1], tbmax[1]) - tbmin[1];
		dtReal maxz = dtClamp(qmax[2], tbmin[2], tbmax[2]) - tbmin[2];

		// Quantize
		const dtReal bvQuantFactor = m_params.resolutionParams[tile->header->resolution].bvQuantFactor; 	//@UE
		UE_CLOGF(bvQuantFactor == 0.f, LogDetour, Warning, "dtNavMesh::queryPolygonsInTile bounding volume quantization factor is zero! The query might not return the right result");
		bmin[0] = (unsigned short)(bvQuantFactor * minx) & 0xfffe;
		bmin[1] = (unsigned short)(bvQuantFactor * miny) & 0xfffe;
		bmin[2] = (unsigned short)(bvQuantFactor * minz) & 0xfffe;
		bmax[0] = (unsigned short)(bvQuantFactor * maxx + 1) | 1;
		bmax[1] = (unsigned short)(bvQuantFactor * maxy + 1) | 1;
		bmax[2] = (unsigned short)(bvQuantFactor * maxz + 1) | 1;
		
		// Traverse tree
		dtPolyRef base = getPolyRefBase(tile);
		int n = 0;
		while (node < end)
		{
			const bool overlap = dtOverlapQuantBounds(bmin, bmax, node->bmin, node->bmax);
			const bool isLeafNode = node->i >= 0;
			
			if (isLeafNode && overlap)
			{
				if (n < maxPolys)
				{
					if (!bExcludeUnwalkable || tile->polys[node->i].flags)
					{
						polys[n++] = base | (dtPolyRef)node->i;
					}
				}
			}
			
			if (overlap || isLeafNode)
				node++;
			else
			{
				const int escapeIndex = -node->i;
				node += escapeIndex;
			}
		}
		
		return n;
	}
	else
	{
		dtReal bmin[3], bmax[3];
		int n = 0;
		dtPolyRef base = getPolyRefBase(tile);
		for (int i = 0; i < tile->header->polyCount; ++i)
		{
			dtPoly* p = &tile->polys[i];
			// Do not return off-mesh connection polygons.
			if (p->getType() != DT_POLYTYPE_GROUND)
				continue;
			if (p->flags == 0 && bExcludeUnwalkable)
				continue;

			// Calc polygon bounds.
			dtReal v[3]; //@UE
			tile->verts[p->verts[0]].toWorld(tile->header->bmin, v); //@UE
			dtVcopy(bmin, v);
			dtVcopy(bmax, v);
			for (int j = 1; j < p->vertCount; ++j)
			{
				tile->verts[p->verts[j]].toWorld(tile->header->bmin, v); //@UE
				dtVmin(bmin, v);
				dtVmax(bmax, v);
			}
			if (dtOverlapBounds(qmin,qmax, bmin,bmax))
			{
				if (n < maxPolys)
					polys[n++] = base | (dtPolyRef)i;
			}
		}
		return n;
	}
}

/// @par
///
/// The add operation will fail if the data is in the wrong format, the allocated tile
/// space is full, or there is a tile already at the specified reference.
///
/// The lastRef parameter is used to restore a tile with the same tile
/// reference it had previously used.  In this case the #dtPolyRef's for the
/// tile will be restored to the same values they were before the tile was 
/// removed.
///
/// @see dtCreateNavMeshData, #removeTile
//@UE BEGIN
dtStatus dtNavMesh::addTile(unsigned char* data, int dataSize, int flags,
							dtTileRef lastRef, dtTileRef* result)
{
	// Make sure the data is in right format.
	dtMeshHeader* header = (dtMeshHeader*)data;
	if (header->version != DT_NAVMESH_VERSION)
		return DT_FAILURE | DT_WRONG_VERSION;
		
	// Make sure the location is free.
	if (getTileAt(header->x, header->y, header->layer))
		return DT_FAILURE;
		
	// Allocate a tile.
	dtMeshTile* tile = 0;
	if (!lastRef)
	{
		if (m_nextFree)
		{
			tile = m_nextFree;
			m_nextFree = tile->next;
			tile->next = 0;
		}
	}
	else
	{
		// Try to relocate the tile to specific index with same salt.
		int tileIndex = (int)decodePolyIdTile((dtPolyRef)lastRef);
		if (tileIndex >= m_maxTiles)
			return DT_FAILURE | DT_OUT_OF_MEMORY;
		// Try to find the specific tile id from the free list.
		dtMeshTile* target = &m_tiles[tileIndex];
		dtMeshTile* prev = 0;
		tile = m_nextFree;
		while (tile && tile != target)
		{
			prev = tile;
			tile = tile->next;
		}
		// Could not find the correct location.
		if (tile != target)
			return DT_FAILURE | DT_OUT_OF_MEMORY;
		// Remove from freelist
		if (!prev)
			m_nextFree = tile->next;
		else
			prev->next = tile->next;

		// Restore salt.
		tile->salt = decodePolyIdSalt((dtPolyRef)lastRef);
	}

	// Make sure we could allocate a tile.
	if (!tile)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	
	// Insert tile into the position lut.
	int h = computeTileHash(header->x, header->y, m_tileLutMask);
	tile->next = m_posLookup[h];
	m_posLookup[h] = tile;
	
	// Patch header pointers.
	const int headerSize = dtAlign(sizeof(dtMeshHeader));
	const int vertsSize = dtAlign(sizeof(dtTileVert)*header->vertCount); //@UE
	const int polysSize = dtAlign(sizeof(dtPoly)*header->polyCount);
	const int linksSize = dtAlign(sizeof(dtLink)*(header->maxLinkCount));
	const int detailMeshesSize = dtAlign(sizeof(dtPolyDetail)*header->detailMeshCount);
	const int detailVertsSize = dtAlign(sizeof(dtTileVert)*header->detailVertCount); //@UE
	const int detailTrisSize = dtAlign(sizeof(unsigned char)*4*header->detailTriCount);
	const int bvtreeSize = dtAlign(sizeof(dtBVNode)*header->bvNodeCount);
	const int offMeshLinksSize = dtAlign(sizeof(dtOffMeshConnection)*header->offMeshConCount);

#if WITH_NAVMESH_SEGMENT_LINKS
	const int offMeshSegsSize = dtAlign(sizeof(dtOffMeshSegmentConnection)*header->offMeshSegConCount);
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	const int clustersSize = dtAlign(sizeof(dtCluster)*header->clusterCount);
	const int clusterPolysSize = dtAlign(sizeof(unsigned short)*header->offMeshBase);
#endif // WITH_NAVMESH_CLUSTER_LINKS

	const unsigned char* d = data + headerSize;
	tile->verts = (dtTileVert*)d; d += vertsSize; //@UE
	tile->polys = (dtPoly*)d; d += polysSize;
	tile->links = (dtLink*)d; d += linksSize;
	tile->detailMeshes = (dtPolyDetail*)d; d += detailMeshesSize;
	tile->detailVerts = (dtTileVert*)d; d += detailVertsSize; //@UE
	tile->detailTris = (unsigned char*)d; d += detailTrisSize;
	tile->bvTree = (dtBVNode*)d; d += bvtreeSize;
	tile->offMeshCons = (dtOffMeshConnection*)d; d += offMeshLinksSize;

	// If there are no items in the bvtree, reset the tree pointer.
	if (!bvtreeSize)
		tile->bvTree = 0;

#if WITH_NAVMESH_SEGMENT_LINKS
	tile->offMeshSeg = (dtOffMeshSegmentConnection*)d; d += offMeshSegsSize;
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	tile->clusters = (dtCluster*)d; d += clustersSize;
	tile->polyClusters = (unsigned short*)d; d += clusterPolysSize;

	const bool bHasClusters = header->clusterCount > 0;
	if (bHasClusters)
	{
		for (int i = 0; i < header->clusterCount; i++)
		{
			tile->clusters[i].numLinks = 0;
			tile->clusters[i].firstLink = DT_NULL_LINK;
		}
	}
	else
	{
		tile->polyClusters = 0;
	}
#else
	const bool bHasClusters = false;
#endif // WITH_NAVMESH_CLUSTER_LINKS

	// Build links freelist
	tile->linksFreeList = 0;
	tile->links[header->maxLinkCount-1].next = DT_NULL_LINK;
	for (int i = 0; i < header->maxLinkCount-1; ++i)
		tile->links[i].next = i+1;

	// Initialize dynamic links array
	tile->dynamicFreeList = DT_NULL_LINK;	//@UE
	tile->dynamicLinks.resize(0);			//@UE

#if WITH_NAVMESH_CLUSTER_LINKS
	tile->dynamicFreeListC = DT_NULL_LINK;
	tile->dynamicLinksC.resize(0);
#endif // WITH_NAVMESH_CLUSTER_LINKS

	// Init tile.
	tile->header = header;
	tile->data = data;
	tile->dataSize = dataSize;
	tile->flags = flags;

	connectIntLinks(tile);
	if (!m_bDeferOffMeshLinking)
	{
		baseOffMeshLinks(tile);
	}

	// Create connections with neighbour tiles.
	ReadTilesHelper TileArray;
	int nneis = 0;
	dtMeshTile** neis = nullptr;

	// Connect with layers in current tile.
	nneis = getTileCountAt(header->x, header->y);
	neis = TileArray.PrepareArray(nneis);
	if (neis == nullptr)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	getTilesAt(header->x, header->y, neis, nneis);
	for (int j = 0; j < nneis; ++j)
	{
		if (neis[j] != tile)
		{
			connectExtLinks(tile, neis[j], -1, bHasClusters);
			connectExtLinks(neis[j], tile, -1, bHasClusters);
			if (!m_bDeferOffMeshLinking)
			{
				connectExtOffMeshLinks(tile, neis[j], -1, bHasClusters);
			}
		}
		if (!m_bDeferOffMeshLinking)
		{
			connectExtOffMeshLinks(neis[j], tile, -1, bHasClusters);
		}
	}
	
	// Connect with neighbour tiles.
	for (int i = 0; i < 8; ++i)
	{
		const int neighbourTileCount = getNeighbourTilesCountAt(header->x, header->y, i);
		neis = TileArray.PrepareArray(neighbourTileCount );
		if (neis == nullptr)
		{
			return DT_FAILURE | DT_OUT_OF_MEMORY;
		}
		
		getNeighbourTilesAt(header->x, header->y, i, neis, neighbourTileCount);

		for (int j = 0; j < neighbourTileCount; ++j)
		{
			// Skip diagonal tiles, nothing to connect there 
			// (tiles are visited in a ring around the current tile, even tiles are primary directions)
			if ((i & 1) == 0)
			{
				connectExtLinks(tile, neis[j], i, bHasClusters);
				connectExtLinks(neis[j], tile, dtOppositeTile(i), bHasClusters);
			}

			if (!m_bDeferOffMeshLinking)
			{
				connectExtOffMeshLinks(tile, neis[j], i, bHasClusters);
				connectExtOffMeshLinks(neis[j], tile, dtOppositeTile(i), bHasClusters);
			}
		}
	}
	
	if (result)
		*result = getTileRef(tile);

	dtStatsPostAddTile(*tile);
	
	return DT_SUCCESS;
}

void dtNavMesh::connectDeferredOffMeshLinks()
{
	if (!m_bDeferOffMeshLinking)
		return;

	// Called from OnAreaCostChanged after applyAreaCostOrder has set m_areaCostOrder.
	// Must not be called before area costs are applied, or bSnapToCheapestArea links
	// will snap by distance instead of cost.

	m_bDeferOffMeshLinking = false;

	ReadTilesHelper TileArray;

	// Pass 1: Initialize all tiles' off-mesh connections first.
	// baseOffMeshLinks() sets poly->firstLink on each tile's off-mesh polys.
	// connectExtOffMeshLinks() guards on firstLink != DT_NULL_LINK, so all tiles
	// must be initialized before any cross-tile connections are attempted.
	for (int i = 0; i < m_maxTiles; ++i)
	{
		dtMeshTile* tile = &m_tiles[i];
		if (!tile->header) continue;
		baseOffMeshLinks(tile);
	}

	// Pass 2: Connect external links now that firstLink is valid on all tiles.
	for (int i = 0; i < m_maxTiles; ++i)
	{
		dtMeshTile* tile = &m_tiles[i];
		if (!tile->header) continue;

		const int tx = tile->header->x;
		const int ty = tile->header->y;

#if WITH_NAVMESH_CLUSTER_LINKS
		const bool bHasClusters = tile->header->clusterCount > 0;
#else
		const bool bHasClusters = false;
#endif // WITH_NAVMESH_CLUSTER_LINKS

		// Connect layers in same column — landing on this tile
		int nneis = getTileCountAt(tx, ty);
		dtMeshTile** neis = TileArray.PrepareArray(nneis);
		if (!neis) continue;
		getTilesAt(tx, ty, neis, nneis);
		for (int j = 0; j < nneis; ++j)
		{
			// neis[j]'s connections landing on tile (includes self case)
			connectExtOffMeshLinks(tile, neis[j], -1, bHasClusters);
		}

		// Connect neighbour tiles across edges — landing on this tile
		for (int side = 0; side < 8; ++side)
		{
			nneis = getNeighbourTilesCountAt(tx, ty, side);
			neis = TileArray.PrepareArray(nneis);
			if (!neis) continue;
			getNeighbourTilesAt(tx, ty, side, neis, nneis);
			for (int j = 0; j < nneis; ++j)
			{
				// neis[j]'s connections landing on tile
				connectExtOffMeshLinks(tile, neis[j], side, bHasClusters);
			}
		}
	}
}
//@UE END

#if WITH_NAVMESH_SEGMENT_LINKS
void dtNavMesh::processSegmentLinksForTile(dtTileRef tileRef, unsigned int maxSkippedNeigborTiles, dtTileRef* skippedNeigborTiles, unsigned int& numSkippedNeighborTiles)
{
	dtMeshTile* tile = getMutableTileByRef(tileRef);

	if (tile == nullptr || tile->header == nullptr)
	{
		return;
	}

	const int tileX = tile->header->x;
	const int tileY = tile->header->y;
	const unsigned short tileL = tile->header->layer;
	
	UE_LOGF(LogSegmentLink, Log, "%s (%i,%i,%i)", __FUNCTION__, tile->header->x, tile->header->y, tile->header->layer);
	
	// Find list of intersections with tile polys.
	dtOffMeshSegmentData* segList = initSegmentIntersection(*this, tile);

	if (segList == nullptr)
	{
		return;
	}

	numSkippedNeighborTiles = 0;

	// Create connections with neighbour tiles.
	ReadTilesHelper TileArray;

	// Connect with layers in current tile.
	UE_LOGF(LogSegmentLink, Log, "   %s Connect with layers in current tile.", __FUNCTION__);

	int nneis = getTileCountAt(tileX, tileY);
	dtMeshTile** neis = TileArray.PrepareArray(nneis);
	if (neis == nullptr)
	{
		UE_LOGF(LogSegmentLink, Error, "   %s Out of memory.", __FUNCTION__);
		dtFree(segList, DT_ALLOC_TEMP);
		return;
	}
	
	getTilesAt(tileX, tileY, neis, nneis);
	for (int j = 0; j < nneis; ++j)
	{
		dtMeshTile* neiTile = neis[j];
		if (neiTile != tile && neiTile->header)
		{
			const int neiX = neiTile->header->x;
			const int neiY = neiTile->header->y;
			const unsigned short neiL = neiTile->header->layer;
 			if (neiX < tileX && neiY < tileY && neiL < tileL)
			{
				if (numSkippedNeighborTiles < maxSkippedNeigborTiles)
				{
					skippedNeigborTiles[numSkippedNeighborTiles] = getTileRef(neis[j]);
				}
				++numSkippedNeighborTiles;
				continue;
			}

			appendSegmentIntersection(*this, segList, tile, neis[j]);
		}
	}

	// Connect with neighbour tiles.
	UE_LOGF(LogSegmentLink, Log, "   %s Connect with neighbour tiles.", __FUNCTION__);
	for (int i = 0; i < 8; ++i)
	{
		nneis = getNeighbourTilesCountAt(tileX, tileY, i);
		neis = TileArray.PrepareArray(nneis);
		if (neis == nullptr)
		{
			UE_LOGF(LogSegmentLink, Error, "   %s Out of memory.", __FUNCTION__);
			dtFree(segList, DT_ALLOC_TEMP);
			return;
		}
		
		getNeighbourTilesAt(tileX, tileY, i, neis, nneis);

		for (int j = 0; j < nneis; ++j)
		{
			dtMeshTile* neiTile = neis[j];
			if (neiTile->header)
			{
				const int neiX = neiTile->header->x;
				const int neiY = neiTile->header->y;
				const unsigned short neiL = neiTile->header->layer;
				if (neiX < tileX && neiY < tileY && neiL < tileL)
				{
					if (numSkippedNeighborTiles < maxSkippedNeigborTiles)
					{
						skippedNeigborTiles[numSkippedNeighborTiles] = getTileRef(neis[j]);
					}
					++numSkippedNeighborTiles;
					continue;
				}

				appendSegmentIntersection(*this, segList, tile, neis[j]);
			}
		}
	}

	createSegmentLinks(this, segList, tile);
	dtFree(segList, DT_ALLOC_TEMP);
}
#endif // WITH_NAVMESH_SEGMENT_LINKS

const dtMeshTile* dtNavMesh::getTileAt(const int x, const int y, const int layer) const
{
	// Find tile based on hash.
	int h = computeTileHash(x,y,m_tileLutMask);
	dtMeshTile* tile = m_posLookup[h];
	while (tile)
	{
		if (tile->header &&
			tile->header->x == x &&
			tile->header->y == y &&
			tile->header->layer == layer)
		{
			return tile;
		}
		tile = tile->next;
	}
	return 0;
}

int dtNavMesh::getNeighbourTilesAt(const int x, const int y, const int side, dtMeshTile** tiles, const int maxTiles) const
{
	int nx = x, ny = y;
	switch (side)
	{
		case 0: nx++; break;
		case 1: nx++; ny++; break;
		case 2: ny++; break;
		case 3: nx--; ny++; break;
		case 4: nx--; break;
		case 5: nx--; ny--; break;
		case 6: ny--; break;
		case 7: nx++; ny--; break;
	};

	return getTilesAt(nx, ny, (const dtMeshTile**)tiles, maxTiles);
}

// @UE BEGIN
int dtNavMesh::getNeighbourTilesCountAt(const int x, const int y, const int side) const
{
	int nx = x, ny = y;
	switch (side)
	{
		case 0: nx++; break;
		case 1: nx++; ny++; break;
		case 2: ny++; break;
		case 3: nx--; ny++; break;
		case 4: nx--; break;
		case 5: nx--; ny--; break;
		case 6: ny--; break;
		case 7: nx++; ny--; break;
	};

	return getTileCountAt(nx, ny);
}

int dtNavMesh::getTileCountAt(const int x, const int y) const
{
	int n = 0;

	// Find tile based on hash.
	int h = computeTileHash(x,y,m_tileLutMask);
	dtMeshTile* tile = m_posLookup[h];
	while (tile)
	{
		if (tile->header &&
			tile->header->x == x &&
			tile->header->y == y)
		{
			n++;
		}
		tile = tile->next;
	}

	return n;
}
// @UE END

int dtNavMesh::getTilesAt(const int x, const int y, dtMeshTile** tiles, const int maxTiles) const
{
	int n = 0;
	
	// Find tile based on hash.
	int h = computeTileHash(x,y,m_tileLutMask);
	dtMeshTile* tile = m_posLookup[h];
	while (tile)
	{
		if (tile->header &&
			tile->header->x == x &&
			tile->header->y == y)
		{
			if (n < maxTiles)
				tiles[n++] = tile;
		}
		tile = tile->next;
	}
	
	return n;
}

/// @par
///
/// This function will not fail if the tiles array is too small to hold the
/// entire result set.  It will simply fill the array to capacity.
int dtNavMesh::getTilesAt(const int x, const int y, dtMeshTile const** tiles, const int maxTiles) const
{
	int n = 0;
	
	// Find tile based on hash.
	int h = computeTileHash(x,y,m_tileLutMask);
	dtMeshTile* tile = m_posLookup[h];
	while (tile)
	{
		if (tile->header &&
			tile->header->x == x &&
			tile->header->y == y)
		{
			if (n < maxTiles)
				tiles[n++] = tile;
		}
		tile = tile->next;
	}
	
	return n;
}


dtTileRef dtNavMesh::getTileRefAt(const int x, const int y, const int layer) const
{
	// Find tile based on hash.
	int h = computeTileHash(x,y,m_tileLutMask);
	dtMeshTile* tile = m_posLookup[h];
	while (tile)
	{
		if (tile->header &&
			tile->header->x == x &&
			tile->header->y == y &&
			tile->header->layer == layer)
		{
			return getTileRef(tile);
		}
		tile = tile->next;
	}
	return 0;
}

const dtMeshTile* dtNavMesh::getTileByRef(dtTileRef ref) const
{
	if (!ref)
		return 0;
	unsigned int tileIndex = decodePolyIdTile((dtPolyRef)ref);
	unsigned int tileSalt = decodePolyIdSalt((dtPolyRef)ref);
	if ((int)tileIndex >= m_maxTiles)
		return 0;
	const dtMeshTile* tile = &m_tiles[tileIndex];
	if (tile->salt != tileSalt)
		return 0;
	return tile;
}

dtMeshTile* dtNavMesh::getMutableTileByRef(dtTileRef ref) const
{
	if (!ref)
		return 0;
	unsigned int tileIndex = decodePolyIdTile((dtPolyRef)ref);
	unsigned int tileSalt = decodePolyIdSalt((dtPolyRef)ref);
	if ((int)tileIndex >= m_maxTiles)
		return 0;
	dtMeshTile* tile = &m_tiles[tileIndex];
	if (tile->salt != tileSalt)
		return 0;
	return tile;
}

int dtNavMesh::getMaxTiles() const
{
	return m_maxTiles;
}

dtMeshTile* dtNavMesh::getTile(int i)
{
	return &m_tiles[i];
}

const dtMeshTile* dtNavMesh::getTile(int i) const
{
	return &m_tiles[i];
}

bool dtNavMesh::isTileLocInValidRange(const dtReal tx, const dtReal ty) const
{
	return (tx >= (dtReal)std::numeric_limits<int>::min()) &&
		(tx <= (dtReal)std::numeric_limits<int>::max()) &&
		(ty >= (dtReal)std::numeric_limits<int>::min()) &&
		(ty <= (dtReal)std::numeric_limits<int>::max());
}

void dtNavMesh::calcTileLoc(const dtReal* pos, dtReal* tx, dtReal* ty) const
{
	*tx = dtFloor((pos[0] - m_orig[0]) / m_tileWidth);
	*ty = dtFloor((pos[2] - m_orig[2]) / m_tileHeight);
}

void dtNavMesh::calcTileLoc(const dtReal* pos, int* tx, int* ty) const
{
	dtReal txReal = 0.;
	dtReal tyReal = 0.;

	calcTileLoc(pos, &txReal, &tyReal);
	
	*tx = (int)txReal;
	*ty = (int)tyReal;
}

bool dtNavMesh::isTileLocInValidRange(const dtReal* pos) const
{
	dtReal tx = 0.;
	dtReal ty = 0.;

	calcTileLoc(pos, &tx, &ty);
	return isTileLocInValidRange(tx, ty);
}

dtStatus dtNavMesh::getTileAndPolyByRef(const dtPolyRef ref, const dtMeshTile** tile, const dtPoly** poly) const
{
	if (!ref) return DT_FAILURE;
	unsigned int salt, it, ip;
	decodePolyId(ref, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return DT_FAILURE | DT_INVALID_PARAM;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return DT_FAILURE | DT_INVALID_PARAM;
	if (ip >= (unsigned int)m_tiles[it].header->polyCount) return DT_FAILURE | DT_INVALID_PARAM;
	*tile = &m_tiles[it];
	*poly = &m_tiles[it].polys[ip];
	return DT_SUCCESS;
}

/// @par
///
/// @warning Only use this function if it is known that the provided polygon
/// reference is valid. This function is faster than #getTileAndPolyByRef, but
/// it does not validate the reference.
void dtNavMesh::getTileAndPolyByRefUnsafe(const dtPolyRef ref, const dtMeshTile** tile, const dtPoly** poly) const
{
	unsigned int salt, it, ip;
	decodePolyId(ref, salt, it, ip);
	*tile = &m_tiles[it];
	*poly = &m_tiles[it].polys[ip];
}

bool dtNavMesh::isValidPolyRef(dtPolyRef ref) const
{
	if (!ref) return false;
	unsigned int salt, it, ip;
	decodePolyId(ref, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return false;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return false;
	if (ip >= (unsigned int)m_tiles[it].header->polyCount) return false;
	return true;
}

/// @par
///
/// This function returns the data for the tile so that, if desired,
/// it can be added back to the navigation mesh at a later point.
///
/// @see #addTile
dtStatus dtNavMesh::removeTile(dtTileRef ref, unsigned char** data, int* dataSize)
{
	if (!ref)
		return DT_FAILURE | DT_INVALID_PARAM;
	unsigned int tileIndex = decodePolyIdTile((dtPolyRef)ref);
	unsigned int tileSalt = decodePolyIdSalt((dtPolyRef)ref);
	if ((int)tileIndex >= m_maxTiles)
		return DT_FAILURE | DT_INVALID_PARAM;
	dtMeshTile* tile = &m_tiles[tileIndex];
	if (tile->salt != tileSalt)
		return DT_FAILURE | DT_INVALID_PARAM;

	dtStatsPreRemoveTile(*tile);
	
	// Remove tile from hash lookup.
	int h = computeTileHash(tile->header->x,tile->header->y,m_tileLutMask);
	dtMeshTile* prev = 0;
	dtMeshTile* cur = m_posLookup[h];
	while (cur)
	{
		if (cur == tile)
		{
			if (prev)
				prev->next = cur->next;
			else
				m_posLookup[h] = cur->next;
			break;
		}
		prev = cur;
		cur = cur->next;
	}
	
	// Remove connections to neighbour tiles.
	// Create connections with neighbour tiles.
	ReadTilesHelper TileArray;
	int nneis = getTileCountAt(tile->header->x, tile->header->y);
	dtMeshTile** neis = TileArray.PrepareArray(nneis);
	if (neis == nullptr)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}
	
	// Connect with layers in current tile.
	getTilesAt(tile->header->x, tile->header->y, neis, nneis);
	for (int j = 0; j < nneis; ++j)
	{
		CA_ASSUME(j < TileArray.NumAllocated);
		if (neis[j] == tile)
			continue;

		unconnectExtLinks(neis[j], tile);
	}
	
	// Connect with neighbour tiles.
	for (int i = 0; i < 8; ++i)
	{
		nneis = getNeighbourTilesCountAt(tile->header->x, tile->header->y, i);
		neis = TileArray.PrepareArray(nneis);
		if (neis == nullptr)
		{
			return DT_FAILURE | DT_OUT_OF_MEMORY;
		}

		getNeighbourTilesAt(tile->header->x, tile->header->y, i, neis, nneis);
		for (int j = 0; j < nneis; ++j)
		{
			CA_ASSUME(j < TileArray.NumAllocated);
			unconnectExtLinks(neis[j], tile);
		}
	}

	// Whether caller wants to own tile data
	bool callerOwnsData = (data && dataSize);

	// Reset tile.
	if ((tile->flags & DT_TILE_FREE_DATA) && !callerOwnsData)
	{
		// Owns data
		dtFree(tile->data, DT_ALLOC_PERM_TILE_DATA);
		tile->data = 0;
		tile->dataSize = 0;
		if (data) *data = 0;
		if (dataSize) *dataSize = 0;
	}
	else
	{
		if (data) *data = tile->data;
		if (dataSize) *dataSize = tile->dataSize;
	}

	tile->header = 0;
	tile->flags = 0;
	tile->linksFreeList = 0;
	tile->polys = 0;
	tile->verts = 0;
	tile->links = 0;
	tile->detailMeshes = 0;
	tile->detailVerts = 0;
	tile->detailTris = 0;
	tile->bvTree = 0;
	tile->offMeshCons = 0;

	// Update salt, salt should never be zero.
	tile->salt = (tile->salt+1) & ((1<<m_saltBits)-1);
	if (tile->salt == 0)
		tile->salt++;

	// Add to free list.
	tile->next = m_nextFree;
	m_nextFree = tile;

	return DT_SUCCESS;
}

dtTileRef dtNavMesh::getTileRef(const dtMeshTile* tile) const
{
	if (!tile) return 0;
	const unsigned int it = (unsigned int)(tile - m_tiles);
	return (dtTileRef)encodePolyId(tile->salt, it, 0);
}

/// @par
///
/// Example use case:
/// @code
///
/// const dtPolyRef base = navmesh->getPolyRefBase(tile);
/// for (int i = 0; i < tile->header->polyCount; ++i)
/// {
///     const dtPoly* p = &tile->polys[i];
///     const dtPolyRef ref = base | (dtPolyRef)i;
///     
///     // Use the reference to access the polygon data.
/// }
/// @endcode
dtPolyRef dtNavMesh::getPolyRefBase(const dtMeshTile* tile) const
{
	if (!tile) return 0;
	const unsigned int it = (unsigned int)(tile - m_tiles);
	return encodePolyId(tile->salt, it, 0);
}

dtClusterRef dtNavMesh::getClusterRefBase(const dtMeshTile* tile) const
{
	if (!tile) return 0;
	const unsigned int it = (unsigned int)(tile - m_tiles);
	return encodePolyId(tile->salt, it, 0);
}

struct dtTileState
{
	int magic;								// Magic number, used to identify the data.
	int version;							// Data version number.
	dtTileRef ref;							// Tile ref at the time of storing the data.
};

struct dtPolyState
{
	unsigned short flags;						// Flags (see dtPolyFlags).
	unsigned char area;							// Area ID of the polygon.
};

///  @see #storeTileState
int dtNavMesh::getTileStateSize(const dtMeshTile* tile) const
{
	if (!tile) return 0;
	const int headerSize = dtAlign(sizeof(dtTileState));
	const int polyStateSize = dtAlign(sizeof(dtPolyState) * tile->header->polyCount);
	return headerSize + polyStateSize;
}

/// @par
///
/// Tile state includes non-structural data such as polygon flags, area ids, etc.
/// @note The state data is only valid until the tile reference changes.
/// @see #getTileStateSize, #restoreTileState
dtStatus dtNavMesh::storeTileState(const dtMeshTile* tile, unsigned char* data, const int maxDataSize) const
{
	// Make sure there is enough space to store the state.
	const int sizeReq = getTileStateSize(tile);
	if (maxDataSize < sizeReq)
		return DT_FAILURE | DT_BUFFER_TOO_SMALL;
		
	dtTileState* tileState = (dtTileState*)data; data += dtAlign(sizeof(dtTileState));
	dtPolyState* polyStates = (dtPolyState*)data; data += dtAlign(sizeof(dtPolyState) * tile->header->polyCount);
	
	// Store tile state.
	tileState->magic = DT_NAVMESH_STATE_MAGIC;
	tileState->version = DT_NAVMESH_STATE_VERSION;
	tileState->ref = getTileRef(tile);
	
	// Store per poly state.
	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		const dtPoly* p = &tile->polys[i];
		dtPolyState* s = &polyStates[i];
		s->flags = p->flags;
		s->area = p->getArea();
	}
	
	return DT_SUCCESS;
}

/// @par
///
/// Tile state includes non-structural data such as polygon flags, area ids, etc.
/// @note This function does not impact the tile's #dtTileRef and #dtPolyRef's.
/// @see #storeTileState
dtStatus dtNavMesh::restoreTileState(dtMeshTile* tile, const unsigned char* data, const int maxDataSize)
{
	// Make sure there is enough space to store the state.
	const int sizeReq = getTileStateSize(tile);
	if (maxDataSize < sizeReq)
		return DT_FAILURE | DT_INVALID_PARAM;
	
	const dtTileState* tileState = (const dtTileState*)data; data += dtAlign(sizeof(dtTileState));
	const dtPolyState* polyStates = (const dtPolyState*)data; data += dtAlign(sizeof(dtPolyState) * tile->header->polyCount);
	
	// Check that the restore is possible.
	if (tileState->version != DT_NAVMESH_STATE_VERSION)
		return DT_FAILURE | DT_WRONG_VERSION;
	if (tileState->ref != getTileRef(tile))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	// Restore per poly state.
	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		dtPoly* p = &tile->polys[i];
		const dtPolyState* s = &polyStates[i];
		p->flags = s->flags;
		p->setArea(s->area);
	}
	
	return DT_SUCCESS;
}

/// @par
///
/// Off-mesh connections are stored in the navigation mesh as special 2-vertex 
/// polygons with a single edge. At least one of the vertices is expected to be 
/// inside a normal polygon. So an off-mesh connection is "entered" from a 
/// normal polygon at one of its endpoints. This is the polygon identified by 
/// the prevRef parameter.
/// For segment links, find the starting position closest to currentPos return the end position closest to that starting position.
/// If currentPos is not provided, use the midpoint on the starting edge as the starting position.
dtStatus dtNavMesh::getOffMeshConnectionPolyEndPoints(dtPolyRef prevRef, dtPolyRef polyRef, const dtReal* currentPos, dtReal* startPos, dtReal* endPos) const
{
	unsigned int salt, it, ip;

	if (!polyRef)
		return DT_FAILURE;
	
	// Get current polygon
	decodePolyId(polyRef, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return DT_FAILURE | DT_INVALID_PARAM;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return DT_FAILURE | DT_INVALID_PARAM;
	const dtMeshTile* tile = &m_tiles[it];
	if (ip >= (unsigned int)tile->header->polyCount) return DT_FAILURE | DT_INVALID_PARAM;
	const dtPoly* poly = &tile->polys[ip];

	if (poly->getType() == DT_POLYTYPE_GROUND)
		return DT_FAILURE;

	// Figure out which way to hand out the vertices.
	int idx0 = 0, idx1 = 1;

	// Find link that points to first vertex.
	unsigned int i = poly->firstLink;
	while (i != DT_NULL_LINK)
	{
		const dtLink& link = getLink(tile, i);
		if (link.edge == 0)
		{
			if (link.ref != prevRef)
			{
				idx0 = 1;
				idx1 = 0;
			}
			break;
		}

		i = link.next;
	}
	
	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	if (poly->getType() == DT_POLYTYPE_OFFMESH_SEGMENT)
	{
		idx0 = (idx0 == 0) ? 0 : 2;
		idx1 = (idx1 == 1) ? 1 : 3;
		const int idx2 = (idx0 == 0) ? 2 : 0;
		const int idx3 = (idx1 == 1) ? 3 : 1;

		dtReal start0[3], start1[3];
		tile->verts[poly->verts[idx0]].toWorld(tile->header->bmin, start0); //@UE
		tile->verts[poly->verts[idx1]].toWorld(tile->header->bmin, start1); //@UE
		dtReal t = 0;
		
		// Find the point on the segment link edge that is closest to the agent's current position.
		// However, this function can be called with a null currentPos. In that case, just use the midpoint of the edge.
		if (currentPos != nullptr)
		{
			dtDistancePtSegSqr2D(currentPos, start0, start1, t);
		}
		else
		{
			t = 0.5;
		}

		dtVlerp(startPos, start0, start1, t);
		
		// We want to use the shortest path to cross the segment link as this is generally the most sensible thing to do.
		// This change also matches the change to the string pulling algorithm used by traditional path following.
		dtReal end0[3], end1[3];
		tile->verts[poly->verts[idx2]].toWorld(tile->header->bmin, end0); //@UE
		tile->verts[poly->verts[idx3]].toWorld(tile->header->bmin, end1); //@UE
		dtDistancePtSegSqr2D(startPos, end0, end1, t);
		dtVlerp(endPos, end0, end1, t);
	}
	else
#endif // WITH_NAVMESH_SEGMENT_LINKS
	//@UE END
	{
		tile->verts[poly->verts[idx0]].toWorld(tile->header->bmin, startPos); //@UE
		tile->verts[poly->verts[idx1]].toWorld(tile->header->bmin, endPos); //@UE
	}

	return DT_SUCCESS;
}


const dtOffMeshConnection* dtNavMesh::getOffMeshConnectionByRef(dtPolyRef ref) const
{
	unsigned int salt, it, ip;
	
	if (!ref)
		return 0;
	
	// Get current polygon
	decodePolyId(ref, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return 0;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return 0;
	const dtMeshTile* tile = &m_tiles[it];
	if (ip >= (unsigned int)tile->header->polyCount) return 0;
	const dtPoly* poly = &tile->polys[ip];
	
	// Make sure that the current poly is indeed off-mesh link.
	if (poly->getType() != DT_POLYTYPE_OFFMESH_POINT)
		return 0;

	const unsigned int idx =  ip - tile->header->offMeshBase;
	dtAssert(idx < (unsigned int)tile->header->offMeshConCount);
	return &tile->offMeshCons[idx];
}

//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
const dtOffMeshSegmentConnection* dtNavMesh::getOffMeshSegmentConnectionByRef(dtPolyRef ref) const
{
	unsigned int salt, it, ip;

	if (!ref)
		return 0;

	// Get current polygon
	decodePolyId(ref, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return 0;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return 0;
	const dtMeshTile* tile = &m_tiles[it];
	if (ip >= (unsigned int)tile->header->polyCount) return 0;
	const dtPoly* poly = &tile->polys[ip];

	// Make sure that the current poly is indeed off-mesh link.
	if (poly->getType() != DT_POLYTYPE_OFFMESH_SEGMENT)
		return 0;

	const unsigned int idx = (ip - tile->header->offMeshSegPolyBase) / DT_MAX_OFFMESH_SEGMENT_PARTS;
	dtAssert(idx < (unsigned int)tile->header->offMeshSegConCount);
	return &tile->offMeshSeg[idx];
}

void dtNavMesh::updateOffMeshSegmentConnectionByUserId(unsigned int userId, unsigned char newArea, unsigned short newFlags)
{
	for (int it = 0; it < m_maxTiles; it++)
	{
		dtMeshTile* tile = &m_tiles[it];
		if (tile == 0 || tile->header == 0)
			continue;

		for (int ic = 0; ic < tile->header->offMeshSegConCount; ic++)
		{
			dtOffMeshSegmentConnection& con = tile->offMeshSeg[ic];
			if (con.userId == userId)
			{
				for (int ip = 0; ip < con.npolys; ip++)
				{
					dtPoly* poly = &tile->polys[tile->header->offMeshSegPolyBase + con.firstPoly + ip];
					poly->setArea(newArea);
					poly->flags = newFlags;
				}
			}
		}
	}
}
#endif // WITH_NAVMESH_SEGMENT_LINKS
//@UE END

void dtNavMesh::updateOffMeshConnectionByUserId(unsigned long long int userId, unsigned char newArea, unsigned short newFlags)
{
	for (int it = 0; it < m_maxTiles; it++)
	{
		dtMeshTile* tile = &m_tiles[it];
		if (tile == 0 || tile->header == 0)
			continue;

		for (int ic = 0; ic < tile->header->offMeshConCount; ic++)
		{
			dtOffMeshConnection& con = tile->offMeshCons[ic];
			if (con.userId == userId)
			{
				dtPoly* poly = &tile->polys[con.poly];
				poly->setArea(newArea);
				poly->flags = newFlags;
			}
		}
	}
}

dtStatus dtNavMesh::setPolyFlags(dtPolyRef ref, unsigned short flags)
{
	if (!ref) return DT_FAILURE;
	unsigned int salt, it, ip;
	decodePolyId(ref, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return DT_FAILURE | DT_INVALID_PARAM;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return DT_FAILURE | DT_INVALID_PARAM;
	dtMeshTile* tile = &m_tiles[it];
	if (ip >= (unsigned int)tile->header->polyCount) return DT_FAILURE | DT_INVALID_PARAM;
	dtPoly* poly = &tile->polys[ip];
	
	// Change flags.
	poly->flags = flags;
	
	return DT_SUCCESS;
}

dtStatus dtNavMesh::getPolyFlags(dtPolyRef ref, unsigned short* resultFlags) const
{
	if (!ref) return DT_FAILURE;
	unsigned int salt, it, ip;
	decodePolyId(ref, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return DT_FAILURE | DT_INVALID_PARAM;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return DT_FAILURE | DT_INVALID_PARAM;
	const dtMeshTile* tile = &m_tiles[it];
	if (ip >= (unsigned int)tile->header->polyCount) return DT_FAILURE | DT_INVALID_PARAM;
	const dtPoly* poly = &tile->polys[ip];

	*resultFlags = poly->flags;
	
	return DT_SUCCESS;
}

dtStatus dtNavMesh::setPolyArea(dtPolyRef ref, unsigned char area)
{
	if (!ref) return DT_FAILURE;
	unsigned int salt, it, ip;
	decodePolyId(ref, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return DT_FAILURE | DT_INVALID_PARAM;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return DT_FAILURE | DT_INVALID_PARAM;
	dtMeshTile* tile = &m_tiles[it];
	if (ip >= (unsigned int)tile->header->polyCount) return DT_FAILURE | DT_INVALID_PARAM;
	dtPoly* poly = &tile->polys[ip];
	
	poly->setArea(area);
	
	return DT_SUCCESS;
}

dtStatus dtNavMesh::getPolyArea(dtPolyRef ref, unsigned char* resultArea) const
{
	if (!ref) return DT_FAILURE;
	unsigned int salt, it, ip;
	decodePolyId(ref, salt, it, ip);
	if (it >= (unsigned int)m_maxTiles) return DT_FAILURE | DT_INVALID_PARAM;
	if (m_tiles[it].salt != salt || m_tiles[it].header == 0) return DT_FAILURE | DT_INVALID_PARAM;
	const dtMeshTile* tile = &m_tiles[it];
	if (ip >= (unsigned int)tile->header->polyCount) return DT_FAILURE | DT_INVALID_PARAM;
	const dtPoly* poly = &tile->polys[ip];
	
	*resultArea = poly->getArea();
	
	return DT_SUCCESS;
}

//@UE BEGIN 
void dtNavMesh::applyWorldOffset(const dtReal* offset)
{
	//Shift navmesh origin
	dtVadd(m_params.orig, m_params.orig, offset);
	dtVadd(m_orig, m_orig, offset);
						
	// Iterate over all tiles and apply provided offset
	for (int i = 0; i < m_maxTiles; ++i)
	{
		dtMeshTile& tile = m_tiles[i];
		if (tile.header != NULL)
		{
			// Shift tile bounds
			dtVadd(tile.header->bmin, tile.header->bmin, offset);
			dtVadd(tile.header->bmax, tile.header->bmax, offset);

			// Tile vertices are stored as local offsets from bmin; shifting bmin above is sufficient.

			//Shift off-mesh connections
			for (int j = 0; j < tile.header->offMeshConCount; ++j)
			{
				dtVadd(&(tile.offMeshCons[j].pos[0]), &(tile.offMeshCons[j].pos[0]), offset);
				dtVadd(&(tile.offMeshCons[j].pos[3]), &(tile.offMeshCons[j].pos[3]), offset);
			}

#if WITH_NAVMESH_SEGMENT_LINKS
			// Shift off-mesh segment connections
			for (int j = 0; j < tile.header->offMeshSegConCount; ++j)
			{
				dtVadd(&(tile.offMeshSeg[j].startA[0]), &(tile.offMeshSeg[j].startA[0]), offset);
				dtVadd(&(tile.offMeshSeg[j].endA[0]),	&(tile.offMeshSeg[j].endA[0]), offset);
				dtVadd(&(tile.offMeshSeg[j].startB[0]), &(tile.offMeshSeg[j].startB[0]), offset);
				dtVadd(&(tile.offMeshSeg[j].endB[0]),	&(tile.offMeshSeg[j].endB[0]), offset);
			}
#endif // WITH_NAVMESH_SEGMENT_LINKS
			
#if WITH_NAVMESH_CLUSTER_LINKS
			// Shift clusters
			for (int j = 0; j < tile.header->clusterCount; ++j)
			{
				dtVadd(&(tile.clusters[j].center[0]), &(tile.clusters[j].center[0]), offset);
			}
#endif // WITH_NAVMESH_CLUSTER_LINKS
		}
	}
}

void dtNavMesh::applyAreaCostOrder(unsigned char* costOrder)
{
	memcpy(m_areaCostOrder, costOrder, sizeof(m_areaCostOrder));
}
//@UE END