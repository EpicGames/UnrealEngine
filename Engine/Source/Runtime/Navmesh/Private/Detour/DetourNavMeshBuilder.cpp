// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "Detour/DetourNavMeshBuilder.h"
#include "Detour/DetourNavMesh.h"
#include "Detour/DetourCommon.h"
#include <limits>

static unsigned short MESH_NULL_IDX = 0xffff;


struct BVItem
{
	unsigned short bmin[3];
	unsigned short bmax[3];
	int i;
};

static int compareItemX(const void* va, const void* vb)
{
	const BVItem* a = (const BVItem*)va;
	const BVItem* b = (const BVItem*)vb;
	if (a->bmin[0] < b->bmin[0])
		return -1;
	if (a->bmin[0] > b->bmin[0])
		return 1;
	return 0;
}

static int compareItemY(const void* va, const void* vb)
{
	const BVItem* a = (const BVItem*)va;
	const BVItem* b = (const BVItem*)vb;
	if (a->bmin[1] < b->bmin[1])
		return -1;
	if (a->bmin[1] > b->bmin[1])
		return 1;
	return 0;
}

static int compareItemZ(const void* va, const void* vb)
{
	const BVItem* a = (const BVItem*)va;
	const BVItem* b = (const BVItem*)vb;
	if (a->bmin[2] < b->bmin[2])
		return -1;
	if (a->bmin[2] > b->bmin[2])
		return 1;
	return 0;
}

static void calcExtends(BVItem* items, const int /*nitems*/, const int imin, const int imax,
						unsigned short* bmin, unsigned short* bmax)
{
	bmin[0] = items[imin].bmin[0];
	bmin[1] = items[imin].bmin[1];
	bmin[2] = items[imin].bmin[2];
	
	bmax[0] = items[imin].bmax[0];
	bmax[1] = items[imin].bmax[1];
	bmax[2] = items[imin].bmax[2];
	
	for (int i = imin+1; i < imax; ++i)
	{
		const BVItem& it = items[i];
		if (it.bmin[0] < bmin[0]) bmin[0] = it.bmin[0];
		if (it.bmin[1] < bmin[1]) bmin[1] = it.bmin[1];
		if (it.bmin[2] < bmin[2]) bmin[2] = it.bmin[2];
		
		if (it.bmax[0] > bmax[0]) bmax[0] = it.bmax[0];
		if (it.bmax[1] > bmax[1]) bmax[1] = it.bmax[1];
		if (it.bmax[2] > bmax[2]) bmax[2] = it.bmax[2];
	}
}

inline int longestAxis(unsigned short x, unsigned short y, unsigned short z)
{
	int	axis = 0;
	unsigned short maxVal = x;
	if (y > maxVal)
	{
		axis = 1;
		maxVal = y;
	}
	if (z > maxVal)
	{
		axis = 2;
		maxVal = z;
	}
	return axis;
}

static void subdivide(BVItem* items, int nitems, int imin, int imax, int& curNode, dtBVNode* nodes)
{
	int inum = imax - imin;
	int icur = curNode;
	
	dtBVNode& node = nodes[curNode++];
	
	if (inum == 1)
	{
		// Leaf
		node.bmin[0] = items[imin].bmin[0];
		node.bmin[1] = items[imin].bmin[1];
		node.bmin[2] = items[imin].bmin[2];
		
		node.bmax[0] = items[imin].bmax[0];
		node.bmax[1] = items[imin].bmax[1];
		node.bmax[2] = items[imin].bmax[2];
		
		node.i = items[imin].i;
	}
	else
	{
		// Split
		calcExtends(items, nitems, imin, imax, node.bmin, node.bmax);
		
		int	axis = longestAxis(node.bmax[0] - node.bmin[0],
							   node.bmax[1] - node.bmin[1],
							   node.bmax[2] - node.bmin[2]);
		
		if (axis == 0)
		{
			// Sort along x-axis
			qsort(items+imin, inum, sizeof(BVItem), compareItemX);
		}
		else if (axis == 1)
		{
			// Sort along y-axis
			qsort(items+imin, inum, sizeof(BVItem), compareItemY);
		}
		else
		{
			// Sort along z-axis
			qsort(items+imin, inum, sizeof(BVItem), compareItemZ);
		}
		
		int isplit = imin+inum/2;
		
		// Left
		subdivide(items, nitems, imin, isplit, curNode, nodes);
		// Right
		subdivide(items, nitems, isplit, imax, curNode, nodes);
		
		int iescape = curNode - icur;
		// Negative index means escape.
		node.i = -iescape;
	}
}

static int createBVTree(const unsigned short* verts, const int /*nverts*/,
						const unsigned short* polys, const int npolys, const int nvp,
						const dtPolyDetail* DMeshes, const dtTileVert* DVerts, const unsigned char* DTris, //@UE
						const dtReal cs, const dtReal ch,
						const int /*nnodes*/, dtBVNode* nodes)
{
	//@UE BEGIN
	const uint64 itemSize = sizeof(BVItem)*npolys;
	if (itemSize > INT_MAX)
	{
		return 0;
	}

	// Build tree
	BVItem* items = (BVItem*)dtAlloc(itemSize, DT_ALLOC_TEMP);

	if (items == nullptr)
	{
		return 0;
	}

	if (cs <= 0 || ch <= 0)
	{
		dtFree(items, DT_ALLOC_TEMP);
		return 0;
	}
	//@UE END

	for (int i = 0; i < npolys; i++)
	{
		BVItem& it = items[i];
		it.i = i;
		// Calc polygon bounds.
		const unsigned short* p = &polys[i*nvp*2];
		it.bmin[0] = it.bmax[0] = verts[p[0]*3+0];
		it.bmin[1] = it.bmax[1] = verts[p[0]*3+1];
		it.bmin[2] = it.bmax[2] = verts[p[0]*3+2];
		
		int vertCount = nvp;
		for (int j = 1; j < nvp; ++j)
		{
			if (p[j] == MESH_NULL_IDX)
			{
				vertCount = j;
				break;
			}

			unsigned short x = verts[p[j]*3+0];
			unsigned short y = verts[p[j]*3+1];
			unsigned short z = verts[p[j]*3+2];
			
			if (x < it.bmin[0]) it.bmin[0] = x;
			if (y < it.bmin[1]) it.bmin[1] = y;
			if (z < it.bmin[2]) it.bmin[2] = z;
			
			if (x > it.bmax[0]) it.bmax[0] = x;
			if (y > it.bmax[1]) it.bmax[1] = y;
			if (z > it.bmax[2]) it.bmax[2] = z;
		}

		// include y from detail mesh
		const dtPolyDetail* pd = &DMeshes[i];
		for (int k = 0; k < pd->triCount; ++k)
		{
			const unsigned char* t = &DTris[(pd->triBase + k) * 4];
			for (int m = 0; m < 3; ++m)
			{
				if (t[m] >= vertCount)
				{
					const dtReal qY = DVerts[pd->vertBase + (t[m] - vertCount)].y / ch; //@UE
					const unsigned short qYmin = (unsigned short)dtFloor(qY);
					const unsigned short qYmax = (unsigned short)dtCeil(qY);

					if (qYmin < it.bmin[1]) it.bmin[1] = qYmin;
					if (qYmax > it.bmax[1]) it.bmax[1] = qYmax;
				}
			}
		}

		// Remap y
		it.bmin[1] = (unsigned short)dtFloor((dtReal)it.bmin[1]*ch/cs);
		it.bmax[1] = (unsigned short)dtCeil((dtReal)it.bmax[1]*ch/cs);
	}
	
	int curNode = 0;
	subdivide(items, npolys, 0, npolys, curNode, nodes);
	
	dtFree(items, DT_ALLOC_TEMP);
	
	return curNode;
}

static unsigned char classifyOffMeshPoint(const dtReal* pt, const dtReal* bmin, const dtReal* bmax)
{
	static const unsigned char XP = 1<<0;
	static const unsigned char ZP = 1<<1;
	static const unsigned char XM = 1<<2;
	static const unsigned char ZM = 1<<3;	

	unsigned char outcode = 0; 
	outcode |= (pt[0] >= bmax[0]) ? XP : 0;
	outcode |= (pt[2] >= bmax[2]) ? ZP : 0;
	outcode |= (pt[0] < bmin[0])  ? XM : 0;
	outcode |= (pt[2] < bmin[2])  ? ZM : 0;

	switch (outcode)
	{
	case XP: return 0;
	case XP|ZP: return 1;
	case ZP: return 2;
	case XM|ZP: return 3;
	case XM: return 4;
	case XM|ZM: return 5;
	case ZM: return 6;
	case XP|ZM: return 7;
	};

	return 0xff;	
}

// TODO: Better error handling.

/// @par
/// 
/// The output data array is allocated using the detour allocator (dtAlloc()).  The method
/// used to free the memory will be determined by how the tile is added to the navigation
/// mesh.
///
/// @see dtNavMesh, dtNavMesh::addTile()
bool dtCreateNavMeshData(dtNavMeshCreateParams* params, unsigned char** outData, int* outDataSize)
{
	if (params->nvp > DT_VERTS_PER_POLYGON)
		return false;
	if (params->vertCount >= 0xffff)
		return false;
	if (!params->vertCount || !params->verts)
		return false;
	if (!params->polyCount || !params->polys)
		return false;

	const int nvp = params->nvp;
	
	// Classify off-mesh connection points. We store only the connections
	// whose start point is inside the tile.
	unsigned char* offMeshConClass = 0;
	int storedOffMeshConCount = 0;
	int storedOffMeshSegCount = 0;

	if (params->offMeshConCount > 0)
	{
		offMeshConClass = (unsigned char*)dtAlloc(sizeof(unsigned char)*params->offMeshConCount*2, DT_ALLOC_TEMP);
		if (!offMeshConClass)
			return false;

		memset(offMeshConClass, 0, sizeof(unsigned char)*params->offMeshConCount*2);

		// Find tight height bounds, used for culling out off-mesh start locations.
		dtReal hmin = DT_REAL_MAX;
		dtReal hmax = -DT_REAL_MAX;
		
		for (int i = 0; i < params->vertCount; ++i)
		{
			const unsigned short* iv = &params->verts[i*3];
			const dtReal h = params->bmin[1] + iv[1] * params->ch;
			hmin = dtMin(hmin,h);
			hmax = dtMax(hmax,h);
		}

		if (params->detailVerts && params->detailVertsCount)
		{
			for (int i = 0; i < params->detailVertsCount; ++i)
			{
				const dtReal h = params->detailVerts[i*3+1];
				hmin = dtMin(hmin,h);
				hmax = dtMax(hmax,h);
			}
		}

		hmin -= params->walkableClimb;
		hmax += params->walkableClimb;

		dtReal bmin[3], bmax[3];
		dtVcopy(bmin, params->bmin);
		dtVcopy(bmax, params->bmax);
		bmin[1] = hmin;
		bmax[1] = hmax;

#if WITH_NAVMESH_SEGMENT_LINKS
		// This is to fix an issues observed with dtIntersectSegmentPoly2D. The poly verts need to be in clockwise order.
		dtReal bverts[3*4];
		bverts[0] = bmin[0]; bverts[2] = bmax[2];
		bverts[3] = bmax[0]; bverts[5] = bmax[2];
		bverts[6] = bmax[0]; bverts[8] = bmin[2];
		bverts[9] = bmin[0]; bverts[11] = bmin[2];
#endif // WITH_NAVMESH_SEGMENT_LINKS

		for (int i = 0; i < params->offMeshConCount; ++i)
		{
			
			const dtOffMeshLinkCreateParams& offMeshCon = params->offMeshCons[i];
			if (offMeshCon.type & DT_OFFMESH_CON_POINT)
			{
				offMeshConClass[i*2+0] = classifyOffMeshPoint(offMeshCon.vertsA0, bmin, bmax);
				offMeshConClass[i*2+1] = classifyOffMeshPoint(offMeshCon.vertsB0, bmin, bmax);

				// Zero out off-mesh start positions which are not even potentially touching the mesh.
				if (offMeshConClass[i*2+0] == 0xff)
				{
					if ((offMeshCon.vertsA0[1] - offMeshCon.snapHeight) > bmax[1] ||
						(offMeshCon.vertsA0[1] + offMeshCon.snapHeight) < bmin[1])
					{
						offMeshConClass[i * 2 + 0] = 0;
					}
				}

				if (offMeshConClass[i*2+0] == 0xff)
					storedOffMeshConCount++;
			}
#if WITH_NAVMESH_SEGMENT_LINKS
			else if (offMeshCon.type & DT_OFFMESH_CON_SEGMENT)
			{
				int smin, smax;
				dtReal tmin, tmax;
				if ((offMeshCon.vertsA0[1] >= bmin[1] && offMeshCon.vertsA0[1] <= bmax[1] && classifyOffMeshPoint(offMeshCon.vertsA0, bmin, bmax) == 0xff) ||
					(offMeshCon.vertsA1[1] >= bmin[1] && offMeshCon.vertsA1[1] <= bmax[1] && classifyOffMeshPoint(offMeshCon.vertsA1, bmin, bmax) == 0xff) ||
					(offMeshCon.vertsB0[1] >= bmin[1] && offMeshCon.vertsB0[1] <= bmax[1] && classifyOffMeshPoint(offMeshCon.vertsB0, bmin, bmax) == 0xff) ||
					(offMeshCon.vertsB1[1] >= bmin[1] && offMeshCon.vertsB1[1] <= bmax[1] && classifyOffMeshPoint(offMeshCon.vertsB1, bmin, bmax) == 0xff) ||
					dtIntersectSegmentPoly2D(offMeshCon.vertsA0, offMeshCon.vertsA1, bverts, 4, tmin, tmax, smin, smax) ||
					dtIntersectSegmentPoly2D(offMeshCon.vertsB0, offMeshCon.vertsB1, bverts, 4, tmin, tmax, smin, smax))
				{
					offMeshConClass[i*2] = 0xff;
					storedOffMeshSegCount++;
				}
			}
#endif // WITH_NAVMESH_SEGMENT_LINKS
		}
	}
	
	// Off-mesh connections are stored as polygons, adjust values.
	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	const int firstSegVert = params->vertCount + storedOffMeshConCount*2;
	const int firstSegPoly = params->polyCount + storedOffMeshConCount;
	const int totPolyCount = firstSegPoly + storedOffMeshSegCount*DT_MAX_OFFMESH_SEGMENT_PARTS;
	const int totVertCount = firstSegVert + storedOffMeshSegCount*DT_MAX_OFFMESH_SEGMENT_PARTS*4;
#else
	const int totPolyCount = params->polyCount + storedOffMeshConCount;
	const int totVertCount = params->vertCount + storedOffMeshConCount * 2;
#endif // WITH_NAVMESH_SEGMENT_LINKS
	//@UE END
	
	// Find portal edges which are at tile borders.
	int edgeCount = 0;
	int portalCount = 0;
	for (int i = 0; i < params->polyCount; ++i)
	{
		const unsigned short* p = &params->polys[i*2*nvp];
		for (int j = 0; j < nvp; ++j)
		{
			if (p[j] == MESH_NULL_IDX) break;
			edgeCount++;
			
			if (p[nvp+j] & 0x8000)
			{
				unsigned short dir = p[nvp+j] & 0xf;
				if (dir != 0xf)
					portalCount++;
			}
		}
	}

//@UE BEGIN: Link pool sizing
	// Compute the size of the base link pool.
	// Offmesh links and extra preallocated links are added in a dynamic array.
	int maxLinkCount;
	if (params->bMinimizeLinkPoolSize)
	{
		// Minimized link pool size.
		// Navigation meshes are primarily triangles/quads (3-4 edges per polygon).
		// Each edge may connect to one or more neighbors, requiring links for each connection.
		// The x4 multiplier provides enough capacity for typical connectivity patterns and
		// tiles exceeding this capacity fall back to secondary storage.
		maxLinkCount = dtMax(params->polyCount * 4, 16);
	}
	else
	{
		maxLinkCount = edgeCount + portalCount*2;
	}
//@UE END
	
	// Find unique detail vertices.
	int uniqueDetailVertCount = 0;
	int detailTriCount = 0;
	if (params->detailMeshes)
	{
		// Has detail mesh, count unique detail vertex count and use input detail tri count.
		detailTriCount = params->detailTriCount;
		for (int i = 0; i < params->polyCount; ++i)
		{
			const unsigned short* p = &params->polys[i*nvp*2];
			int ndv = params->detailMeshes[i*4+1];
			int nv = 0;
			for (int j = 0; j < nvp; ++j)
			{
				if (p[j] == MESH_NULL_IDX) break;
				nv++;
			}
			ndv -= nv;
			uniqueDetailVertCount += ndv;
		}
	}
	else
	{
		// No input detail mesh, build detail mesh from nav polys.
		uniqueDetailVertCount = 0; // No extra detail verts.
		detailTriCount = 0;
		for (int i = 0; i < params->polyCount; ++i)
		{
			const unsigned short* p = &params->polys[i*nvp*2];
			int nv = 0;
			for (int j = 0; j < nvp; ++j)
			{
				if (p[j] == MESH_NULL_IDX) break;
				nv++;
			}
			detailTriCount += nv-2;
		}
	}
 
	// Calculate data size
	const int headerSize = dtAlign(sizeof(dtMeshHeader));
	const int vertsSize = dtAlign(sizeof(dtTileVert)*totVertCount); //@UE
	const int polysSize = dtAlign(sizeof(dtPoly)*totPolyCount);
	const int linksSize = dtAlign(sizeof(dtLink)*maxLinkCount);
	const int detailMeshesSize = dtAlign(sizeof(dtPolyDetail)*params->polyCount);
	const int detailVertsSize = dtAlign(sizeof(dtTileVert)*uniqueDetailVertCount); //@UE
	const int detailTrisSize = dtAlign(sizeof(unsigned char)*4*detailTriCount);
	const int bvTreeSize = params->buildBvTree ? dtAlign(sizeof(dtBVNode)*params->polyCount*2) : 0;
	const int offMeshConsSize = dtAlign(sizeof(dtOffMeshConnection)*storedOffMeshConCount);

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	const int offMeshSegSize = dtAlign(sizeof(dtOffMeshSegmentConnection)*storedOffMeshSegCount);
#else
	const int offMeshSegSize = 0;
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	const int clustersSize = dtAlign(sizeof(dtCluster)*params->clusterCount);
	const int polyClustersSize = dtAlign(sizeof(unsigned short)*params->polyCount);
#else
	const int clustersSize = 0;
	const int polyClustersSize = 0;
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	const int dataSize = headerSize + vertsSize + polysSize + linksSize +
						 detailMeshesSize + detailVertsSize + detailTrisSize +
						 bvTreeSize + offMeshConsSize + offMeshSegSize +
						 clustersSize + polyClustersSize;
						 
	unsigned char* data = (unsigned char*)dtAlloc(sizeof(unsigned char)*dataSize, DT_ALLOC_PERM_TILE_DATA);
	if (!data)
	{
		dtFree(offMeshConClass, DT_ALLOC_TEMP);
		return false;
	}
	memset(data, 0, dataSize);
	
	unsigned char* d = data;
	dtMeshHeader* header = (dtMeshHeader*)d; d += headerSize;
	dtTileVert* navVerts = (dtTileVert*)d; d += vertsSize; //@UE
	dtPoly* navPolys = (dtPoly*)d; d += polysSize;
	d += linksSize;
	dtPolyDetail* navDMeshes = (dtPolyDetail*)d; d += detailMeshesSize;
	dtTileVert* navDVerts = (dtTileVert*)d; d += detailVertsSize; //@UE
	unsigned char* navDTris = (unsigned char*)d; d += detailTrisSize;
	dtBVNode* navBvtree = (dtBVNode*)d; d += bvTreeSize;
	dtOffMeshConnection* offMeshCons = (dtOffMeshConnection*)d; d += offMeshConsSize;

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	dtOffMeshSegmentConnection* offMeshSegs = (dtOffMeshSegmentConnection*)d; d += offMeshSegSize;
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	dtCluster* clusters = (dtCluster*)d; d += clustersSize;
	unsigned short* polyClusters = (unsigned short*)d; d += polyClustersSize;
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	//@UE BEGIN Memory optimization
	auto setHeaderShort = [](const int value, unsigned short& headerVal, const TCHAR* text, bool& bAllValuesSet)
	{
		dtAssert(value >= 0);

		//check value will fit in to an unsigned short
		if (value > std::numeric_limits<unsigned short>::max())
		{
			UE_LOGF(LogDetour, Error, "dtCreateNavMeshData: %ls is too large to fit in an unsigned short!", text);

			bAllValuesSet = false;
			headerVal = 0;
		}
		else
		{
			headerVal = (unsigned short)value;
		}
	};

	bool bAllValuesSet = true;

	header->version = DT_NAVMESH_VERSION;
	header->resolution = params->tileResolutionLevel;	//@UE
	header->x = params->tileX;
	header->y = params->tileY;
	setHeaderShort(params->tileLayer, header->layer, TEXT("tile layer"), bAllValuesSet);
	setHeaderShort(totPolyCount, header->polyCount, TEXT("poly count"), bAllValuesSet);
	setHeaderShort(totVertCount, header->vertCount, TEXT("vert count"), bAllValuesSet);
	setHeaderShort(maxLinkCount, header->maxLinkCount, TEXT("max link count"), bAllValuesSet);
	dtVcopy(header->bmin, params->bmin);
	dtVcopy(header->bmax, params->bmax);
	setHeaderShort(params->polyCount, header->detailMeshCount, TEXT("detail mesh count"), bAllValuesSet);
	setHeaderShort(uniqueDetailVertCount, header->detailVertCount, TEXT("detail vert count"), bAllValuesSet);
	setHeaderShort(detailTriCount, header->detailTriCount, TEXT("detail tri count"), bAllValuesSet);
	setHeaderShort(params->polyCount, header->offMeshBase, TEXT("off mesh base"), bAllValuesSet);
	setHeaderShort(storedOffMeshConCount, header->offMeshConCount, TEXT("off mesh connection count"), bAllValuesSet);
	setHeaderShort(params->buildBvTree ? params->polyCount * 2 : 0, header->bvNodeCount, TEXT("bv node count"), bAllValuesSet);

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	setHeaderShort(firstSegPoly, header->offMeshSegPolyBase, TEXT("off mesh seg poly base"), bAllValuesSet);
	setHeaderShort(firstSegVert, header->offMeshSegVertBase, TEXT("off mesh seg vert base"), bAllValuesSet);
	setHeaderShort(storedOffMeshSegCount, header->offMeshSegConCount, TEXT("off mesh seg con count"), bAllValuesSet);
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	setHeaderShort(params->clusterCount, header->clusterCount, TEXT("cluster count"), bAllValuesSet);
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	if (!bAllValuesSet)
	{
		UE_LOGF(LogDetour, Error, "dtCreateNavMeshData: Tile %d, %d: Layer %d: Some dtMeshHeader values are too large to fit in an unsigned short, tile not Generated! Please decrease the size of your nav mesh tiles.", params->tileX, params->tileY, params->tileLayer);

		dtFree(offMeshConClass, DT_ALLOC_TEMP);
		dtFree(data, DT_ALLOC_PERM_TILE_DATA);

		return false;
	}
	//@UE END Memory optimization


	const int offMeshVertsBase = params->vertCount;
	const int offMeshPolyBase = params->polyCount;

	// Store vertices
	// Mesh vertices: stored as tile-local float offsets from params->bmin //@UE
	for (int i = 0; i < params->vertCount; ++i)
	{
		const unsigned short* iv = &params->verts[i*3];
		navVerts[i].x = (float)(iv[0] * params->cs); //@UE
		navVerts[i].y = (float)(iv[1] * params->ch); //@UE
		navVerts[i].z = (float)(iv[2] * params->cs); //@UE
	}
	// Off-mesh point link vertices.
	int n = 0;
	for (int i = 0; i < params->offMeshConCount; ++i)
	{
		const dtOffMeshLinkCreateParams& offMeshCon = params->offMeshCons[i];

		// Only store connections which start from this tile.
		if ((offMeshConClass[i*2+0] == 0xff) && (offMeshCon.type & DT_OFFMESH_CON_POINT))
		{
			navVerts[offMeshVertsBase + n*2    ].setFromWorld(params->bmin, &offMeshCon.vertsA0[0]); //@UE
			navVerts[offMeshVertsBase + n*2 + 1].setFromWorld(params->bmin, &offMeshCon.vertsB0[0]); //@UE
			n++;
		}
	}

	// Store polygons
	// Mesh polys
	const unsigned short* src = params->polys;
	for (int i = 0; i < params->polyCount; ++i)
	{
		dtPoly* p = &navPolys[i];
		p->vertCount = 0;
		p->flags = params->polyFlags[i];
		p->setArea(params->polyAreas[i]);
		p->setType(DT_POLYTYPE_GROUND);
		for (int j = 0; j < nvp; ++j)
		{
			if (src[j] == MESH_NULL_IDX) break;
			p->verts[j] = src[j];
			if (src[nvp+j] & 0x8000)
			{
				// Border or portal edge.
				unsigned short dir = src[nvp+j] & 0xf;
				if (dir == 0xf) // Border
					p->neis[j] = 0;
				else if (dir == 0) // Portal x-
					p->neis[j] = DT_EXT_LINK | 4;
				else if (dir == 1) // Portal z+
					p->neis[j] = DT_EXT_LINK | 2;
				else if (dir == 2) // Portal x+
					p->neis[j] = DT_EXT_LINK | 0;
				else if (dir == 3) // Portal z-
					p->neis[j] = DT_EXT_LINK | 6;
			}
			else
			{
				// Normal connection
				p->neis[j] = src[nvp+j]+1;
			}
			
			p->vertCount++;
		}
		src += nvp*2;
	}
	// Off-mesh point connection polygons.
	n = 0;
	int nseg = 0;
	for (int i = 0; i < params->offMeshConCount; ++i)
	{
		const dtOffMeshLinkCreateParams& offMeshCon = params->offMeshCons[i];

		// Only store connections which start from this tile.
		if (offMeshConClass[i*2+0] == 0xff)
		{
			if (offMeshCon.type & DT_OFFMESH_CON_POINT)
			{
				dtPoly* p = &navPolys[offMeshPolyBase+n];
				p->vertCount = 2;
				p->verts[0] = (unsigned short)(offMeshVertsBase + n*2+0);
				p->verts[1] = (unsigned short)(offMeshVertsBase + n*2+1);
				p->flags = offMeshCon.polyFlag;
				p->setArea(offMeshCon.area);
				p->setType(DT_POLYTYPE_OFFMESH_POINT);
				n++;
			}
			//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
			else
			{
				for (int j = 0; j < DT_MAX_OFFMESH_SEGMENT_PARTS; j++)
				{
					dtPoly* p = &navPolys[firstSegPoly+nseg];
					p->vertCount = 0;
					p->flags = offMeshCon.polyFlag;
					p->setArea(offMeshCon.area);
					p->setType(DT_POLYTYPE_OFFMESH_SEGMENT);
					nseg++;
				}
			}
#endif // WITH_NAVMESH_SEGMENT_LINKS
			//@UE END
		}
	}

	// Store detail meshes and vertices.
	// The nav polygon vertices are stored as the first vertices on each mesh.
	// We compress the mesh data by skipping them and using the navmesh coordinates.
	if (params->detailMeshes)
	{
		unsigned short vbase = 0;
		for (int i = 0; i < params->polyCount; ++i)
		{
			dtPolyDetail& dtl = navDMeshes[i];
			const int vb = (int)params->detailMeshes[i*4+0];
			const int ndv = (int)params->detailMeshes[i*4+1];
			const int nv = navPolys[i].vertCount;
			dtl.vertBase = (unsigned int)vbase;
			dtl.vertCount = (unsigned char)(ndv-nv);
			dtl.triBase = (unsigned short)params->detailMeshes[i*4+2];
			dtl.triCount = (unsigned char)params->detailMeshes[i*4+3];
			// Copy vertices except the first 'nv' verts which are equal to nav poly verts.
			if (ndv-nv)
			{
				for (int k = 0; k < (ndv-nv); ++k) //@UE
				{ //@UE
					navDVerts[vbase+k].setFromWorld(params->bmin, &params->detailVerts[(vb+nv+k)*3]); //@UE
				} //@UE
				vbase += (unsigned short)(ndv-nv);
			}
		}
		// Store triangles.
		memcpy(navDTris, params->detailTris, sizeof(unsigned char)*4*params->detailTriCount);
	}
	else
	{
		// Create dummy detail mesh by triangulating polys.
		int tbase = 0;
		for (int i = 0; i < params->polyCount; ++i)
		{
			dtPolyDetail& dtl = navDMeshes[i];
			const int nv = navPolys[i].vertCount;
			dtl.vertBase = 0;
			dtl.vertCount = 0;
			dtl.triBase = (unsigned short)tbase;
			dtl.triCount = (unsigned char)(nv-2);
			// Triangulate polygon (local indices).
			for (int j = 2; j < nv; ++j)
			{
				unsigned char* t = &navDTris[tbase*4];
				t[0] = 0;
				t[1] = (unsigned char)(j-1);
				t[2] = (unsigned char)j;
				// Bit for each edge that belongs to poly boundary.
				t[3] = (1<<2);
				if (j == 2) t[3] |= (1<<0);
				if (j == nv-1) t[3] |= (1<<4);
				tbase++;
			}
		}
	}

	// Store and create BVtree.
	if (params->buildBvTree)
	{
		//@UE BEGIN
		const int nodeCount = createBVTree(params->verts, params->vertCount, params->polys, params->polyCount, nvp,
					 navDMeshes, navDVerts, navDTris,
					 params->cs, params->ch, params->polyCount*2, navBvtree);
		if (nodeCount == 0)
		{
			dtFree(data, DT_ALLOC_PERM_TILE_DATA);
			dtFree(offMeshConClass, DT_ALLOC_TEMP);
			return false;
		}
		//@UE END
	}
	
	// Store Off-Mesh connections.
	n = 0;
	nseg = 0;
	for (int i = 0; i < params->offMeshConCount; ++i)
	{
		const dtOffMeshLinkCreateParams& offMeshCon = params->offMeshCons[i];

		// Only store connections which start from this tile.
		if (offMeshConClass[i*2+0] == 0xff)
		{
			if (offMeshCon.type & DT_OFFMESH_CON_POINT)
			{
				dtOffMeshConnection* con = &offMeshCons[n];
				con->poly = (unsigned short)(offMeshPolyBase + n);
				// Copy connection end-points.
				dtVcopy(&con->pos[0], &offMeshCon.vertsA0[0]);
				dtVcopy(&con->pos[3], &offMeshCon.vertsB0[0]);
				con->rad = offMeshCon.snapRadius;
				con->height = offMeshCon.snapHeight;
				con->setFlags(offMeshCon.type);
				con->side = offMeshConClass[i*2+1] == 0xff ? DT_CONNECTION_INTERNAL : offMeshConClass[i*2+1];
				if (offMeshCon.userID)
					con->userId = offMeshCon.userID;
				n++;
			}
			//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
			else
			{
				dtOffMeshSegmentConnection* con = &offMeshSegs[nseg];
				dtVcopy(con->startA, &offMeshCon.vertsA0[0]);
				dtVcopy(con->endA, &offMeshCon.vertsA1[0]);
				dtVcopy(con->startB, &offMeshCon.vertsB0[0]);
				dtVcopy(con->endB, &offMeshCon.vertsB1[0]);

				con->rad = offMeshCon.snapRadius;
				con->setFlags(offMeshCon.type);
				if (offMeshCon.userID)
					con->userId = offMeshCon.userID;

				nseg++;
			}
#endif // WITH_NAVMESH_SEGMENT_LINKS
			//@UE END
		}
	}

	dtFree(offMeshConClass, DT_ALLOC_TEMP);

	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	// Store clusters
	if (params->polyClusters)
	{
		memcpy(polyClusters, params->polyClusters, sizeof(unsigned short)*params->polyCount);
	}

	for (int i = 0; i < params->clusterCount; i++)
	{
		dtCluster& cluster = clusters[i];
		cluster.firstLink = DT_NULL_LINK;
		cluster.numLinks = 0;
		dtVset(cluster.center, 0.f, 0.f, 0.f);

		// calculate center point: take from first poly
		for (int j = 0; j < params->polyCount; j++)
		{
			if (polyClusters[j] != i)
			{
				continue;
			}

			const dtPoly* poly = &navPolys[j];
			dtReal c[3] = { 0.0f, 0.0f, 0.0f };

			// Sum vertices in local space
			for (int iv = 0; iv < poly->vertCount; iv++)
			{
				const dtTileVert& v = navVerts[poly->verts[iv]];
				c[0] += (dtReal)v.x;
				c[1] += (dtReal)v.y;
				c[2] += (dtReal)v.z;
			}

			// Convert centroid to world space.
			const dtReal invCount = dtReal(1.) / poly->vertCount;
			dtReal cWorld[3];
			cWorld[0] = c[0] * invCount + params->bmin[0];
			cWorld[1] = c[1] * invCount + params->bmin[1];
			cWorld[2] = c[2] * invCount + params->bmin[2];

			dtVmad(cluster.center, cluster.center, cWorld, dtReal(1.));
			break;
		}
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END
 
	*outData = data;
	*outDataSize = dataSize;
	
	return true;
}

bool dtNavMeshHeaderSwapEndian(unsigned char* data, const int /*dataSize*/)
{
	dtMeshHeader* header = (dtMeshHeader*)data;
	
	int swappedVersion = DT_NAVMESH_VERSION;
	dtSwapEndian(&swappedVersion);
	
	if ((header->version != DT_NAVMESH_VERSION) &&
		(header->version != swappedVersion))
	{
		return false;
	}
		
	dtSwapEndian(&header->version);
	dtSwapEndian(&header->x);
	dtSwapEndian(&header->y);
	dtSwapEndian(&header->layer);
	dtSwapEndian(&header->polyCount);
	dtSwapEndian(&header->vertCount);
	dtSwapEndian(&header->maxLinkCount);
	dtSwapEndian(&header->detailMeshCount);
	dtSwapEndian(&header->detailVertCount);
	dtSwapEndian(&header->detailTriCount);
	dtSwapEndian(&header->bvNodeCount);
	dtSwapEndian(&header->offMeshConCount);
	dtSwapEndian(&header->offMeshBase);
	dtSwapEndian(&header->bmin[0]);
	dtSwapEndian(&header->bmin[1]);
	dtSwapEndian(&header->bmin[2]);
	dtSwapEndian(&header->bmax[0]);
	dtSwapEndian(&header->bmax[1]);
	dtSwapEndian(&header->bmax[2]);

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	dtSwapEndian(&header->offMeshSegConCount);
	dtSwapEndian(&header->offMeshSegPolyBase);
	dtSwapEndian(&header->offMeshSegVertBase);
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	dtSwapEndian(&header->clusterCount);
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	// Freelist index and pointers are updated when tile is added, no need to swap.

	return true;
}

/// @par
///
/// @warning This function assumes that the header is in the correct endianess already. 
/// Call #dtNavMeshHeaderSwapEndian() first on the data if the data is expected to be in wrong endianess 
/// to start with. Call #dtNavMeshHeaderSwapEndian() after the data has been swapped if converting from 
/// native to foreign endianess.
bool dtNavMeshDataSwapEndian(unsigned char* data, const int /*dataSize*/)
{
	// Make sure the data is in right format.
	dtMeshHeader* header = (dtMeshHeader*)data;
	if (header->version != DT_NAVMESH_VERSION)
		return false;
	
	// Patch header pointers.
	const int headerSize = dtAlign(sizeof(dtMeshHeader));
	const int vertsSize = dtAlign(sizeof(dtTileVert)*header->vertCount); //@UE
	const int polysSize = dtAlign(sizeof(dtPoly)*header->polyCount);
	const int linksSize = dtAlign(sizeof(dtLink)*(header->maxLinkCount));
	const int detailMeshesSize = dtAlign(sizeof(dtPolyDetail)*header->detailMeshCount);
	const int detailVertsSize = dtAlign(sizeof(dtTileVert)*header->detailVertCount); //@UE
	const int detailTrisSize = dtAlign(sizeof(unsigned char)*4*header->detailTriCount);
	const int bvtreeSize = dtAlign(sizeof(dtBVNode)*header->bvNodeCount);
	const int offMeshLinksSize = dtAlign(sizeof(dtOffMeshConnection)*header->offMeshConCount);

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	const int offMeshSegSize = dtAlign(sizeof(dtOffMeshSegmentConnection)*header->offMeshSegConCount);
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	const int clustersSize = dtAlign(sizeof(dtCluster)*header->clusterCount);
	const int polyClustersSize = dtAlign(sizeof(unsigned short)*header->offMeshBase);
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	unsigned char* d = data + headerSize;
	dtTileVert* verts = (dtTileVert*)d; d += vertsSize; //@UE
	dtPoly* polys = (dtPoly*)d; d += polysSize;
	/*dtLink* links = (dtLink*)d;*/ d += linksSize;
	dtPolyDetail* detailMeshes = (dtPolyDetail*)d; d += detailMeshesSize;
	dtTileVert* detailVerts = (dtTileVert*)d; d += detailVertsSize; //@UE
	/*unsigned char* detailTris = (unsigned char*)d;*/ d += detailTrisSize;
	dtBVNode* bvTree = (dtBVNode*)d; d += bvtreeSize;
	dtOffMeshConnection* offMeshCons = (dtOffMeshConnection*)d; d += offMeshLinksSize;

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	dtOffMeshSegmentConnection* offMeshSegs = (dtOffMeshSegmentConnection*)d; d += offMeshSegSize;
#endif // WITH_NAVMESH_SEGMENT_LINKS	
 	
#if WITH_NAVMESH_CLUSTER_LINKS
	dtCluster* clusters = (dtCluster*)d; d += clustersSize;
	unsigned short* polyClusters = (unsigned short*)d; d += polyClustersSize;
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	// Vertices
	for (int i = 0; i < header->vertCount; ++i)
	{
		dtSwapEndian(&verts[i].x); //@UE
		dtSwapEndian(&verts[i].y); //@UE
		dtSwapEndian(&verts[i].z); //@UE
	}

	// Polys
	for (int i = 0; i < header->polyCount; ++i)
	{
		dtPoly* p = &polys[i];
		// poly->firstLink is update when tile is added, no need to swap.
		for (int j = 0; j < DT_VERTS_PER_POLYGON; ++j)
		{
			dtSwapEndian(&p->verts[j]);
			dtSwapEndian(&p->neis[j]);
		}
		dtSwapEndian(&p->flags);
	}

	// Links are rebuild when tile is added, no need to swap.

	// Detail meshes
	for (int i = 0; i < header->detailMeshCount; ++i)
	{
		dtPolyDetail* pd = &detailMeshes[i];
		dtSwapEndian(&pd->vertBase);
		dtSwapEndian(&pd->triBase);
	}
	
	// Detail verts
	for (int i = 0; i < header->detailVertCount; ++i)
	{
		dtSwapEndian(&detailVerts[i].x); //@UE
		dtSwapEndian(&detailVerts[i].y); //@UE
		dtSwapEndian(&detailVerts[i].z); //@UE
	}

	// BV-tree
	for (int i = 0; i < header->bvNodeCount; ++i)
	{
		dtBVNode* node = &bvTree[i];
		for (int j = 0; j < 3; ++j)
		{
			dtSwapEndian(&node->bmin[j]);
			dtSwapEndian(&node->bmax[j]);
		}
		dtSwapEndian(&node->i);
	}

	// Off-mesh Connections: point type
	for (int i = 0; i < header->offMeshConCount; ++i)
	{
		dtOffMeshConnection* con = &offMeshCons[i];
		for (int j = 0; j < 6; ++j)
			dtSwapEndian(&con->pos[j]);
		dtSwapEndian(&con->rad);
		dtSwapEndian(&con->poly);
		dtSwapEndian(&con->userId);
	}

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	// Off-mesh Connections: segment type
	for (int i = 0; i < header->offMeshSegConCount; ++i)
	{
		dtOffMeshSegmentConnection* con = &offMeshSegs[i];
		for (int j = 0; j < 3; ++j)
		{
			dtSwapEndian(&con->startA[j]);
			dtSwapEndian(&con->startB[j]);
			dtSwapEndian(&con->endA[j]);
			dtSwapEndian(&con->endB[j]);
		}
		dtSwapEndian(&con->rad);
		dtSwapEndian(&con->firstPoly);
		dtSwapEndian(&con->userId);
	}
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	for (int i = 0; i < header->offMeshBase; i++)
	{
		dtSwapEndian(&polyClusters[i]);
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END
 
	return true;
}

// @UE BEGIN
// Experimental tile transform
bool dtTransformTileData(unsigned char* data, const int dataSize, const int offsetX, const int offsetY, const dtReal tileWidth, const dtReal tileHeight, const dtReal rotationDeg, const dtReal bvQuantFactor)
{
	// Validate minimum size before casting to header
	if (dataSize < (int)sizeof(dtMeshHeader))
		return false;

	// Make sure the data is in right format.
	dtMeshHeader* header = (dtMeshHeader*)data;
	if (header->version != DT_NAVMESH_VERSION)
		return false;

	// Set new coords
	header->x += offsetX;
	header->y += offsetY;

	// Patch header pointers.
	const int headerSize = dtAlign(sizeof(dtMeshHeader));
	const int vertsSize = dtAlign(sizeof(dtTileVert)*header->vertCount);
	const int polysSize = dtAlign(sizeof(dtPoly)*header->polyCount);
	const int linksSize = dtAlign(sizeof(dtLink)*(header->maxLinkCount));
	const int detailMeshesSize = dtAlign(sizeof(dtPolyDetail)*header->detailMeshCount);
	const int detailVertsSize = dtAlign(sizeof(dtTileVert)*header->detailVertCount);
	const int detailTrisSize = dtAlign(sizeof(unsigned char)*4*header->detailTriCount);
	const int bvtreeSize = dtAlign(sizeof(dtBVNode)*header->bvNodeCount);
	const int offMeshLinksSize = dtAlign(sizeof(dtOffMeshConnection)*header->offMeshConCount);

#if WITH_NAVMESH_SEGMENT_LINKS
	const int offMeshSegsSize = dtAlign(sizeof(dtOffMeshSegmentConnection)*header->offMeshSegConCount);
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	const int clustersSize = dtAlign(sizeof(dtCluster)*header->clusterCount);
	const int clusterPolysSize = dtAlign(sizeof(unsigned short)*header->offMeshBase);
#endif //WITH_NAVMESH_CLUSTER_LINKS

	dtMeshTile tile;
	tile.header = header;

	unsigned char* d = data + headerSize;
	tile.verts = (dtTileVert*)d; d += vertsSize;
	tile.polys = (dtPoly*)d; d += polysSize;
	tile.links = (dtLink*)d; d += linksSize;
	tile.detailMeshes = (dtPolyDetail*)d; d += detailMeshesSize;
	tile.detailVerts = (dtTileVert*)d; d += detailVertsSize;
	tile.detailTris = (unsigned char*)d; d += detailTrisSize;
	tile.bvTree = (dtBVNode*)d; d += bvtreeSize;
	tile.offMeshCons = (dtOffMeshConnection*)d; d += offMeshLinksSize;

#if WITH_NAVMESH_SEGMENT_LINKS
	tile.offMeshSeg = (dtOffMeshSegmentConnection*)d; d += offMeshSegsSize;
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	tile.clusters = (dtCluster*)d; d += clustersSize;
	tile.polyClusters = (unsigned short*)d; d += clusterPolysSize;
#endif //WITH_NAVMESH_CLUSTER_LINKS

	// Validate that the computed layout fits within the provided buffer
	const int computedSize = (int)(d - data);
	if (computedSize > dataSize)
	{
		return false;
	}

	// Save old bmin before transforming the header; needed to toWorld tile-local vertex offsets
	dtReal oldBmin[3];
	dtVcopy(oldBmin, header->bmin);

	// Rotate on original center position
	dtReal rotationCenter[3];
	dtVadd(rotationCenter, header->bmin, header->bmax);
	dtVscale(rotationCenter, rotationCenter, 0.5f);

	// Compute offset
	dtReal offset[3];
	offset[0] = tileWidth * offsetX;
	offset[1] = 0.f;
	offset[2] = tileHeight * offsetY;

	// Compute center for bvtree
	unsigned short qMin[3];
	unsigned short qMax[3];
	qMin[0] = (unsigned short)(bvQuantFactor * tile.header->bmin[0]);
	qMin[1] = (unsigned short)(bvQuantFactor * tile.header->bmin[1]);
	qMin[2] = (unsigned short)(bvQuantFactor * tile.header->bmin[2]);
	qMax[0] = (unsigned short)(bvQuantFactor * tile.header->bmax[0]);
	qMax[1] = (unsigned short)(bvQuantFactor * tile.header->bmax[1]);
	qMax[2] = (unsigned short)(bvQuantFactor * tile.header->bmax[2]);
	unsigned short qLocalCenter[3];
	qLocalCenter[0] = (qMax[0] - qMin[0])/2;
	qLocalCenter[1] = (qMax[1] - qMin[1])/2;
	qLocalCenter[2] = (qMax[2] - qMin[2])/2;

	const dtRotation rot = dtSelectRotation(rotationDeg);

	// Transform tile bounds
	dtReal tmin[3];
	dtReal tmax[3];
	dtRotate90(tmin, tile.header->bmin, rotationCenter, rot);
	dtVadd(tmin, tmin, offset);
	dtRotate90(tmax, tile.header->bmax, rotationCenter, rot);
	dtVadd(tmax, tmax, offset);
	dtVcopy(tile.header->bmin, tmin);
	dtVcopy(tile.header->bmax, tmax);
	dtVmin(tile.header->bmin, tmax);
	dtVmax(tile.header->bmax, tmin);

	// Update sides
	for (int j = 0; j < tile.header->polyCount; ++j)
	{
		dtPoly* poly = &tile.polys[j];
		const int nv = poly->vertCount;
		for (int vi = 0; vi < nv; ++vi)
		{
			// Skip non-portal edges.
			if ((poly->neis[vi] & DT_EXT_LINK) == 0)
				continue;

			// Rotate
			// 3 2 1
			// 4   0
			// 5 6 7
			const unsigned short side = poly->neis[vi] & DT_LINK_FLAG_SIDE_MASK;
			const unsigned short newSide = (side + (2*rot)) % 8; //rot [0..3], newSide [0,2,4,6]
			poly->neis[vi] = (poly->neis[vi] & ~DT_LINK_FLAG_SIDE_MASK) | newSide;
		}
	}

	// Transform tile vertices (transform from old bmin, rotate, transform into new bmin)
	for (int j = 0; j < tile.header->vertCount; ++j)
	{
		dtReal v[3], vOut[3];
		tile.verts[j].toWorld(oldBmin, v); //@UE
		dtRotate90(vOut, v, rotationCenter, rot);
		dtVadd(vOut, vOut, offset);
		tile.verts[j].setFromWorld(tile.header->bmin, vOut); //@UE
	}

	// Transform tile detail vertices (transform from old bmin, rotate, transform into new bmin)
	for (int j = 0; j < tile.header->detailVertCount; ++j)
	{
		dtReal v[3], vOut[3];
		tile.detailVerts[j].toWorld(oldBmin, v); //@UE
		dtRotate90(vOut, v, rotationCenter, rot);
		dtVadd(vOut, vOut, offset);
		tile.detailVerts[j].setFromWorld(tile.header->bmin, vOut); //@UE
	}

	// Transfrom BVTree (bmin and bmax are local to the tile)
	for (int j = 0; j < tile.header->bvNodeCount; ++j)
	{
		dtBVNode* node = &tile.bvTree[j];
		unsigned short min[3];
		unsigned short max[3];
		dtRotate90(min, node->bmin, qLocalCenter, rot);
		dtRotate90(max, node->bmax, qLocalCenter, rot);
		node->bmin[0] = dtMin(min[0], max[0]);
		node->bmin[1] = dtMin(min[1], max[1]);
		node->bmin[2] = dtMin(min[2], max[2]);
		node->bmax[0] = dtMax(min[0], max[0]);
		node->bmax[1] = dtMax(min[1], max[1]);
		node->bmax[2] = dtMax(min[2], max[2]);
	}

	// Transform off-mesh connections
	for (int j = 0; j < tile.header->offMeshConCount; ++j)
	{
		dtRotate90(&(tile.offMeshCons[j].pos[0]), &(tile.offMeshCons[j].pos[0]), rotationCenter, rot);
		dtRotate90(&(tile.offMeshCons[j].pos[3]), &(tile.offMeshCons[j].pos[3]), rotationCenter, rot);
		dtVadd(&(tile.offMeshCons[j].pos[0]), &(tile.offMeshCons[j].pos[0]), offset);
		dtVadd(&(tile.offMeshCons[j].pos[3]), &(tile.offMeshCons[j].pos[3]), offset);

		const unsigned short realSide = tile.offMeshCons[j].side & DT_LINK_FLAG_SIDE_MASK;
		const unsigned short newSide = (realSide + (2*rot)) % 8; //rot [0..3], side [0,1,2,3,4,5,6,7]
		tile.offMeshCons[j].side = (unsigned char)((tile.offMeshCons[j].side & ~DT_LINK_FLAG_SIDE_MASK) | newSide);
	}

#if WITH_NAVMESH_SEGMENT_LINKS
	// Transform off-mesh segment connections
	for (int j = 0; j < tile.header->offMeshSegConCount; ++j)
	{
		dtRotate90(&(tile.offMeshSeg[j].startA[0]), &(tile.offMeshSeg[j].startA[0]), rotationCenter, rot);
		dtRotate90(&(tile.offMeshSeg[j].endA[0]), &(tile.offMeshSeg[j].endA[0]), rotationCenter, rot);
		dtRotate90(&(tile.offMeshSeg[j].startB[0]), &(tile.offMeshSeg[j].startB[0]), rotationCenter, rot);
		dtRotate90(&(tile.offMeshSeg[j].endB[0]), &(tile.offMeshSeg[j].endB[0]), rotationCenter, rot);
		dtVadd(&(tile.offMeshSeg[j].startA[0]), &(tile.offMeshSeg[j].startA[0]), offset);
		dtVadd(&(tile.offMeshSeg[j].endA[0]), &(tile.offMeshSeg[j].endA[0]), offset);
		dtVadd(&(tile.offMeshSeg[j].startB[0]), &(tile.offMeshSeg[j].startB[0]), offset);
		dtVadd(&(tile.offMeshSeg[j].endB[0]), &(tile.offMeshSeg[j].endB[0]), offset);
	}
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	// Transform clusters
	for (int j = 0; j < tile.header->clusterCount; ++j)
	{
		dtRotate90(&(tile.clusters[j].center[0]), &(tile.clusters[j].center[0]), rotationCenter, rot);
		dtVadd(&(tile.clusters[j].center[0]), &(tile.clusters[j].center[0]), offset);
	}
#endif //WITH_NAVMESH_CLUSTER_LINKS

	return true;
}

void dtComputeTileOffsetFromRotation(const dtReal* position, const dtReal* rotationCenter, const dtReal rotationDeg, const dtReal tileWidth, const dtReal tileHeight, int& deltaX, int& deltaY)
{
	dtReal relativeTilePos[3];
	dtVsub(relativeTilePos, position, rotationCenter);
	dtReal newRelativeTilePos[3] = { 0.f, 0.f, 0.f };
	dtVRot90(newRelativeTilePos, relativeTilePos, dtSelectRotation(rotationDeg));

	dtReal RcTilePosChangedVector[3];
	dtVsub(RcTilePosChangedVector, newRelativeTilePos, relativeTilePos);
	deltaX = (int)(RcTilePosChangedVector[0] / tileWidth);
	deltaY = (int)(RcTilePosChangedVector[2] / tileHeight);
}
// @UE END
