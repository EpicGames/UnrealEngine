// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "Detour/DetourNavMeshQuery.h"
#include "Detour/DetourNode.h"
#include "Detour/DetourAssert.h"

DEFINE_LOG_CATEGORY_STATIC(LogDebugRaycastCrash, All, All);

// To debug the recast AStar enable this define
#define ENABLE_RECAST_ASTAR_LOGGING 0
#if ENABLE_RECAST_ASTAR_LOGGING
	DEFINE_LOG_CATEGORY_STATIC(LogRecastAStar, Display, All);
	#define UE_RECAST_ASTAR_LOG(Verbosity, Format, ...) UE_LOG(LogRecastAStar, Verbosity, Format, __VA_ARGS__)
#else
	#define UE_RECAST_ASTAR_LOG(...)
#endif

/// @class dtQueryFilter
///
/// <b>The Default Implementation</b>
/// 
/// At construction: All area costs default to 1.0.  All flags are included
/// and none are excluded.
/// 
/// If a polygon has both an include and an exclude flag, it will be excluded.
/// 
/// The way filtering works, a navigation mesh polygon must have at least one flag 
/// set to ever be considered by a query. So a polygon with no flags will never
/// be considered.
///
/// Setting the include flags to 0 will result in all polygons being excluded.
///
/// <b>Custom Implementations</b>
/// 
/// dtQueryFilter.isVIrtual must be true in order to extend this class.
/// 
/// Implement a custom query filter by overriding the virtual passFilter() 
/// and getCost() functions. If this is done, both functions should be as 
/// fast as possible. Use cached local copies of data rather than accessing 
/// your own objects where possible.
/// 
/// Custom implementations do not need to adhere to the flags or cost logic 
/// used by the default implementation.  
/// 
/// In order for A* searches to work properly, the cost should be proportional to
/// the travel distance. Implementing a cost modifier less than 1.0 is likely 
/// to lead to problems during pathfinding.
///
/// @see dtNavMeshQuery

dtQueryFilterData::dtQueryFilterData() : heuristicScale(0.999f), lowestAreaCost(1.0f), m_includeFlags(0xffff), m_excludeFlags(0), m_isBacktracking(0)
//@UE BEGIN
	, m_shouldIgnoreClosedNodes(0)
//@UE END
{
	for (int i = 0; i < DT_MAX_AREAS; ++i)
		m_areaCost[i] = 1.0f;

#if WITH_FIXED_AREA_ENTERING_COST
	memset(&m_areaFixedCost, 0, sizeof(m_areaFixedCost));
#endif // WITH_FIXED_AREA_ENTERING_COST
}

bool dtQueryFilterData::equals(const dtQueryFilterData* other) const
{
	bool bEqual =	(heuristicScale == other->heuristicScale) &&
					(lowestAreaCost == other->lowestAreaCost) &&
					(m_includeFlags == other->m_includeFlags) &&
					(m_excludeFlags == other->m_excludeFlags) &&
					(m_isBacktracking == other->m_isBacktracking) &&
//@UE BEGIN
					(m_shouldIgnoreClosedNodes == other->m_shouldIgnoreClosedNodes) &&
//@UE END
					(memcmp(&m_areaCost, &other->m_areaCost, sizeof(m_areaCost)) == 0);

#if WITH_FIXED_AREA_ENTERING_COST
	bEqual = bEqual && (memcmp(&m_areaFixedCost, &other->m_areaFixedCost, sizeof(m_areaFixedCost)) == 0);
#endif // WITH_FIXED_AREA_ENTERING_COST

	return bEqual;
}

void dtQueryFilterData::copyFrom(const dtQueryFilterData* source)
{
	memcpy((void*)this, source, sizeof(dtQueryFilterData));
}

//@UE BEGIN
static const dtReal DEFAULT_HEURISTIC_SCALE = 0.999f; // Search heuristic scale.

/// Compute triangle area in tile-local space.
inline dtReal dtTriArea2DLocal(const dtTileVert& a, const dtTileVert& b, const dtTileVert& c)
{
	const float abx = b.x - a.x;
	const float abz = b.z - a.z;
	const float acx = c.x - a.x;
	const float acz = c.z - a.z;
	return (dtReal)(acx * abz - abx * acz);
}
//@UE END

dtNavMeshQuery* dtAllocNavMeshQuery()
{
	void* mem = dtAlloc(sizeof(dtNavMeshQuery), DT_ALLOC_PERM_NAVQUERY);
	if (!mem) return 0;
	return new(mem) dtNavMeshQuery;
}

void dtFreeNavMeshQuery(dtNavMeshQuery* navmesh)
{
	if (!navmesh) return;
	navmesh->~dtNavMeshQuery();
	dtFree(navmesh, DT_ALLOC_PERM_NAVQUERY);
}

dtQueryResultPack::dtQueryResultPack(dtPolyRef inRef, dtReal inCost, const dtReal* inPos, unsigned int inFlag) :
	ref(inRef), cost(inCost), flag(inFlag)
{
	if (inPos)
	{
		dtVcopy(pos, inPos);
	}
}

void dtQueryResult::getPos(int idx, dtReal* pos)
{
	dtVcopy(pos, data[idx].pos);
}

void dtQueryResult::setPos(int idx, const dtReal* pos)
{
	dtVcopy(data[idx].pos, pos);
}

void dtQueryResult::copyRefs(dtPolyRef* refs, int nmax)
{
	const int count = dtMin(nmax, data.size());
	for (int i = 0; i < count; i++)
	{
		refs[i] = data[i].ref;
	}
}

void dtQueryResult::copyCosts(dtReal* costs, int nmax)
{
	const int count = dtMin(nmax, data.size());
	for (int i = 0; i < count; i++)
	{
		costs[i] = data[i].cost;
	}
}

void dtQueryResult::copyPos(dtReal* pos, int nmax)
{
	const int count = dtMin(nmax, data.size());
	for (int i = 0; i < count; i++)
	{
		dtVcopy(&pos[i * 3], data[i].pos);
	}
}

void dtQueryResult::copyFlags(unsigned char* flags, int nmax)
{
	const int count = dtMin(nmax, data.size());
	for (int i = 0; i < count; i++)
	{
		flags[i] = (unsigned char)data[i].flag;
	}
}

void dtQueryResult::copyFlags(unsigned int* flags, int nmax)
{
	const int count = dtMin(nmax, data.size());
	for (int i = 0; i < count; i++)
	{
		flags[i] = data[i].flag;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////

/// @class dtNavMeshQuery
///
/// For methods that support undersized buffers, if the buffer is too small 
/// to hold the entire result set the return status of the method will include 
/// the #DT_BUFFER_TOO_SMALL flag.
///
/// Constant member functions can be used by multiple clients without side
/// effects. (E.g. No change to the closed list. No impact on an in-progress
/// sliced path query. Etc.)
/// 
/// Walls and portals: A @e wall is a polygon segment that is 
/// considered impassable. A @e portal is a passable segment between polygons.
/// A portal may be treated as a wall based on the dtQueryFilter used for a query.
///
/// @see dtNavMesh, dtQueryFilter, #dtAllocNavMeshQuery(), #dtAllocNavMeshQuery()

dtNavMeshQuery::dtNavMeshQuery() :
	m_nav(0),
	m_linkFilter(0),
	m_tinyNodePool(0),
	m_nodePool(0),
	m_openList(0),
	m_queryNodes(0)
{
	memset(&m_query, 0, sizeof(dtQueryData));
}

dtNavMeshQuery::~dtNavMeshQuery()
{
	if (m_tinyNodePool)
		m_tinyNodePool->~dtNodePool();
	if (m_nodePool)
		m_nodePool->~dtNodePool();
	if (m_openList)
		m_openList->~dtNodeQueue();
	dtFree(m_tinyNodePool, DT_ALLOC_PERM_NAVQUERY);
	dtFree(m_nodePool, DT_ALLOC_PERM_NAVQUERY);
	dtFree(m_openList, DT_ALLOC_PERM_NAVQUERY);
}

/// @par 
///
/// Must be the first function called after construction, before other
/// functions are used.
///
/// This function can be used multiple times.
dtStatus dtNavMeshQuery::init(const dtNavMesh* nav, const int maxNodes, dtQuerySpecialLinkFilter* linkFilter)
{
	m_nav = nav;
	updateLinkFilter(linkFilter);

	if (maxNodes > 0)
	{
		if (!m_nodePool || m_nodePool->getMaxNodes() < maxNodes)
		{
			if (m_nodePool)
			{
				m_nodePool->~dtNodePool();
				dtFree(m_nodePool, DT_ALLOC_PERM_NAVQUERY);
				m_nodePool = 0;
			}
			m_nodePool = new (dtAlloc(sizeof(dtNodePool), DT_ALLOC_PERM_NAVQUERY)) dtNodePool(maxNodes, dtNextPow2(maxNodes / 4));
			if (!m_nodePool
//@UE BEGIN
				&& maxNodes > 0
//@UE END
				)
				return DT_FAILURE | DT_OUT_OF_MEMORY;
		}
		else
		{
			m_nodePool->clear();
		}
//@UE BEGIN
		m_nodePool->setMaxRuntimeNodes(maxNodes);
//@UE END

		if (!m_tinyNodePool)
		{
			m_tinyNodePool = new (dtAlloc(sizeof(dtNodePool), DT_ALLOC_PERM_NAVQUERY)) dtNodePool(64, 32);
			if (!m_tinyNodePool)
				return DT_FAILURE | DT_OUT_OF_MEMORY;
		}
		else
		{
			m_tinyNodePool->clear();
		}

		// TODO: check the open list size too.
		if (!m_openList || m_openList->getCapacity() < maxNodes)
		{
			if (m_openList)
			{
				m_openList->~dtNodeQueue();
				dtFree(m_openList, DT_ALLOC_PERM_NAVQUERY);
				m_openList = 0;
			}
			m_openList = new (dtAlloc(sizeof(dtNodeQueue), DT_ALLOC_PERM_NAVQUERY)) dtNodeQueue(maxNodes);
			if (!m_openList)
				return DT_FAILURE | DT_OUT_OF_MEMORY;
		}
		else
		{
			m_openList->clear();
		}
	}
	
	return DT_SUCCESS;
}

void dtNavMeshQuery::updateLinkFilter(dtQuerySpecialLinkFilter* linkFilter)
{
	m_linkFilter = linkFilter;
	if (m_linkFilter)
	{
		m_linkFilter->initialize();
	}
}

// LWC_TODO_AI: Should be double(*frand)() to be consistent with the rest of the API, UE currently does not have a double rand function but it is planned!
dtStatus dtNavMeshQuery::findRandomPoint(const dtQueryFilter* filter, float(*frand)(),
										 dtPolyRef* randomRef, dtReal* randomPt) const
{
	dtAssert(m_nav);
	
	// Randomly pick one tile. Assume that all tiles cover roughly the same area.
	const dtMeshTile* tile = 0;
	dtReal tsum = 0.0f;
	for (int i = 0; i < m_nav->getMaxTiles(); i++)
	{
		const dtMeshTile* t = m_nav->getTile(i);
		if (!t || !t->header) continue;
		
		// Choose random tile using reservoi sampling.
		const dtReal area = 1.0f; // Could be tile area too.
		tsum += area;
		const dtReal u = frand();
		if (u*tsum <= area)
			tile = t;
	}
	
	if (!tile)
		return DT_FAILURE;

	// Randomly pick one polygon weighted by polygon area.
	const dtPoly* poly = 0;
	dtPolyRef polyRef = 0;
	const dtPolyRef base = m_nav->getPolyRefBase(tile);

	dtReal areaSum = 0.0f;
	for (int i = 0; i < tile->header->polyCount; ++i)
	{
		const dtPoly* p = &tile->polys[i];
		
		// Do not return off-mesh connection polygons.
		if (p->getType() != DT_POLYTYPE_GROUND)
			continue;
		
		// Must pass filter
		const dtPolyRef ref = base | (dtPolyRef)i;
		if (!filter->passFilter(ref, tile, p) || !passLinkFilter(tile, i))
			continue;

		// Calc area of the polygon.
		dtReal polyArea = 0.0f;
		for (int j = 2; j < p->vertCount; ++j)
		{
			polyArea += dtTriArea2DLocal( //@UE
				tile->verts[p->verts[0]], //@UE
				tile->verts[p->verts[j-1]], //@UE
				tile->verts[p->verts[j]] //@UE
			); //@UE
		}

		// Choose random polygon weighted by area, using reservoi sampling.
		areaSum += polyArea;
		const dtReal u = frand();
		if (u*areaSum <= polyArea)
		{
			poly = p;
			polyRef = ref;
		}
	}
	
	if (!poly)
		return DT_FAILURE;

	// Randomly pick point on polygon.
	dtReal verts[3*DT_VERTS_PER_POLYGON];
	dtReal areas[DT_VERTS_PER_POLYGON];
	for (int j = 0; j < poly->vertCount; ++j)
	{
		tile->verts[poly->verts[j]].toWorld(tile->header->bmin, &verts[j*3]); //@UE
	}
	
	const dtReal s = frand();
	const dtReal t = frand();
	
	dtReal pt[3];
	dtRandomPointInConvexPoly(verts, poly->vertCount, areas, s, t, pt);
	
	dtReal h = 0.0f;
	dtStatus status = getPolyHeight(polyRef, pt, &h);
	if (dtStatusFailed(status))
		return status;
	pt[1] = h;
	
	dtVcopy(randomPt, pt);
	*randomRef = polyRef;

	return DT_SUCCESS;
}

dtStatus dtNavMeshQuery::findRandomPointAroundCircle(dtPolyRef startRef, const dtReal* centerPos, const dtReal radius,
													 const dtQueryFilter* filter, float (*frand)(),
													 dtPolyRef* randomRef, dtReal* randomPt) const
{
	dtAssert(m_nav);
	dtAssert(m_nodePool);
	dtAssert(m_openList);
	
	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	const dtMeshTile* startTile = 0;
	const dtPoly* startPoly = 0;
	m_nav->getTileAndPolyByRefUnsafe(startRef, &startTile, &startPoly);
	if (!filter->passFilter(startRef, startTile, startPoly) || !passLinkFilterByRef(startTile, startRef))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	m_nodePool->clear();
	m_openList->clear();
	
	dtNode* startNode = m_nodePool->getNode(startRef);
	dtVcopy(startNode->pos, centerPos);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_OPEN;
	m_openList->push(startNode);
	
	dtStatus status = DT_SUCCESS;
	
	const dtReal radiusSqr = dtSqr(radius);
	dtReal areaSum = 0.0f;

	const int maxPtsPerPoly = 4;
	const int maxRandomPolys = 4; 
	int numRandomPolys = 0;
	int randomPolyIdx = 0;
	dtPolyRef randomRefs[maxRandomPolys] = { 0 };

	while (!m_openList->empty())
	{
		dtNode* bestNode = m_openList->pop();
		bestNode->flags &= ~DT_NODE_OPEN;
		bestNode->flags |= DT_NODE_CLOSED;
		
		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef bestRef = bestNode->id;
		const dtMeshTile* bestTile = 0;
		const dtPoly* bestPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(bestRef, &bestTile, &bestPoly);

		// Place random locations on on ground.
		if (bestPoly->getType() == DT_POLYTYPE_GROUND)
		{
			// Calc area of the polygon.
			dtReal polyArea = 0.0f;
			for (int j = 2; j < bestPoly->vertCount; ++j)
			{
				polyArea += dtTriArea2DLocal( //@UE
					bestTile->verts[bestPoly->verts[0]], //@UE
					bestTile->verts[bestPoly->verts[j-1]], //@UE
					bestTile->verts[bestPoly->verts[j]] //@UE
				); //@UE
			}
			// Choose random polygon weighted by area, using reservoi sampling.
			areaSum += polyArea;
			const dtReal u = frand();
			if (u*areaSum <= polyArea)
			{
				randomRefs[randomPolyIdx] = bestRef;
				numRandomPolys++;
				randomPolyIdx = (randomPolyIdx + 1) % maxRandomPolys;
			}
		}
		
		
		// Get parent poly and tile.
		dtPolyRef parentRef = 0;
		const dtMeshTile* parentTile = 0;
		const dtPoly* parentPoly = 0;
		if (bestNode->pidx)
			parentRef = m_nodePool->getNodeAtIdx(bestNode->pidx)->id;
		if (parentRef)
			m_nav->getTileAndPolyByRefUnsafe(parentRef, &parentTile, &parentPoly);
		
		unsigned int i = bestPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(bestTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours and do not follow back to parent.
			if (!neighbourRef || neighbourRef == parentRef
//@UE BEGIN
				|| !filter->isValidLinkSide(link.side))
//@UE END
				continue;
			
			// Expand to neighbour
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);
			
			// Do not advance if the polygon is excluded by the filter.
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
				continue;
			
			// Find edge and calc distance to the edge.
			dtReal va[3], vb[3];
			if (!getPortalPoints(bestRef, bestPoly, bestTile, neighbourRef, neighbourPoly, neighbourTile, va, vb))
				continue;
			
			// If the circle is not touching the next polygon, skip it.
			dtReal tseg;
			dtReal distSqr = dtDistancePtSegSqr2D(centerPos, va, vb, tseg);
			if (distSqr > radiusSqr)
				continue;
			
			dtNode* neighbourNode = m_nodePool->getNode(neighbourRef);
			if (!neighbourNode)
			{
				status |= DT_OUT_OF_NODES;
				continue;
			}
			
			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;
			
			// Cost
			if (neighbourNode->flags == 0)
				dtVlerp(neighbourNode->pos, va, vb, 0.5f);
			
			const dtReal total = bestNode->total + dtVdist(bestNode->pos, neighbourNode->pos);
			
			// The node is already in open list and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_OPEN) && total >= neighbourNode->total)
				continue;
			
			neighbourNode->id = neighbourRef;
			neighbourNode->flags = (neighbourNode->flags & ~DT_NODE_CLOSED);
			neighbourNode->pidx = m_nodePool->getNodeIdx(bestNode);
			neighbourNode->total = total;
			
			if (neighbourNode->flags & DT_NODE_OPEN)
			{
				m_openList->modify(neighbourNode);
			}
			else
			{
				neighbourNode->flags = DT_NODE_OPEN;
				m_openList->push(neighbourNode);
			}
		}
	}

	dtReal verts[3*DT_VERTS_PER_POLYGON] = {0};
	dtReal areas[DT_VERTS_PER_POLYGON] = {0};
	bool foundPt = false;

	numRandomPolys = dtMin(numRandomPolys, maxRandomPolys);
	for (int iPoly = numRandomPolys - 1; iPoly >= 0 && !foundPt; iPoly--)
	{
		const dtPolyRef testRef = randomRefs[iPoly];
		const dtMeshTile* testTile = 0;
		const dtPoly* testPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(testRef, &testTile, &testPoly);

		for (int j = 0; j < testPoly->vertCount; ++j)
		{
			testTile->verts[testPoly->verts[j]].toWorld(testTile->header->bmin, &verts[j*3]); //@UE
		}

		dtReal pt[3];
		for (int iTry = 0; iTry < maxPtsPerPoly; iTry++)
		{
			const dtReal s = frand();
			const dtReal t = frand();
			dtRandomPointInConvexPoly(verts, testPoly->vertCount, areas, s, t, pt);

			const dtReal distSqr = dtVdist2DSqr(centerPos, pt);
			if (distSqr < radiusSqr)
			{
				dtReal h = 0.0f;
				const dtStatus stat = getPolyHeight(testRef, pt, &h);
				if (!dtStatusFailed(stat))
				{
					pt[1] = h;
					dtVcopy(randomPt, pt);
					*randomRef = testRef;
					foundPt = true;
					break;
				}
			}
		}
	}

	return foundPt ? DT_SUCCESS : DT_FAILURE;
}

dtStatus dtNavMeshQuery::findRandomPointInPoly(dtPolyRef ref, float(*frand)(), dtReal* randomPt) const
{
	dtAssert(m_nav);

	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(ref, &tile, &poly)))
		return DT_FAILURE | DT_INVALID_PARAM;

	if (poly->getType() == DT_POLYTYPE_OFFMESH_POINT)
	{
		dtReal v0[3], v1[3]; //@UE
		tile->verts[poly->verts[0]].toWorld(tile->header->bmin, v0); //@UE
		tile->verts[poly->verts[1]].toWorld(tile->header->bmin, v1); //@UE
		const dtReal s = frand();
		dtVlerp(randomPt, v0, v1, s);
		return DT_SUCCESS;
	}
	else
	{
		dtReal verts[3*DT_VERTS_PER_POLYGON] = {0};
		dtReal areas[DT_VERTS_PER_POLYGON] = {0};
		for (int j = 0; j < poly->vertCount; ++j)
		{
			tile->verts[poly->verts[j]].toWorld(tile->header->bmin, &verts[j*3]); //@UE
		}

		const dtReal s = frand();
		const dtReal t = frand();

		dtReal pt[3];
		dtRandomPointInConvexPoly(verts, poly->vertCount, areas, s, t, pt);

		dtReal h = 0.0;
		dtStatus status = getPolyHeight(ref, pt, &h);
		if (dtStatusFailed(status))
			return status;
		pt[1] = h;

		dtVcopy(randomPt, pt);
	}

	return DT_SUCCESS;
}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
dtStatus dtNavMeshQuery::findRandomPointInCluster(dtClusterRef clusterRef, float(*frand)(), dtPolyRef* randomRef, dtReal* randomPt) const
{
	dtAssert(m_nav);
	dtAssert(m_nodePool);
	dtAssert(m_openList);

	// Validate input
	if (!clusterRef)
		return DT_FAILURE | DT_INVALID_PARAM;

	const dtMeshTile* searchTile = m_nav->getTileByRef(clusterRef);
	const unsigned int clusterIdx = m_nav->decodeClusterIdCluster(clusterRef);
	if (searchTile == 0 || searchTile->polyClusters == 0 ||
		clusterIdx >= (unsigned int)searchTile->header->clusterCount)
	{
		// this means most probably the hierarchical graph has not been build at all
		return DT_FAILURE | DT_INVALID_PARAM;
	}

	dtReal areaSum = 0.0f;
	const dtPoly* randomPoly = 0;
	int randomPolyIdx = 0;

	const int maxGroundPolys = searchTile->header->offMeshBase;
	for (int i = 0; i < maxGroundPolys; i++)
	{
		if (searchTile->polyClusters[i] == clusterIdx)
		{
			dtPoly* testPoly = &searchTile->polys[i];

			// Calc area of the polygon.
			dtReal polyArea = 0.0f;
			for (int j = 2; j < testPoly->vertCount; ++j)
			{
				polyArea += dtTriArea2DLocal( //@UE
					searchTile->verts[testPoly->verts[0]], //@UE
					searchTile->verts[testPoly->verts[j-1]], //@UE
					searchTile->verts[testPoly->verts[j]] //@UE
				); //@UE
			}
			// Choose random polygon weighted by area, using reservoi sampling.
			areaSum += polyArea;
			const dtReal u = frand();
			if (u*areaSum <= polyArea)
			{
				randomPoly = testPoly;
				randomPolyIdx = i;
			}
		}
	}

	if (!randomPoly)
		return DT_FAILURE;

	dtPolyRef randomPolyRef = m_nav->getPolyRefBase(searchTile) | (dtPolyRef)randomPolyIdx;

	// Randomly pick point on polygon.
	dtReal verts[3*DT_VERTS_PER_POLYGON];
	dtReal areas[DT_VERTS_PER_POLYGON];
	for (int j = 0; j < randomPoly->vertCount; ++j)
	{
		searchTile->verts[randomPoly->verts[j]].toWorld(searchTile->header->bmin, &verts[j*3]); //@UE
	}

	const dtReal s = frand();
	const dtReal t = frand();

	dtReal pt[3];
	dtRandomPointInConvexPoly(verts, randomPoly->vertCount, areas, s, t, pt);

	dtReal h = 0.0f;
	dtStatus status = getPolyHeight(randomPolyRef, pt, &h);
	if (dtStatusFailed(status))
		return status;

	pt[1] = h;

	dtVcopy(randomPt, pt);
	*randomRef = randomPolyRef;

	return DT_SUCCESS;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

//////////////////////////////////////////////////////////////////////////////////////////

/// @par
///
/// Uses the detail polygons to find the surface height. (Most accurate.)
///
/// @p pos does not have to be within the bounds of the polygon or navigation mesh.
///
/// See closestPointOnPolyBoundary() for a limited but faster option.
///
dtStatus dtNavMeshQuery::closestPointOnPoly(dtPolyRef ref, const dtReal* pos, dtReal* closest) const
{
	dtAssert(m_nav);
	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(ref, &tile, &poly)))
		return DT_FAILURE | DT_INVALID_PARAM;
	if (!tile)
		return DT_FAILURE | DT_INVALID_PARAM;
	
	closestPointOnPolyInTile(tile, poly, pos, closest);
	
	return DT_SUCCESS;
}

void dtNavMeshQuery::closestPointOnPolyInTile(const dtMeshTile* tile, const dtPoly* poly,
											  const dtReal* pos, dtReal* closest) const
{
	// Off-mesh connections don't have detail polygons.
	if (poly->getType() == DT_POLYTYPE_OFFMESH_POINT)
	{
		dtReal v0[3], v1[3]; //@UE
		tile->verts[poly->verts[0]].toWorld(tile->header->bmin, v0); //@UE
		tile->verts[poly->verts[1]].toWorld(tile->header->bmin, v1); //@UE
		const dtReal d0 = dtVdist(pos, v0);
		const dtReal d1 = dtVdist(pos, v1);

		//@UE BEGIN
		dtReal u;
		const dtReal sum = d0 + d1;
		if (sum != 0)
		    u = d0 / sum;
		else
		    u = 0;
		//@UE END

		dtVlerp(closest, v0, v1, u);
		return;
	}

	const unsigned int ip = (unsigned int)(poly - tile->polys);

	// Clamp point to be inside the polygon.
	dtReal verts[DT_VERTS_PER_POLYGON*3];
	dtReal edged[DT_VERTS_PER_POLYGON];
	dtReal edget[DT_VERTS_PER_POLYGON];
	const int nv = dtMin(poly->vertCount, DT_VERTS_PER_POLYGON);
	for (int i = 0; i < nv; ++i)
		tile->verts[poly->verts[i]].toWorld(tile->header->bmin, &verts[i*3]); //@UE
	
	dtVcopy(closest, pos);
	if (nv == 0)
	{
		return;
	}

	if (!dtDistancePtPolyEdgesSqr(pos, verts, nv, edged, edget))
	{
		// Point is outside the polygon, dtClamp to nearest edge.
		dtReal dmin = edged[0];
		int imin = 0;
		for (int i = 1; i < nv; ++i)
		{
			if (edged[i] < dmin)
			{
				dmin = edged[i];
				imin = i;
			}
		}
		CA_ASSUME(imin < nv);
		const dtReal* va = &verts[imin*3];
		const dtReal* vb = &verts[((imin+1)%nv)*3];
		dtVlerp(closest, va, vb, edget[imin]);
	}

	// Find height at the location.
	if (poly->getType() == DT_POLYTYPE_GROUND)
	{
		const dtPolyDetail* pd = &tile->detailMeshes[ip];

		for (int j = 0; j < pd->triCount; ++j)
		{
			const unsigned char* t = &tile->detailTris[(pd->triBase+j)*4];
			dtReal vBuf[3][3]; //@UE
			const dtReal* v[3];
			for (int k = 0; k < 3; ++k)
			{
				if (t[k] < poly->vertCount)
				{
					CA_SUPPRESS(6385);
					tile->verts[poly->verts[t[k]]].toWorld(tile->header->bmin, vBuf[k]); //@UE
				}
				else
				{
					tile->detailVerts[pd->vertBase+(t[k]-poly->vertCount)].toWorld(tile->header->bmin, vBuf[k]); //@UE
				}
				v[k] = vBuf[k]; //@UE
			}
			dtReal h;
			if (dtClosestHeightPointTriangle(closest, v[0], v[1], v[2], h))
			{
				closest[1] = h;
				break;
			}
		}
	}
	else
	{
		dtReal h;
		if (dtClosestHeightPointTriangle(closest, &verts[0], &verts[6], &verts[3], h))
		{
			closest[1] = h;
		}
		else if (dtClosestHeightPointTriangle(closest, &verts[3], &verts[6], &verts[9], h))
		{
			closest[1] = h;
		}
	}
}

/// @par
///
/// Much faster than closestPointOnPoly().
///
/// If the provided position lies within the polygon's xz-bounds (above or below), 
/// then @p pos and @p closest will be equal.
///
/// The height of @p closest will be the polygon boundary.  The height detail is not used.
/// 
/// @p pos does not have to be within the bounds of the polybon or the navigation mesh.
/// 
dtStatus dtNavMeshQuery::closestPointOnPolyBoundary(dtPolyRef ref, const dtReal* pos, dtReal* closest) const
{
	dtAssert(m_nav);
	
	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(ref, &tile, &poly)))
	{
		return DT_FAILURE | DT_INVALID_PARAM;
	}

	if (poly->vertCount == 0)
	{
		return DT_FAILURE;
	}
	
	// Collect vertices.
	dtReal verts[DT_VERTS_PER_POLYGON*3];
	dtReal edged[DT_VERTS_PER_POLYGON];
	dtReal edget[DT_VERTS_PER_POLYGON];
	int nv = 0;
	for (int i = 0; i < (int)poly->vertCount; ++i)
	{
		tile->verts[poly->verts[i]].toWorld(tile->header->bmin, &verts[nv*3]); //@UE
		nv++;
	}		
	
	bool inside = dtDistancePtPolyEdgesSqr(pos, verts, nv, edged, edget);
	if (inside)
	{
		// Point is inside the polygon, return the point.
		dtVcopy(closest, pos);
	}
	else
	{
		// Point is outside the polygon, dtClamp to nearest edge.
		dtReal dmin = edged[0];
		int imin = 0;
		for (int i = 1; i < nv; ++i)
		{
			if (edged[i] < dmin)
			{
				dmin = edged[i];
				imin = i;
			}
		}
		CA_ASSUME(imin < nv);
		const dtReal* va = &verts[imin*3];
		const dtReal* vb = &verts[((imin+1)%nv)*3];
		dtVlerp(closest, va, vb, edget[imin]);
	}
	
	return DT_SUCCESS;
}

/// @par
///
/// Uses the detail polygons to find the surface height. (Most accurate.)
///
/// @p pos does not have to be within the bounds of the polygon or navigation mesh.
///
dtStatus dtNavMeshQuery::projectedPointOnPoly(dtPolyRef ref, const dtReal* pos, dtReal* projected) const
{
	dtAssert(m_nav);
	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(ref, &tile, &poly)))
		return DT_FAILURE | DT_INVALID_PARAM;
	if (!tile)
		return DT_FAILURE | DT_INVALID_PARAM;

	return projectedPointOnPolyInTile(tile, poly, pos, projected);
}

dtStatus dtNavMeshQuery::isPointInsidePoly(dtPolyRef ref, const dtReal* pos, bool& result) const
{
	dtAssert(m_nav);
	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	result = false; //@UE
	
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(ref, &tile, &poly)))
		return DT_FAILURE | DT_INVALID_PARAM;

	if (!tile)
		return DT_FAILURE | DT_INVALID_PARAM;

	if (poly->getType() == DT_POLYTYPE_OFFMESH_POINT)
		return DT_FAILURE; //@UE

	// Clamp point to be inside the polygon.
	dtReal verts[DT_VERTS_PER_POLYGON * 3];
	const int nv = poly->vertCount;
	for (int i = 0; i < nv; ++i)
		tile->verts[poly->verts[i]].toWorld(tile->header->bmin, &verts[i * 3]); //@UE

	result = dtPointInPolygon(pos, verts, nv);

	return DT_SUCCESS;
}

dtStatus dtNavMeshQuery::projectedPointOnPolyInTile(const dtMeshTile* tile, const dtPoly* poly,
	const dtReal* pos, dtReal* projected) const
{
	// Off-mesh connections don't have detail polygons.
	if (poly->getType() == DT_POLYTYPE_OFFMESH_POINT)
	{
		dtReal v0[3], v1[3]; //@UE
		tile->verts[poly->verts[0]].toWorld(tile->header->bmin, v0); //@UE
		tile->verts[poly->verts[1]].toWorld(tile->header->bmin, v1); //@UE
		const dtReal d0 = dtVdist(pos, v0);
		const dtReal d1 = dtVdist(pos, v1);

		//@UE BEGIN
		dtReal u;
		const dtReal sum = d0 + d1;
		if (sum != 0)
			u = d0 / sum;
		else
			u = 0;
		//@UE END

		dtVlerp(projected, v0, v1, u);
		
		// @todo this is not quite true, this calculated the closes point, not a projection
		return DT_SUCCESS;
	}

	const unsigned int ip = (unsigned int)(poly - tile->polys);

	// Clamp point to be inside the polygon.
	dtReal verts[DT_VERTS_PER_POLYGON * 3];
	const int nv = poly->vertCount;
	for (int i = 0; i < nv; ++i)
		tile->verts[poly->verts[i]].toWorld(tile->header->bmin, &verts[i * 3]); //@UE

	// copy source to output, just to have any valid information there
	dtVcopy(projected, pos);

	if (dtPointInPolygon(pos, verts, nv))
	{
		// adjust point's height
		// @todo this is an approximation. Implement a proper solution if needed
		dtReal h = 0;
		for (int i = 0; i < nv; ++i)
			h += verts[i * 3 + 1];
		
		projected[1] = h / nv;

		return DT_SUCCESS;
	}

	return DT_FAILURE;
}

//@UE BEGIN
void dtNavMeshQuery::dtApplyEpsilon(dtReal* extents)
{
	constexpr dtReal EPS = 1.0e-6;
	dtReal epsilonVector[3] = { EPS, EPS, EPS };
	dtVmax(extents, epsilonVector);
}
//@UE END

/// @par
///
/// Will return #DT_FAILURE if the provided position is outside the xz-bounds 
/// of the polygon.
/// 
dtStatus dtNavMeshQuery::getPolyHeight(dtPolyRef ref, const dtReal* pos, dtReal* height) const
{
	dtAssert(m_nav);

	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(ref, &tile, &poly)))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	if (poly->getType() == DT_POLYTYPE_OFFMESH_POINT)
	{
		dtReal v0[3], v1[3]; //@UE
		tile->verts[poly->verts[0]].toWorld(tile->header->bmin, v0); //@UE
		tile->verts[poly->verts[1]].toWorld(tile->header->bmin, v1); //@UE
		const dtReal d0 = dtVdist(pos, v0);
		const dtReal d1 = dtVdist(pos, v1);

		//@UE BEGIN
		dtReal u;
		const dtReal sum = d0 + d1;
		if (sum != 0)
			u = d0 / sum;
		else
			u = 0;
		//@UE END

		if (height)
			*height = v0[1] + (v1[1] - v0[1]) * u;
		return DT_SUCCESS;
	}
	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	else if (poly->getType() == DT_POLYTYPE_OFFMESH_SEGMENT)
	{
		dtReal sv0[3], sv1[3], sv2[3], sv3[3]; //@UE
		tile->verts[poly->verts[0]].toWorld(tile->header->bmin, sv0); //@UE
		tile->verts[poly->verts[1]].toWorld(tile->header->bmin, sv1); //@UE
		tile->verts[poly->verts[2]].toWorld(tile->header->bmin, sv2); //@UE
		tile->verts[poly->verts[3]].toWorld(tile->header->bmin, sv3); //@UE
		dtReal h;
		if (dtClosestHeightPointTriangle(pos, sv0, sv2, sv1, h)) //@UE
		{
			if (height) *height = h;
			return DT_SUCCESS;
		}
		else if (dtClosestHeightPointTriangle(pos, sv1, sv2, sv3, h)) //@UE
		{
			if (height) *height = h;
			return DT_SUCCESS;
		}
	}
#endif // WITH_NAVMESH_SEGMENT_LINKS
	//@UE END
	else
	{
		const unsigned int ip = (unsigned int)(poly - tile->polys);
		const dtPolyDetail* pd = &tile->detailMeshes[ip];
		for (int j = 0; j < pd->triCount; ++j)
		{
			const unsigned char* t = &tile->detailTris[(pd->triBase+j)*4];
			dtReal vBuf[3][3]; //@UE
			const dtReal* v[3];
			for (int k = 0; k < 3; ++k)
			{
				if (t[k] < poly->vertCount)
					tile->verts[poly->verts[t[k]]].toWorld(tile->header->bmin, vBuf[k]); //@UE
				else
					tile->detailVerts[pd->vertBase+(t[k]-poly->vertCount)].toWorld(tile->header->bmin, vBuf[k]); //@UE
				v[k] = vBuf[k]; //@UE
			}
			dtReal h;
			if (dtClosestHeightPointTriangle(pos, v[0], v[1], v[2], h))
			{
				if (height)
					*height = h;
				return DT_SUCCESS;
			}
		}
	}
	
	return DT_FAILURE | DT_INVALID_PARAM;
}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
dtStatus dtNavMeshQuery::getPolyCluster(dtPolyRef polyRef, dtClusterRef& clusterRef) const
{
	dtAssert(m_nav);

	if (!polyRef || !m_nav->isValidPolyRef(polyRef))
		return DT_FAILURE | DT_INVALID_PARAM;

	const dtMeshTile* testTile = m_nav->getTileByRef(polyRef);
	const unsigned int testPolyIdx = m_nav->decodePolyIdPoly(polyRef);
	if (testTile->polyClusters == 0)
	{
		// this means most probably the hierarchical graph has not been build at all
		return DT_FAILURE;
	}

	if (testPolyIdx >= (unsigned int)testTile->header->offMeshBase)
	{
		// only ground type polygons are assigned to clusters
		return DT_FAILURE;
	}

	const unsigned short clusterIdx = testTile->polyClusters[testPolyIdx];
	clusterRef = m_nav->getClusterRefBase(testTile) | (dtClusterRef)clusterIdx;
	return DT_SUCCESS;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

/// @par 
///
/// @note If the search box does not intersect any polygons the search will 
/// return #DT_SUCCESS, but @p nearestRef will be zero. So if in doubt, check 
/// @p nearestRef before using @p nearestPt.
///
/// @warning This function is not suitable for large area searches.  If the search
/// extents overlaps more than 128 polygons it may return an invalid result.
///
dtStatus dtNavMeshQuery::findNearestPoly(const dtReal* center, const dtReal* extents,
										 const dtQueryFilter* filter,
										 dtPolyRef* nearestRef, dtReal* nearestPt,
										 const dtReal* referencePt) const
{
	dtAssert(m_nav);

	*nearestRef = 0;

	//@UE BEGIN
	// Make sure the extents are at least epsilon in size. Zero extents (0,y,0) are sometimes used as "ray cast down".
	// The epsilon is there to prevent the query to miss results when the query is right on the (internal) edge of a polygon.
	dtReal localExtents[3];
	dtVcopy(localExtents, extents);
	dtApplyEpsilon(localExtents);
	//@UE END
	
	// Get nearby polygons from proximity grid.
	dtPolyRef polys[128];
	int polyCount = 0;
	if (dtStatusFailed(queryPolygons(center, localExtents, filter, polys, &polyCount, 128)))
		return DT_FAILURE | DT_INVALID_PARAM;
	
//@UE BEGIN
	dtReal referenceLocation[3];
	dtVcopy(referenceLocation, referencePt ? referencePt : center);
	
	// Find nearest polygon amongst the nearby polygons.
	dtPolyRef nearest = 0;
	dtReal nearestDistanceSqr = DT_REAL_MAX;
	for (int i = 0; i < polyCount; ++i)
	{
		dtPolyRef ref = polys[i];
		dtReal closestPtPoly[3];
		const dtStatus result = closestPointOnPoly(ref, referenceLocation, closestPtPoly);
		if (dtStatusFailed(result))
		{
			continue;
		}

		const dtReal d = dtVdistSqr(referenceLocation, closestPtPoly);
		const dtReal h = dtAbs(center[1] - closestPtPoly[1]);
//@UE END
		if (d < nearestDistanceSqr && h <= localExtents[1])
		{
			// Checking horizontal extents except when using a referencePt since that can lead to missing results.
			const bool insideX = referencePt || dtAbs(center[0] - closestPtPoly[0]) <= localExtents[0];	//@UE
			const bool insideZ = referencePt || dtAbs(center[2] - closestPtPoly[2]) <= localExtents[2];	//@UE
			if (insideX && insideZ)
			{
				if (nearestPt)
					dtVcopy(nearestPt, closestPtPoly);
				nearestDistanceSqr = d;
				nearest = ref;
			}
		}
	}
	
	if (nearestRef)
		*nearestRef = nearest;
	
	return DT_SUCCESS;
}

//@UE BEGIN
dtStatus dtNavMeshQuery::findNearestPoly2D(const dtReal* center, const dtReal* extents,
											const dtQueryFilter* filter,
											dtPolyRef* nearestRef, dtReal* nearestPt,
											const dtReal* referencePt, dtReal tolerance) const
{
	dtAssert(m_nav);

	*nearestRef = 0;

	// Make sure the extents are at least epsilon in size. Zero extents (0,y,0) are sometimes used as "ray cast down".
	// The epsilon is there to prevent the query to miss results when the query is right on the (internal) edge of a polygon.
	dtReal localExtents[3];
	dtVcopy(localExtents, extents);
	dtApplyEpsilon(localExtents);
	
	// Get nearby polygons from proximity grid.
	dtPolyRef polys[128];
	int polyCount = 0;
	if (dtStatusFailed(queryPolygons(center, localExtents, filter, polys, &polyCount, 128)))
		return DT_FAILURE | DT_INVALID_PARAM;

	const dtReal toleranceSq = dtSqr(tolerance);
	dtReal referenceLocation[3];
	dtVcopy(referenceLocation, referencePt ? referencePt : center);

	// Find nearest polygon amongst the nearby polygons.
	dtReal bestScoreInTolerance = DT_REAL_MAX;
	dtReal nearestDistanceSqr = DT_REAL_MAX;
	dtReal nearestVertDist = DT_REAL_MAX;
	int32 bestPolyInTolerance = -1;
	int32 bestPolyOutside = -1;

	for (int i = 0; i < polyCount; ++i)
	{
		dtPolyRef ref = polys[i];
		dtReal closestPtPoly[3];
		const dtStatus result = closestPointOnPoly(ref, referenceLocation, closestPtPoly);
		if (result & DT_FAILURE)
		{
			continue;
		}

		const dtReal dSq = dtVdist2DSqr(referenceLocation, closestPtPoly);
		const dtReal h = dtAbs(center[1] - closestPtPoly[1]);

		if (h > localExtents[1])
			continue;

		// If we are not using a referencePt, check extents (else it can lead to missing results).
		if (!referencePt)
		{
			const double deltaX = dtAbs(center[0] - closestPtPoly[0]);
			const double deltaZ = dtAbs(center[2] - closestPtPoly[2]);
			const bool outsideX = deltaX > localExtents[0];
			const bool outsideZ = deltaZ > localExtents[2];
			if (outsideX || outsideZ)
				continue;
		}
		
		// scoring depends if 2D distance to center is within tolerance value
		if (dSq < toleranceSq)
		{
			const dtReal score = dtSqrt(dSq) + h;
			if (score < bestScoreInTolerance)
			{
				dtVcopy(nearestPt, closestPtPoly);
				bestScoreInTolerance = score;
				bestPolyInTolerance = i;
			}
		}
		else
		{
			if ((dSq < nearestDistanceSqr) || (dSq < nearestDistanceSqr + KINDA_SMALL_NUMBER && h < nearestVertDist))
			{
				if (bestPolyInTolerance < 0)
				{
					dtVcopy(nearestPt, closestPtPoly);
				}

				nearestDistanceSqr = dSq;
				nearestVertDist = h;
				bestPolyOutside = i;
			}
		}
	}

	if (nearestRef)
		*nearestRef = (bestPolyInTolerance >= 0) ? polys[bestPolyInTolerance] : ((bestPolyOutside >= 0) ? polys[bestPolyOutside] : 0);

	return DT_SUCCESS;
}
//@UE END

/// @par 
///
/// @note If the search box does not intersect any polygons the search will 
/// return #DT_SUCCESS, but @p nearestRef will be zero. So if in doubt, check 
/// @p nearestRef before using @p nearestPt.
///
/// @warning This function is not suitable for large area searches.  If the search
/// extents overlaps more than 128 polygons it may return an invalid result.
///
dtStatus dtNavMeshQuery::findNearestContainingPoly(const dtReal* center, const dtReal* extents,
												   const dtQueryFilter* filter,
												   dtPolyRef* nearestRef, dtReal* nearestPt) const
{
	dtAssert(m_nav);

	*nearestRef = 0;

	//@UE BEGIN
	// Make sure the extents are at least epsilon in size. Zero extents (0,y,0) are sometimes used as "ray cast down".
	// The epsilon is there to prevent the query to miss results when the query is right on the (internal) edge of a polygon.
	dtReal localExtents[3];
	dtVcopy(localExtents, extents);
	dtApplyEpsilon(localExtents);
	//@UE END
	
	// Get nearby polygons from proximity grid.
	dtPolyRef polys[128];
	int polyCount = 0;
	if (dtStatusFailed(queryPolygons(center, localExtents, filter, polys, &polyCount, 128)))
		return DT_FAILURE | DT_INVALID_PARAM;

	// Find nearest polygon amongst the nearby polygons.
	dtPolyRef nearest = 0;
	dtReal nearestDistanceSqr = DT_REAL_MAX;
	for (int i = 0; i < polyCount; ++i)
	{
		dtPolyRef ref = polys[i];

		bool inPoly = false;
		isPointInsidePoly(ref, center, inPoly);

		if (inPoly)
		{
			dtReal closestPtPoly[3];
			const dtStatus result = closestPointOnPoly(ref, center, closestPtPoly);
			if (result & DT_FAILURE)
			{
				continue;
			}

			const dtReal d = dtVdistSqr(center, closestPtPoly);
			const dtReal x = dtAbs(center[0] - closestPtPoly[0]);
			const dtReal z = dtAbs(center[2] - closestPtPoly[2]);
			const dtReal h = dtAbs(center[1] - closestPtPoly[1]);

			if (d < nearestDistanceSqr && h <= localExtents[1] && x <= localExtents[0] && z <= localExtents[2]) //@UE
			{
				if (nearestPt)
					dtVcopy(nearestPt, closestPtPoly);
				nearestDistanceSqr = d;
				nearest = ref;
			}
		}
	}

	if (nearestRef)
		*nearestRef = nearest;

	return DT_SUCCESS;
}

dtPolyRef dtNavMeshQuery::findNearestPolyInTile(const dtMeshTile* tile, const dtReal* center, const dtReal* extents,
												const dtQueryFilter* filter, dtReal* nearestPt) const
{
	dtAssert(m_nav);

	//@UE BEGIN
	// Make sure the extents are at least epsilon in size. Zero extents (0,y,0) are sometimes used as "ray cast down".
	// The epsilon is there to prevent the query to miss results when the query is right on the (internal) edge of a polygon.
	dtReal localExtents[3];
	dtVcopy(localExtents, extents);
	dtApplyEpsilon(localExtents);
	//@UE END
	
	dtReal bmin[3], bmax[3];
	dtVsub(bmin, center, localExtents);
	dtVadd(bmax, center, localExtents);
	
	// Get nearby polygons from proximity grid.
	dtPolyRef polys[128];
	int polyCount = queryPolygonsInTile(tile, bmin, bmax, filter, polys, 128);
	
	// Find nearest polygon amongst the nearby polygons.
	dtPolyRef nearest = 0;
	dtReal nearestDistanceSqr = DT_REAL_MAX;
	for (int i = 0; i < polyCount; ++i)
	{
		dtPolyRef ref = polys[i];
		const dtPoly* poly = &tile->polys[m_nav->decodePolyIdPoly(ref)];
		dtReal closestPtPoly[3];
		closestPointOnPolyInTile(tile, poly, center, closestPtPoly);
		const dtReal d = dtVdistSqr(center, closestPtPoly);
		const dtReal x = dtAbs(center[0] - closestPtPoly[0]);
		const dtReal z = dtAbs(center[2] - closestPtPoly[2]);
		const dtReal h = dtAbs(center[1] - closestPtPoly[1]);

		if (d < nearestDistanceSqr && h <= localExtents[1] && x <= localExtents[0] && z <= localExtents[2])	//@UE
		{
			if (nearestPt)
				dtVcopy(nearestPt, closestPtPoly);
			nearestDistanceSqr = d;
			nearest = ref;
		}
	}
	
	return nearest;
}

int dtNavMeshQuery::queryPolygonsInTile(const dtMeshTile* tile, const dtReal* qmin, const dtReal* qmax,
										const dtQueryFilter* filter,
										dtPolyRef* polys, const int maxPolys) const
{
	dtAssert(m_nav);

	const bool bIsInsideTile = dtOverlapBounds(qmin,qmax, tile->header->bmin,tile->header->bmax);
	if (!bIsInsideTile)
	{
		return 0;
	}

	if (tile->bvTree)
	{
		const dtBVNode* node = &tile->bvTree[0];
		const dtBVNode* end = &tile->bvTree[tile->header->bvNodeCount];
		const dtReal* tbmin = tile->header->bmin;
		const dtReal* tbmax = tile->header->bmax;
		const dtReal qfac = m_nav->getBVQuantFactor(tile->header->resolution);	//@UE

		// Calculate quantized box
		unsigned short bmin[3], bmax[3];
		// dtClamp query box to world box.
		dtReal minx = dtClamp(qmin[0], tbmin[0], tbmax[0]) - tbmin[0];
		dtReal miny = dtClamp(qmin[1], tbmin[1], tbmax[1]) - tbmin[1];
		dtReal minz = dtClamp(qmin[2], tbmin[2], tbmax[2]) - tbmin[2];
		dtReal maxx = dtClamp(qmax[0], tbmin[0], tbmax[0]) - tbmin[0];
		dtReal maxy = dtClamp(qmax[1], tbmin[1], tbmax[1]) - tbmin[1];
		dtReal maxz = dtClamp(qmax[2], tbmin[2], tbmax[2]) - tbmin[2];
		// Quantize
		bmin[0] = (unsigned short)(qfac * minx) & 0xfffe;
		bmin[1] = (unsigned short)(qfac * miny) & 0xfffe;
		bmin[2] = (unsigned short)(qfac * minz) & 0xfffe;
		bmax[0] = (unsigned short)(qfac * maxx + 1) | 1;
		bmax[1] = (unsigned short)(qfac * maxy + 1) | 1;
		bmax[2] = (unsigned short)(qfac * maxz + 1) | 1;
		
		// Traverse tree
		const dtPolyRef base = m_nav->getPolyRefBase(tile);
		int n = 0;
		while (node < end)
		{
			const bool overlap = dtOverlapQuantBounds(bmin, bmax, node->bmin, node->bmax);
			const bool isLeafNode = node->i >= 0;
			
			if (isLeafNode && overlap)
			{
				dtPolyRef ref = base | (dtPolyRef)node->i;
				if (filter->passFilter(ref, tile, &tile->polys[node->i]) && passLinkFilter(tile, node->i))
				{
					if (n < maxPolys)
						polys[n++] = ref;
				}
			}
			
			if (overlap || isLeafNode)
				node++;
			else
			{
				const int escapeIndex = -node->i;
				node += escapeIndex;
			}
		}
		
		return n;
	}
	else
	{
		dtReal bmin[3], bmax[3];
		int n = 0;
		const dtPolyRef base = m_nav->getPolyRefBase(tile);
		for (int i = 0; i < tile->header->polyCount; ++i)
		{
			const dtPoly* p = &tile->polys[i];
			// Do not return off-mesh connection polygons.
			if (p->getType() != DT_POLYTYPE_GROUND)
				continue;
			// Must pass filter
			const dtPolyRef ref = base | (dtPolyRef)i;
			if (!filter->passFilter(ref, tile, p) || !passLinkFilter(tile, i))
				continue;
			// Calc polygon bounds.
			dtReal v[3]; //@UE
			tile->verts[p->verts[0]].toWorld(tile->header->bmin, v); //@UE
			dtVcopy(bmin, v);
			dtVcopy(bmax, v);
			for (int j = 1; j < p->vertCount; ++j)
			{
				tile->verts[p->verts[j]].toWorld(tile->header->bmin, v); //@UE
				dtVmin(bmin, v);
				dtVmax(bmax, v);
			}
			if (dtOverlapBounds(qmin,qmax, bmin,bmax))
			{
				if (n < maxPolys)
					polys[n++] = ref;
			}
		}
		return n;
	}
}

/// @par 
///
/// If no polygons are found, the function will return #DT_SUCCESS with a
/// @p polyCount of zero.
///
/// If @p polys is too small to hold the entire result set, then the array will 
/// be filled to capacity. The method of choosing which polygons from the 
/// full set are included in the partial result set is undefined.
///
dtStatus dtNavMeshQuery::queryPolygons(const dtReal* center, const dtReal* extents,
									   const dtQueryFilter* filter,
									   dtPolyRef* polys, int* polyCount, const int maxPolys) const
{
	dtAssert(m_nav);
	
	dtReal bmin[3], bmax[3];
	dtVsub(bmin, center, extents);
	dtVadd(bmax, center, extents);
	
	// Find tiles the query touches.
	int minx, miny, maxx, maxy;
	m_nav->calcTileLoc(bmin, &minx, &miny);
	m_nav->calcTileLoc(bmax, &maxx, &maxy);

	ReadTilesHelper TileArray;

	int n = 0;
	for (int y = miny; y <= maxy; ++y)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			int nneis = m_nav->getTileCountAt(x,y);
			const dtMeshTile** neis = (const dtMeshTile**)TileArray.PrepareArray(nneis);
			if (neis == nullptr)						//@UE
			{											//@UE
				return DT_FAILURE | DT_OUT_OF_MEMORY;	//@UE
			}											//@UE

			m_nav->getTilesAt(x, y, neis, nneis);
			for (int j = 0; j < nneis; ++j)
			{
				n += queryPolygonsInTile(neis[j], bmin, bmax, filter, polys+n, maxPolys-n);
				if (n >= maxPolys)
				{
					*polyCount = n;
					return DT_SUCCESS | DT_BUFFER_TOO_SMALL;
				}
			}
		}
	}
	*polyCount = n;
	
	return DT_SUCCESS;
}

/// @par
///
/// If the end polygon cannot be reached through the navigation graph,
/// the last polygon in the path will be the nearest the end polygon.
///
/// If the path array is to small to hold the full result, it will be filled as 
/// far as possible from the start polygon toward the end polygon.
///
/// The start and end positions are used to calculate traversal costs. 
/// (The y-values impact the result.)
///
dtStatus dtNavMeshQuery::findPath(dtPolyRef startRef, dtPolyRef endRef,
								  const dtReal* startPos, const dtReal* endPos,
								  const dtReal costLimit, const dtQueryFilter* filter, //@UE
								  dtQueryResult& result, dtReal* totalCost) const
{
	dtAssert(m_nav);
	dtAssert(m_nodePool);
	dtAssert(m_openList);
	
	m_queryNodes = 0;

	if (!startRef || (m_query.requireNavigableEndLocation && !endRef))		//@UE
		return DT_FAILURE | DT_INVALID_PARAM;
	
	// Validate input
	// endRef could be 0 if requireNavigableEndLocation is false, but we don't want it to reference a polygon that doesn't exist
	if (!m_nav->isValidPolyRef(startRef) || (!m_nav->isValidPolyRef(endRef) && (endRef || m_query.requireNavigableEndLocation)))	//@UE
		return DT_FAILURE | DT_INVALID_PARAM;
	
	if (startRef == endRef)
	{
		result.addItem(startRef, 0.0f, 0, 0);
		return DT_SUCCESS;
	}
	
	//@UE BEGIN
	const dtReal H_SCALE = filter->getModifiedHeuristicScale();
	const bool shouldIgnoreClosedNodes = filter->getShouldIgnoreClosedNodes();
	//@UE END

	m_nodePool->clear();
	m_openList->clear();
	
	dtNode* startNode = m_nodePool->getNode(startRef);
	dtVcopy(startNode->pos, startPos);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = dtVdist(startPos, endPos) * H_SCALE;
	startNode->id = startRef;
	startNode->flags = DT_NODE_OPEN;
	UE_RECAST_ASTAR_LOG(Display, TEXT("Start by pushing %lld"), startRef);
	m_openList->push(startNode);
	m_queryNodes++;

	dtNode* lastBestNode = startNode;
	dtReal lastBestNodeCost = startNode->total;
	
	dtStatus status = DT_SUCCESS;

	int loopCounter = 0;
	const int loopLimit = m_nodePool->getMaxRuntimeNodes() + 1;

	while (!m_openList->empty())
	{
		// Remove node from open list and put it in closed list.
		dtNode* bestNode = m_openList->pop();
		UE_RECAST_ASTAR_LOG(Display, TEXT("   Pop %lld"), bestNode->id);
		bestNode->flags &= ~DT_NODE_OPEN;
		bestNode->flags |= DT_NODE_CLOSED;
		
		// Reached the goal, stop searching.
		if (bestNode->id == endRef)
		{
			lastBestNode = bestNode;
			break;
		}
		
		loopCounter++;
		// failsafe for cycles in navigation graph resulting in infinite loop 
		if (loopCounter >= loopLimit * 4)
		{
			break;
		}

		// Get current poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef bestRef = bestNode->id;
		const dtMeshTile* bestTile = 0;
		const dtPoly* bestPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(bestRef, &bestTile, &bestPoly);
		
		// Get parent poly and tile.
		dtPolyRef parentRef = 0;
		const dtMeshTile* parentTile = 0;
		const dtPoly* parentPoly = 0;
		if (bestNode->pidx)
			parentRef = m_nodePool->getNodeAtIdx(bestNode->pidx)->id;
		if (parentRef)
			m_nav->getTileAndPolyByRefUnsafe(parentRef, &parentTile, &parentPoly);
		
		unsigned int i = bestPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(bestTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			
			// Skip invalid ids and do not expand back to where we came from.
			if (!neighbourRef || neighbourRef == parentRef
				//@UE BEGIN
				|| !filter->isValidLinkSide(link.side))
				//@UE END
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("      Filtered %lld from %lld, isValidLinkSide %i"), neighbourRef, bestRef, filter->isValidLinkSide(link.side));
				continue;
			}
			
			// Get neighbour poly and tile.
			// The API input has been cheked already, skip checking internal data.
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);			
			
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("      Filtered %lld from %lld, filter->isVirtual %i, passFilter %i, passLinkFilterByRef %i"),
					neighbourRef, bestRef, filter->getIsVirtual(),
					filter->passFilter(neighbourRef, neighbourTile, neighbourPoly), passLinkFilterByRef(neighbourTile, neighbourRef));
				continue;
			}

			dtNode* neighbourNode = m_nodePool->getNode(neighbourRef);
			if (!neighbourNode)
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("      Reach Limit %lld from %lld"), neighbourRef, bestRef);
				status |= DT_OUT_OF_NODES;
				continue;
			}
//@UE BEGIN
			else if (shouldIgnoreClosedNodes && (neighbourNode->flags & DT_NODE_CLOSED) != 0)
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("      Skipping closed %lld from %lld"), neighbourRef, bestRef);
				continue;
			}
//@UE END

			// Try to update node position for current edge to make paths more precise
			// Unless heuristic is not admissible (overestimates), in which case
			// we have to use constant values for given node to avoid creating cycles
			dtReal neiPos[3] = { 0.0f, 0.0f, 0.0f };
			if (H_SCALE <= 1.0f || neighbourNode->flags == 0)
			{
				getEdgeMidPoint(bestRef, bestPoly, bestTile,
					neighbourRef, neighbourPoly, neighbourTile,
					neiPos);
			}
			else
			{
				dtVcopy(neiPos, neighbourNode->pos);
			}

			// Calculate cost and heuristic.
			dtReal cost = 0;
			dtReal heuristic = 0;
			dtReal curCost = 0;

			// Special case for last node.
			if (neighbourRef != endRef)
			{
				curCost = filter->getCost(bestNode->pos, neiPos, parentRef, parentTile, parentPoly, bestRef, bestTile, bestPoly, neighbourRef, neighbourTile, neighbourPoly);
				cost = bestNode->cost + curCost;
				heuristic = dtVdist(neiPos, endPos)*H_SCALE;
			}
			else
			{
				const dtReal endCost = filter->getCost(neiPos, endPos, bestRef, bestTile, bestPoly, neighbourRef, neighbourTile, neighbourPoly, 0, 0, 0);
				curCost = filter->getCost(bestNode->pos, neiPos, parentRef, parentTile, parentPoly, bestRef, bestTile, bestPoly, neighbourRef, neighbourTile, neighbourPoly);
				cost = bestNode->cost + curCost + endCost;
				heuristic = 0;
			}

			const dtReal total = cost + heuristic;

			// The node is already in open list and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_OPEN) && total >= neighbourNode->total)
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("      Skipping new cost higher %lld from %lld cost %f total %f prev cost %f"), neighbourRef, bestRef, cost, total, neighbourNode->total);
				continue;
			}

			// The node is already visited and process, and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_CLOSED) && total >= neighbourNode->total)
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("      Skipping new cost higher %lld from %lld cost %f total %f prev cost %f"), neighbourRef, bestRef, cost, total, neighbourNode->total);
				continue;
			}

			// Cost of current link is DT_UNWALKABLE_POLY_COST, skip.
			if (curCost == DT_UNWALKABLE_POLY_COST)
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("      Skipping unwalkable poly %lld from %lld cost %f total %f prev cost %f"), neighbourRef, bestRef, cost, total, neighbourNode->total);
				continue;
			}

			if (total > costLimit) //@UE
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("      Skipping reach cost limit poly %lld from %lld cost %f total %f prev cost %f limit %f"), neighbourRef, bestRef, cost, total, neighbourNode->total, costLimit);
				continue;
			}

			// Add or update the node.
			neighbourNode->pidx = m_nodePool->getNodeIdx(bestNode);
			neighbourNode->id = neighbourRef;
			neighbourNode->flags = (neighbourNode->flags & ~DT_NODE_CLOSED);
			neighbourNode->cost = cost;
			neighbourNode->total = total;
			dtVcopy(neighbourNode->pos, neiPos);

			if (neighbourNode->flags & DT_NODE_OPEN)
			{
				// Already in open, update node location.
				m_openList->modify(neighbourNode);
				UE_RECAST_ASTAR_LOG(Display, TEXT("   Modifying %lld from %lld cost %f total %f"), neighbourRef, bestRef, cost, total);
			}
			else
			{
				// Put the node in open list.
				neighbourNode->flags |= DT_NODE_OPEN;
				m_openList->push(neighbourNode);
				m_queryNodes++;
				UE_RECAST_ASTAR_LOG(Display, TEXT("   Pushing %lld from %lld cost %f total %f"), neighbourRef, bestRef, cost, total);
			}
			
			// Update nearest node to target so far.
			if (heuristic < lastBestNodeCost)
			{
				UE_RECAST_ASTAR_LOG(Display, TEXT("New best path %lld from %lld new best heuristic %f prev best heuristic %f"), neighbourRef, bestRef, heuristic, lastBestNodeCost);
				lastBestNodeCost = heuristic;
				lastBestNode = neighbourNode;
			}
		}
	}
	
	if (lastBestNode->id != endRef)
		status |= DT_PARTIAL_RESULT;
	
	// Reverse the path.
	dtNode* prev = 0;
	dtNode* node = lastBestNode;
	int n = 1;
	do
	{
		dtNode* next = m_nodePool->getNodeAtIdx(node->pidx);
		node->pidx = m_nodePool->getNodeIdx(prev);
		prev = node;
		node = next;
	}
	while (node && ++n < loopLimit);
	
	if (n >= loopLimit)
	{
		return DT_FAILURE | DT_INVALID_CYCLE_PATH;
	}

	result.reserve(n);

	// Store path
	dtReal prevCost = 0.0f;
	node = prev;
	check(node);
	do
	{
		result.addItem(node->id, node->cost - prevCost, 0, 0);
		prevCost = node->cost;

		node = m_nodePool->getNodeAtIdx(node->pidx);
	}
	while (node && --n > 0);

	if (totalCost)
	{
		*totalCost = lastBestNode->total;
	}

	return status;
}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
dtStatus dtNavMeshQuery::testClusterPath(dtPolyRef startRef, dtPolyRef endRef) const
{
	const dtMeshTile* startTile = m_nav->getTileByRef(startRef);
	const dtMeshTile* endTile = m_nav->getTileByRef(endRef);
	const unsigned int startPolyIdx = m_nav->decodePolyIdPoly(startRef);
	const unsigned int endPolyIdx = m_nav->decodePolyIdPoly(endRef);

	m_queryNodes = 0;

	if (startTile == 0 || endTile == 0 ||
		startTile->polyClusters == 0 || endTile->polyClusters == 0 ||
		startPolyIdx >= (unsigned int)startTile->header->offMeshBase ||
		endPolyIdx >= (unsigned int)endTile->header->offMeshBase)
	{
		// this means most probably the hierarchical graph has not been build at all
		return DT_FAILURE | DT_INVALID_PARAM;
	}

	const unsigned int startIdx = startTile->polyClusters[startPolyIdx];
	const unsigned int endIdx = endTile->polyClusters[endPolyIdx];
	const dtCluster& startCluster = startTile->clusters[startIdx];
	const dtCluster& endCluster = endTile->clusters[endIdx];

	const dtClusterRef startCRef = m_nav->getClusterRefBase(startTile) | (dtClusterRef)startIdx;
	const dtClusterRef endCRef = m_nav->getClusterRefBase(endTile) | (dtClusterRef)endIdx;
	if (startCRef == endCRef)
	{
		return DT_SUCCESS;
	}

	m_nodePool->clear();
	m_openList->clear();

	dtNode* startNode = m_nodePool->getNode(startCRef);
	dtVcopy(startNode->pos, startCluster.center);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = dtVdist(startCluster.center, endCluster.center) * DEFAULT_HEURISTIC_SCALE;
	startNode->id = startCRef;
	startNode->flags = DT_NODE_OPEN;
	m_openList->push(startNode);
	m_queryNodes++;

	dtNode* lastBestNode = startNode;
	dtReal lastBestNodeCost = startNode->total;
	
	dtStatus status = DT_FAILURE;
	while (!m_openList->empty())
	{
		// Remove node from open list and put it in closed list.
		dtNode* bestNode = m_openList->pop();
		bestNode->flags &= ~DT_NODE_OPEN;
		bestNode->flags |= DT_NODE_CLOSED;

		// Reached the goal, stop searching.
		if (bestNode->id == endCRef)
		{
			lastBestNode = bestNode;
			break;
		}

		// Get current cluster
		const dtClusterRef bestRef = bestNode->id;
		const dtMeshTile* bestTile = m_nav->getTileByRef(bestRef);
		const unsigned int bestClusterIdx = m_nav->decodeClusterIdCluster(bestRef);
		const dtCluster* bestCluster = &bestTile->clusters[bestClusterIdx];

		// Get parent ref
		const dtClusterRef parentRef = (bestNode->pidx) ? m_nodePool->getNodeAtIdx(bestNode->pidx)->id : 0;

		// Iterate through links
		unsigned int i = bestCluster->firstLink;
		while (i != DT_NULL_LINK)
		{
			// don't update link, cost is not important
			const dtClusterLink& link = m_nav->getClusterLink(bestTile, i);
			i = link.next;

			const dtClusterRef& neighbourRef = link.ref;

			// do not expand back to where we came from.
			if (!neighbourRef || neighbourRef == parentRef)
				continue;

			// Check backtracking
			if ((link.flags & DT_CLINK_VALID_FWD) == 0)
				continue;

			// Get neighbour poly and tile.
			// The API input has been cheked already, skip checking internal data.
			const dtMeshTile* neighbourTile = m_nav->getTileByRef(neighbourRef);
			const dtCluster* neighbourCluster = &neighbourTile->clusters[m_nav->decodeClusterIdCluster(neighbourRef)];

			dtNode* neighbourNode = m_nodePool->getNode(neighbourRef);
			if (!neighbourNode)
			{
				status |= DT_OUT_OF_NODES;
				continue;
			}

			// If the node is visited the first time, calculate node position.
			if (neighbourNode->flags == 0)
			{
				dtVcopy(neighbourNode->pos, neighbourCluster->center);
			}

			// Calculate cost and heuristic.
			const dtReal cost = bestNode->cost;
			const dtReal heuristic = (neighbourRef != endCRef) ? dtVdist(neighbourNode->pos, endCluster.center)*DEFAULT_HEURISTIC_SCALE : 0.0f;
			const dtReal total = cost + heuristic;

			// The node is already in open list and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_OPEN) && total >= neighbourNode->total)
				continue;
			// The node is already visited and process, and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_CLOSED) && total >= neighbourNode->total)
				continue;

			// Add or update the node.
			neighbourNode->pidx = m_nodePool->getNodeIdx(bestNode);
			neighbourNode->id = neighbourRef;
			neighbourNode->flags = (neighbourNode->flags & ~DT_NODE_CLOSED);
			neighbourNode->cost = cost;
			neighbourNode->total = total;

			if (neighbourNode->flags & DT_NODE_OPEN)
			{
				// Already in open, update node location.
				m_openList->modify(neighbourNode);
			}
			else
			{
				// Put the node in open list.
				neighbourNode->flags |= DT_NODE_OPEN;
				m_openList->push(neighbourNode);
				m_queryNodes++;
			}

			// Update nearest node to target so far.
			if (heuristic < lastBestNodeCost)
			{
				lastBestNodeCost = heuristic;
				lastBestNode = neighbourNode;
			}
		}
	}

	if (lastBestNode->id == endCRef)
		status = DT_SUCCESS;

	return status;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

/// @par
///
/// @warning Calling any non-slice methods before calling finalizeSlicedFindPath() 
/// or finalizeSlicedFindPathPartial() may result in corrupted data!
///
/// The @p filter pointer is stored and used for the duration of the sliced
/// path query.
///
dtStatus dtNavMeshQuery::initSlicedFindPath(dtPolyRef startRef, dtPolyRef endRef,
											const dtReal* startPos, const dtReal* endPos, const dtReal costLimit, const bool requireNavigableEndLocation, //@UE 
											const dtQueryFilter* filter)
{
	dtAssert(m_nav);
	dtAssert(m_nodePool);
	dtAssert(m_openList);

	// Init path state.
	memset(&m_query, 0, sizeof(dtQueryData));
	m_query.status = DT_FAILURE;
	m_query.startRef = startRef;
	m_query.endRef = endRef;
	dtVcopy(m_query.startPos, startPos);
	dtVcopy(m_query.endPos, endPos);
	m_query.costLimit = costLimit; //@UE
	m_query.requireNavigableEndLocation = requireNavigableEndLocation; //@UE
	m_query.filter = filter;
	
	if (!startRef || (m_query.requireNavigableEndLocation && !endRef))	//@UE
		return DT_FAILURE | DT_INVALID_PARAM;
	
	// Validate input
	// endRef could be 0 if requireNavigableEndLocation is false, but we don't want it to reference a polygon that doesn't exist
	if (!m_nav->isValidPolyRef(startRef) || (!m_nav->isValidPolyRef(endRef) && (endRef || m_query.requireNavigableEndLocation)))	//@UE
		return DT_FAILURE | DT_INVALID_PARAM;

	if (startRef == endRef)
	{
		m_query.status = DT_SUCCESS;
		return DT_SUCCESS;
	}
	
	//@UE BEGIN
	const dtReal H_SCALE = filter->getModifiedHeuristicScale();
	//@UE END

	m_nodePool->clear();
	m_openList->clear();
	
	dtNode* startNode = m_nodePool->getNode(startRef);
	dtVcopy(startNode->pos, startPos);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = dtVdist(startPos, endPos) * H_SCALE;
	startNode->id = startRef;
	startNode->flags = DT_NODE_OPEN;
	m_openList->push(startNode);
	
	m_query.status = DT_IN_PROGRESS;
	m_query.lastBestNode = startNode;
	m_query.lastBestNodeCost = startNode->total;
	
	return m_query.status;
}
	
dtStatus dtNavMeshQuery::updateSlicedFindPath(const int maxIter, int* doneIters)
{
	if (!dtStatusInProgress(m_query.status))
		return m_query.status;

	// Make sure the request is still valid.
	// endRef could be 0 if requireNavigableEndLocation is false, but we don't want it to reference a polygon that doesn't exist
	if (!m_nav->isValidPolyRef(m_query.startRef) || (!m_nav->isValidPolyRef(m_query.endRef) && (m_query.endRef || m_query.requireNavigableEndLocation)))	//@UE
	{
		m_query.status = DT_FAILURE;
		return DT_FAILURE;
	}

	//@UE BEGIN
	const dtReal H_SCALE = m_query.filter->getModifiedHeuristicScale();
	const bool shouldIgnoreClosedNodes = m_query.filter->getShouldIgnoreClosedNodes();
	//@UE END
			
	int iter = 0;
	while (iter < maxIter && !m_openList->empty())
	{
		iter++;
		
		// Remove node from open list and put it in closed list.
		dtNode* bestNode = m_openList->pop();
		bestNode->flags &= ~DT_NODE_OPEN;
		bestNode->flags |= DT_NODE_CLOSED;
		
		// Reached the goal, stop searching.
		if (bestNode->id == m_query.endRef)
		{
			m_query.lastBestNode = bestNode;
			const dtStatus details = m_query.status & DT_STATUS_DETAIL_MASK;
			m_query.status = DT_SUCCESS | details;
			if (doneIters)
				*doneIters = iter;
			return m_query.status;
		}
		
		// Get current poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef bestRef = bestNode->id;
		const dtMeshTile* bestTile = 0;
		const dtPoly* bestPoly = 0;
		if (dtStatusFailed(m_nav->getTileAndPolyByRef(bestRef, &bestTile, &bestPoly)))
		{
			// The polygon has disappeared during the sliced query, fail.
			m_query.status = DT_FAILURE;
			if (doneIters)
				*doneIters = iter;
			return m_query.status;
		}
		
		// Get parent poly and tile.
		dtPolyRef parentRef = 0;
		const dtMeshTile* parentTile = 0;
		const dtPoly* parentPoly = 0;
		if (bestNode->pidx)
			parentRef = m_nodePool->getNodeAtIdx(bestNode->pidx)->id;
		if (parentRef)
		{
			if (dtStatusFailed(m_nav->getTileAndPolyByRef(parentRef, &parentTile, &parentPoly)))
			{
				// The polygon has disappeared during the sliced query, fail.
				m_query.status = DT_FAILURE;
				if (doneIters)
					*doneIters = iter;
				return m_query.status;
			}
		}
		
		unsigned int i = bestPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(bestTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			
			// Skip invalid ids and do not expand back to where we came from.
			if (!neighbourRef || neighbourRef == parentRef
//@UE BEGIN
				|| !m_query.filter->isValidLinkSide(link.side))
//@UE END
				continue;
			
			// Get neighbour poly and tile.
			// The API input has been cheked already, skip checking internal data.
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);			
			
			if (!m_query.filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
				continue;
			
			dtNode* neighbourNode = m_nodePool->getNode(neighbourRef);
			if (!neighbourNode)
			{
				m_query.status |= DT_OUT_OF_NODES;
				continue;
			}
//@UE BEGIN
			else if (shouldIgnoreClosedNodes && (neighbourNode->flags & DT_NODE_CLOSED) != 0)
			{
				continue;
			}
//@UE END
			
			// Always calculate correct position on neighbor's edge,
			// skipping to wrong edge may greatly change path cost
			// (if area cost differences are more than 5x default)
			dtReal neiPos[3] = { 0.0f, 0.0f, 0.0f };
			getEdgeMidPoint(bestRef, bestPoly, bestTile,
				neighbourRef, neighbourPoly, neighbourTile,
				neiPos);
			
			// Calculate cost and heuristic.
			dtReal cost = 0;
			dtReal heuristic = 0;
			dtReal curCost = 0; //@UE
			
			// Special case for last node.
			if (neighbourRef != m_query.endRef)
			{
				// Cost
				curCost = m_query.filter->getCost(bestNode->pos, neiPos,
					parentRef, parentTile, parentPoly,
					bestRef, bestTile, bestPoly,
					neighbourRef, neighbourTile, neighbourPoly);
				cost = bestNode->cost + curCost;
				heuristic = dtVdist(neiPos, m_query.endPos)*H_SCALE;
			}
			else
			{
				// Cost
				curCost = m_query.filter->getCost(bestNode->pos, neiPos,
												  parentRef, parentTile, parentPoly,
												  bestRef, bestTile, bestPoly,
												  neighbourRef, neighbourTile, neighbourPoly);
				const dtReal endCost = m_query.filter->getCost(neiPos, m_query.endPos,
												  bestRef, bestTile, bestPoly,
												  neighbourRef, neighbourTile, neighbourPoly,
												  0, 0, 0);
				
				cost = bestNode->cost + curCost + endCost;
				heuristic = 0;
			}
			
			const dtReal total = cost + heuristic;

			// The node is already in open list and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_OPEN) && total >= neighbourNode->total)
				continue;

			// The node is already visited and process, and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_CLOSED) && total >= neighbourNode->total)
				continue;

			// Cost of current link is DT_UNWALKABLE_POLY_COST, skip.
			if (curCost == DT_UNWALKABLE_POLY_COST) //@UE
				continue;

			if (total > m_query.costLimit) //@UE
				continue;
			
			// Add or update the node.
			neighbourNode->pidx = m_nodePool->getNodeIdx(bestNode);
			neighbourNode->id = neighbourRef;
			neighbourNode->flags = (neighbourNode->flags & ~DT_NODE_CLOSED);
			neighbourNode->cost = cost;
			neighbourNode->total = total;
			dtVcopy(neighbourNode->pos, neiPos);
			
			if (neighbourNode->flags & DT_NODE_OPEN)
			{
				// Already in open, update node location.
				m_openList->modify(neighbourNode);
			}
			else
			{
				// Put the node in open list.
				neighbourNode->flags |= DT_NODE_OPEN;
				m_openList->push(neighbourNode);
			}
			
			// Update nearest node to target so far.
			if (heuristic < m_query.lastBestNodeCost)
			{
				m_query.lastBestNodeCost = heuristic;
				m_query.lastBestNode = neighbourNode;
			}
		}
	}
	
	// Exhausted all nodes, but could not find path.
	if (m_openList->empty())
	{
		const dtStatus details = m_query.status & DT_STATUS_DETAIL_MASK;
		m_query.status = DT_SUCCESS | details;
	}

	if (doneIters)
		*doneIters = iter;

	return m_query.status;
}

dtStatus dtNavMeshQuery::finalizeSlicedFindPath(dtPolyRef* path, int* pathCount, const int maxPath)
{
	*pathCount = 0;
	
	if (dtStatusFailed(m_query.status))
	{
		// Reset query.
		memset(&m_query, 0, sizeof(dtQueryData));
		return DT_FAILURE;
	}

	int n = 0;

	if (m_query.startRef == m_query.endRef)
	{
		// Special case: the search starts and ends at same poly.
		path[n++] = m_query.startRef;
	}
	else
	{
		// Reverse the path.
		dtAssert(m_query.lastBestNode);
		
		if (m_query.lastBestNode->id != m_query.endRef)
			m_query.status |= DT_PARTIAL_RESULT;
		
		dtNode* prev = 0;
		dtNode* node = m_query.lastBestNode;
		do
		{
			dtNode* next = m_nodePool->getNodeAtIdx(node->pidx);
			node->pidx = m_nodePool->getNodeIdx(prev);
			prev = node;
			node = next;
		}
		while (node);
		
		// Store path
		node = prev;
		check(node);
		do
		{
			path[n++] = node->id;
			if (n >= maxPath)
			{
				m_query.status |= DT_BUFFER_TOO_SMALL;
				break;
			}
			node = m_nodePool->getNodeAtIdx(node->pidx);
		}
		while (node);
	}
	
	const dtStatus details = m_query.status & DT_STATUS_DETAIL_MASK;

	// Reset query.
	memset(&m_query, 0, sizeof(dtQueryData));
	
	*pathCount = n;
	
	return DT_SUCCESS | details;
}

dtStatus dtNavMeshQuery::finalizeSlicedFindPathPartial(const dtPolyRef* existing, const int existingSize,
													   dtPolyRef* path, int* pathCount, const int maxPath)
{
	*pathCount = 0;
	
	if (existingSize == 0)
	{
		return DT_FAILURE;
	}
	
	if (dtStatusFailed(m_query.status))
	{
		// Reset query.
		memset(&m_query, 0, sizeof(dtQueryData));
		return DT_FAILURE;
	}
	
	int n = 0;
	
	if (m_query.startRef == m_query.endRef)
	{
		// Special case: the search starts and ends at same poly.
		path[n++] = m_query.startRef;
	}
	else
	{
		// Find furthest existing node that was visited.
		dtNode* prev = 0;
		dtNode* node = 0;
		for (int i = existingSize-1; i >= 0; --i)
		{
			node = m_nodePool->findNode(existing[i]);
			if (node)
				break;
		}
		
		if (!node)
		{
			m_query.status |= DT_PARTIAL_RESULT;
			dtAssert(m_query.lastBestNode);
			node = m_query.lastBestNode;
		}
		
		// Reverse the path.
		do
		{
			dtNode* next = m_nodePool->getNodeAtIdx(node->pidx);
			node->pidx = m_nodePool->getNodeIdx(prev);
			prev = node;
			node = next;
		}
		while (node);
		
		// Store path
		node = prev;
		check(node);
		do
		{
			path[n++] = node->id;
			if (n >= maxPath)
			{
				m_query.status |= DT_BUFFER_TOO_SMALL;
				break;
			}
			node = m_nodePool->getNodeAtIdx(node->pidx);
		}
		while (node);
	}
	
	const dtStatus details = m_query.status & DT_STATUS_DETAIL_MASK;

	// Reset query.
	memset(&m_query, 0, sizeof(dtQueryData));
	
	*pathCount = n;
	
	return DT_SUCCESS | details;
}


dtStatus dtNavMeshQuery::appendVertex(const dtReal* pos, const unsigned char flags, const dtPolyRef ref,
									  dtQueryResult& result, const bool bOverrideIdenticalPosition /*= true */) const
{
	if (bOverrideIdenticalPosition && result.size() > 0 && dtVequal(result.getPos(result.size()-1), pos))
	{
		// The vertices are equal, update flags and poly.
		result.setFlag(result.size() - 1, flags);
		result.setRef(result.size() - 1, ref);
	}
	else
	{
		result.addItem(ref, 0, pos, flags);
		
		if (flags == DT_STRAIGHTPATH_END)
		{
			return DT_SUCCESS;
		}
	}
	return DT_IN_PROGRESS;
}

dtStatus dtNavMeshQuery::appendPortals(const int startIdx, const int endIdx, const dtReal* endPos, const dtPolyRef* path,
									   dtQueryResult& result, const int options) const
{
	dtReal startPos[3];
	result.getPos(result.size() - 1, startPos);

	// Append or update last vertex
	dtStatus stat = 0;
	for (int i = startIdx; i < endIdx; i++)
	{
		// Calculate portal
		const dtPolyRef from = path[i];
		const dtMeshTile* fromTile = 0;
		const dtPoly* fromPoly = 0;
		if (dtStatusFailed(m_nav->getTileAndPolyByRef(from, &fromTile, &fromPoly)))
			return DT_FAILURE | DT_INVALID_PARAM;
		
		const dtPolyRef to = path[i+1];
		const dtMeshTile* toTile = 0;
		const dtPoly* toPoly = 0;
		if (dtStatusFailed(m_nav->getTileAndPolyByRef(to, &toTile, &toPoly)))
			return DT_FAILURE | DT_INVALID_PARAM;
		
		dtReal left[3], right[3];
		if (dtStatusFailed(getPortalPoints(from, fromPoly, fromTile, to, toPoly, toTile, left, right)))
			break;
	
		if (options & DT_STRAIGHTPATH_AREA_CROSSINGS)
		{
			// Skip intersection if only area crossings are requested.
			if (fromPoly->getArea() == toPoly->getArea())
				continue;
		}
		
		// Append intersection
		dtReal s,t;
		if (!dtIntersectSegSeg2D(startPos, endPos, left, right, s, t))
		{
			// failsafe for vertical navlinks: if left and right are the same and either start or end matches, append intersection
			if (dtVequal(left, right) && (dtVequal(left, startPos) || dtVequal(left, endPos)))
			{
				// valid intersection, initialize interp value
				t = 0.0f;
			}
			else
			{
				continue;
			}
		}

		dtReal pt[3];
		dtVlerp(pt, left,right, t);

		unsigned char flags = 0;
		if (toPoly->getType() != DT_POLYTYPE_GROUND)
			flags = DT_STRAIGHTPATH_OFFMESH_CONNECTION;

		stat = appendVertex(pt, flags, path[i + 1], result);
		if (stat != DT_IN_PROGRESS)
			return stat;
	}
	return DT_IN_PROGRESS;
}

#if WITH_NAVMESH_SEGMENT_LINKS
enum class ESegmentLinkPortalEntryTestState : uint8
{
	None,
	Requested,
	Active
};
#endif // WITH_NAVMESH_SEGMENT_LINKS

/// @par
/// 
/// This method peforms what is often called 'string pulling'.
///
/// The start position is clamped to the first polygon in the path, and the 
/// end position is clamped to the last. So the start and end positions should 
/// normally be within or very near the first and last polygons respectively.
///
/// The returned polygon references represent the reference id of the polygon 
/// that is entered at the associated path position. The reference id associated 
/// with the end point will always be zero.  This allows, for example, matching 
/// off-mesh link points to their representative polygons.
///
/// If the provided result buffers are too small for the entire result set, 
/// they will be filled as far as possible from the start toward the end 
/// position.
///
dtStatus dtNavMeshQuery::findStraightPath(const dtReal* startPos, const dtReal* endPos,
										  const dtPolyRef* path, const int pathSize,
										  dtQueryResult& result, const int options) const
{
	dtAssert(m_nav);
	
	if (!path[0])
		return DT_FAILURE | DT_INVALID_PARAM;
	
	dtStatus stat = 0;
	
	// TODO: Should this be callers responsibility?
	dtReal closestStartPos[3];
	if (dtStatusFailed(closestPointOnPolyBoundary(path[0], startPos, closestStartPos)))
		return DT_FAILURE | DT_INVALID_PARAM;

	dtReal closestEndPos[3];
	if (dtStatusFailed(closestPointOnPolyBoundary(path[pathSize-1], endPos, closestEndPos)))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	// Add start point.
	stat = appendVertex(closestStartPos, DT_STRAIGHTPATH_START, path[0], result);
	if (stat != DT_IN_PROGRESS)
		return stat;
	
	if (pathSize > 1)
	{
		dtReal portalApex[3], portalLeft[3], portalRight[3];
		dtVcopy(portalApex, closestStartPos);
		dtVcopy(portalLeft, portalApex);
		dtVcopy(portalRight, portalApex);
		int apexIndex = 0;
		int leftIndex = 0;
		int rightIndex = 0;
		
		unsigned char leftPolyType = 0;
		unsigned char rightPolyType = 0;
		dtReal segt = 0.0f;
		bool segSwapped = false;

		dtPolyRef leftPolyRef = path[0];
		dtPolyRef rightPolyRef = path[0];

#if WITH_NAVMESH_SEGMENT_LINKS
		// Store the entry point to test when entering a segment link portal.
		dtReal segmentLinkEntryPoint[3] = { 0., 0., 0. };

		// Store the current state of the segment link portal entry test.
		ESegmentLinkPortalEntryTestState segmentLinkPortalEntryTestState = ESegmentLinkPortalEntryTestState::None;
#endif // WITH_NAVMESH_SEGMENT_LINKS
		
		for (int i = 0; i < pathSize; ++i)
		{
			dtReal left[3], right[3];
			unsigned char fromType, toType;
			
			if (i+1 < pathSize)
			{
				// Next portal.
				if (dtStatusFailed(getPortalPoints(path[i], path[i+1], left, right, fromType, toType)))
				{
					// Failed to get portal points, in practice this means that path[i+1] is invalid polygon.
					// Clamp the end point to path[i], and return the path so far.
					
					if (dtStatusFailed(closestPointOnPolyBoundary(path[i], endPos, closestEndPos)))
					{
						// This should only happen when the first polygon is invalid.
						return DT_FAILURE | DT_INVALID_PARAM;
					}

					// Apeend portals along the current straight path segment.
					if (options & (DT_STRAIGHTPATH_AREA_CROSSINGS | DT_STRAIGHTPATH_ALL_CROSSINGS))
					{
						stat = appendPortals(apexIndex, i, closestEndPos, path, result, options);
					}

					stat = appendVertex(closestEndPos, 0, path[i], result);
					
					return DT_SUCCESS | DT_PARTIAL_RESULT;
				}

				// If starting really close the portal, advance.
				if (i == 0 && toType == DT_POLYTYPE_GROUND)
				{
					dtReal t;
					if (dtDistancePtSegSqr2D(portalApex, left, right, t) < dtSqr(0.001f))
						continue;
				}
			}
			else
			{
				// End of the path.
				dtVcopy(left, closestEndPos);
				dtVcopy(right, closestEndPos);
				
				fromType = toType = DT_POLYTYPE_GROUND;
			}
			
			//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
			// Lock moving through segment off-mesh connections
			if (fromType == DT_POLYTYPE_OFFMESH_SEGMENT)
			{
				// Use the shortest path to exit the segment link, so just find the point on the exit edge
				// that is closest to the point used on the entrance edge.
				dtDistancePtSegSqr2D(segmentLinkEntryPoint, left, right, segt);

				dtReal lockedPortal[3];
				dtVlerp(lockedPortal, left, right, segt);
				dtVcopy(left, lockedPortal);
				dtVcopy(right, lockedPortal);
			}

			// The value of seqSwapped should not be calculated if this iteration is occurring due to a segment link portal entry test.
			// In this case, the value of seqSwapped was previously calculated during the previous iteration.
			if (segmentLinkPortalEntryTestState == ESegmentLinkPortalEntryTestState::None)
			{
				segSwapped = false;
				if (toType == DT_POLYTYPE_OFFMESH_SEGMENT && i != apexIndex)
				{
					dtReal mid0[3], mid1[3];
					dtVadd(mid0, portalLeft, portalRight);
					dtVscale(mid0, mid0, 0.5f);
					dtVadd(mid1, left, right);
					dtVscale(mid1, mid1, 0.5f);
					dtReal dirm[3], dir0[3], dir1[3];
					dtVsub(dirm, mid1, mid0);
					dtVsub(dir0, portalLeft, mid0);
					dtVsub(dir1, left, mid1);
					const dtReal c0 = dtVperp2D(dirm, dir0);
					const dtReal c1 = dtVperp2D(dirm, dir1);
					segSwapped = ((c0 > 0.f) && (c1 < 0.f)) || ((c0 < 0.f) && (c1 > 0.f));
				}
			}

			if (segSwapped)
			{
				dtReal tmp[3];
				dtVcopy(tmp, left);
				dtVcopy(left, right);
				dtVcopy(right, tmp);
			}

			// Handle requested and active segment link portal entry tests.
			if (toType == DT_POLYTYPE_OFFMESH_SEGMENT && segmentLinkPortalEntryTestState == ESegmentLinkPortalEntryTestState::Requested)
			{
				// Lock left and right to test the single segment link entry point.
				dtVcopy(left, segmentLinkEntryPoint);
				dtVcopy(right, segmentLinkEntryPoint);

				// The segment link entry point check is now active.
				segmentLinkPortalEntryTestState = ESegmentLinkPortalEntryTestState::Active;
			}
			else
			{
				// Segment link entry point check is not active for this iteration.
				segmentLinkPortalEntryTestState = ESegmentLinkPortalEntryTestState::None;
			}
#endif // WITH_NAVMESH_SEGMENT_LINKS
//@UE END

			// Right vertex.
			if (dtTriArea2D(portalApex, portalRight, right) <= 0.0f)
			{
				if (dtVequal(portalApex, portalRight) || dtTriArea2D(portalApex, portalLeft, right) > 0.0f)
				{
					dtVcopy(portalRight, right);
					rightPolyRef = (i+1 < pathSize) ? path[i+1] : 0;
					rightPolyType = toType;
					rightIndex = i;
				}
				else
				{
					// Append portals along the current straight path segment.
					if (options & (DT_STRAIGHTPATH_AREA_CROSSINGS | DT_STRAIGHTPATH_ALL_CROSSINGS))
					{
						stat = appendPortals(apexIndex, leftIndex, portalLeft, path, result, options);
						if (stat != DT_IN_PROGRESS)
							return stat;					
					}
				
					dtVcopy(portalApex, portalLeft);
					apexIndex = leftIndex;
					
					unsigned char flags = 0;
					if (!leftPolyRef)
						flags = DT_STRAIGHTPATH_END;
					else if (leftPolyType != DT_POLYTYPE_GROUND)
						flags = DT_STRAIGHTPATH_OFFMESH_CONNECTION;
					dtPolyRef ref = leftPolyRef;
					
					// Append or update vertex
					stat = appendVertex(portalApex, flags, ref, result);
					if (stat != DT_IN_PROGRESS)
						return stat;
					
					dtVcopy(portalLeft, portalApex);
					dtVcopy(portalRight, portalApex);
					leftIndex = apexIndex;
					rightIndex = apexIndex;

					//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
					// The value of segt has already been calculated in the previous iteration when a segment link portal entry check is active.
					if (toType == DT_POLYTYPE_OFFMESH_SEGMENT && segmentLinkPortalEntryTestState == ESegmentLinkPortalEntryTestState::None)
						dtDistancePtSegSqr2D(portalApex, left, right, segt);
#endif // WITH_NAVMESH_SEGMENT_LINKS
					//@UE END

					// Restart
					i = apexIndex;
					
					continue;
				}
			}
			
			// Left vertex.
			if (dtTriArea2D(portalApex, portalLeft, left) >= 0.0f)
			{
				if (dtVequal(portalApex, portalLeft) || dtTriArea2D(portalApex, portalRight, left) < 0.0f)
				{
					dtVcopy(portalLeft, left);
					leftPolyRef = (i+1 < pathSize) ? path[i+1] : 0;
					leftPolyType = toType;
					leftIndex = i;
				}
				else
				{
					// Append portals along the current straight path segment.
					if (options & (DT_STRAIGHTPATH_AREA_CROSSINGS | DT_STRAIGHTPATH_ALL_CROSSINGS))
					{
						stat = appendPortals(apexIndex, rightIndex, portalRight, path, result, options);
						if (stat != DT_IN_PROGRESS)
							return stat;
					}

					dtVcopy(portalApex, portalRight);
					apexIndex = rightIndex;
					
					unsigned char flags = 0;
					if (!rightPolyRef)
						flags = DT_STRAIGHTPATH_END;
					else if (rightPolyType != DT_POLYTYPE_GROUND)
						flags = DT_STRAIGHTPATH_OFFMESH_CONNECTION;
					dtPolyRef ref = rightPolyRef;

					// Append or update vertex
					stat = appendVertex(portalApex, flags, ref, result);
					if (stat != DT_IN_PROGRESS)
						return stat;
					
					dtVcopy(portalLeft, portalApex);
					dtVcopy(portalRight, portalApex);
					leftIndex = apexIndex;
					rightIndex = apexIndex;	

					//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
					// The value of segt has already been calculated in the previous iteration when a segment link portal entry check is active.
					if (toType == DT_POLYTYPE_OFFMESH_SEGMENT && segmentLinkPortalEntryTestState == ESegmentLinkPortalEntryTestState::None)
						dtDistancePtSegSqr2D(portalApex, left, right, segt);
#endif // WITH_NAVMESH_SEGMENT_LINKS
					//@UE END

					// Restart
					i = apexIndex;
					
					continue;
				}
			}

			//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
			// Handle entering off-mesh segments
			if (toType == DT_POLYTYPE_OFFMESH_SEGMENT)
			{
				// Check if the segment link entry point needs to be tested against the current portal apex.
				if (segmentLinkPortalEntryTestState == ESegmentLinkPortalEntryTestState::None)
				{
					// Segment link portal entry test is not needed when the link is being entered directly from
					// the current apex because there is no current funnel to test against. There is guaranteed
					// to be a clear path to the segment link portal entry point.
					if (i == apexIndex)
					{
						dtDistancePtSegSqr2D(portalApex, left, right, segt);
						dtVlerp(portalApex, left, right, segt);

						// segmentLinkEntryPoint still needs to be set to determine the exit point on the other edge of the segment link.
						dtVcopy(segmentLinkEntryPoint, portalApex);
					}
					else
					{
						// Calculate the value of segt to use during the segment link portal entry point test.
						dtDistancePtSegSqr2D(portalApex, left, right, segt);

						// Set the test point for segment link entry instead of portalApex directly.
						// This point will be used as the new portal apex if it passes the funnel algorithm.
						dtVlerp(segmentLinkEntryPoint, left, right, segt);

						// Set up the next iteration for the segment link portal test. 
						segmentLinkPortalEntryTestState = ESegmentLinkPortalEntryTestState::Requested;

						// The current iteration is restarted to perform the segment link portal test.
						--i;
						continue;
					}
				}

				stat = appendVertex(portalApex, DT_STRAIGHTPATH_OFFMESH_CONNECTION, path[i + 1], result);
				if (stat != DT_IN_PROGRESS)
					return stat;

				dtVcopy(portalLeft, portalApex);
				dtVcopy(portalRight, portalApex);
				leftIndex = i;
				rightIndex = i;
			}
#endif // WITH_NAVMESH_SEGMENT_LINKS
			//@UE END
		}

		// Append portals along the current straight path segment.
		if (options & (DT_STRAIGHTPATH_AREA_CROSSINGS | DT_STRAIGHTPATH_ALL_CROSSINGS))
		{
			stat = appendPortals(apexIndex, pathSize - 1, closestEndPos, path, result, options);
			if (stat != DT_IN_PROGRESS)
				return stat;
		}
	}

	stat = appendVertex(closestEndPos, DT_STRAIGHTPATH_END, 0, result);
	return DT_SUCCESS;
}

/// @par
///
/// This method is optimized for small delta movement and a small number of 
/// polygons. If used for too great a distance, the result set will form an 
/// incomplete path.
///
/// @p resultPos will equal the @p endPos if the end is reached. 
/// Otherwise the closest reachable position will be returned.
/// 
/// @p resultPos is not projected onto the surface of the navigation 
/// mesh. Use #getPolyHeight if this is needed.
///
/// This method treats the end position in the same manner as 
/// the #raycast method. (As a 2D point.) See that method's documentation 
/// for details.
/// 
/// If the @p visited array is too small to hold the entire result set, it will 
/// be filled as far as possible from the start position toward the end 
/// position.
///
dtStatus dtNavMeshQuery::moveAlongSurface(dtPolyRef startRef, const dtReal* startPos, const dtReal* endPos,
										  const dtQueryFilter* filter,
										  dtReal* resultPos, dtPolyRef* visited, int* visitedCount, const int maxVisitedSize) const
{
	dtAssert(m_nav);
	dtAssert(m_tinyNodePool);

	*visitedCount = 0;
	
	// Validate input
	if (!startRef)
		return DT_FAILURE | DT_INVALID_PARAM;
	if (!m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	dtStatus status = DT_SUCCESS;
	
	static const int MAX_STACK = 48;
	dtNode* stack[MAX_STACK];
	int nstack = 0;
	
	m_tinyNodePool->clear();
	
	dtNode* startNode = m_tinyNodePool->getNode(startRef);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_CLOSED;
	stack[nstack++] = startNode;
	
	dtReal bestPos[3];
	dtReal bestDist = DT_REAL_MAX;
	dtNode* bestNode = startNode;
	dtVcopy(bestPos, startPos);
	
	// Search constraints
	dtReal searchPos[3], searchRadSqr;
	dtVlerp(searchPos, startPos, endPos, 0.5f);
	searchRadSqr = dtSqr(dtVdist(startPos, endPos)/2.0f + 0.001f);
	
	dtReal verts[DT_VERTS_PER_POLYGON*3];
	
	while (nstack)
	{
		// Pop front.
		dtNode* curNode = stack[0];
		for (int i = 0; i < nstack-1; ++i)
			stack[i] = stack[i+1];
		nstack--;
		
		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef curRef = curNode->id;
		const dtMeshTile* curTile = 0;
		const dtPoly* curPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(curRef, &curTile, &curPoly);			
		
		// Collect vertices.
		const int nverts = curPoly->vertCount;
		for (int i = 0; i < nverts; ++i)
			curTile->verts[curPoly->verts[i]].toWorld(curTile->header->bmin, &verts[i*3]); //@UE
		
		// If target is inside the poly, stop search.
		if (dtPointInPolygon(endPos, verts, nverts))
		{
			bestNode = curNode;
			dtVcopy(bestPos, endPos);
			break;
		}
		
		// Find wall edges and find nearest point inside the walls.
		for (int i = 0, j = (int)curPoly->vertCount-1; i < (int)curPoly->vertCount; j = i++)
		{
			// Find links to neighbours.
			static const int MAX_NEIS = 8;
			int nneis = 0;
			dtPolyRef neis[MAX_NEIS];
			
			if (curPoly->neis[j] & DT_EXT_LINK)
			{
				// Tile border.
				unsigned int k = curPoly->firstLink;
				while (k != DT_NULL_LINK)
				{
					const dtLink& link = m_nav->getLink(curTile, k);
					k = link.next;

					if (link.edge == j)
					{
						if (link.ref != 0)
						{
							const dtMeshTile* neiTile = 0;
							const dtPoly* neiPoly = 0;
							m_nav->getTileAndPolyByRefUnsafe(link.ref, &neiTile, &neiPoly);
							if (filter->passFilter(link.ref, neiTile, neiPoly) && passLinkFilterByRef(neiTile, link.ref))
							{
								if (nneis < MAX_NEIS)
									neis[nneis++] = link.ref;
							}
						}
					}
				}
			}
			else if (curPoly->neis[j])
			{
				const unsigned int idx = (unsigned int)(curPoly->neis[j]-1);
				const dtPolyRef ref = m_nav->getPolyRefBase(curTile) | idx;
				if (filter->passFilter(ref, curTile, &curTile->polys[idx]) && passLinkFilter(curTile, idx))
				{
					// Internal edge, encode id.
					neis[nneis++] = ref;
				}
			}
			
			if (!nneis)
			{
				// Wall edge, calc distance.
				const dtReal* vj = &verts[j*3];
				const dtReal* vi = &verts[i*3];
				dtReal tseg;
				const dtReal distSqr = dtDistancePtSegSqr2D(endPos, vj, vi, tseg);
				if (distSqr < bestDist)
				{
                    // Update nearest distance.
					dtVlerp(bestPos, vj,vi, tseg);
					bestDist = distSqr;
					bestNode = curNode;
				}
			}
			else
			{
				for (int k = 0; k < nneis; ++k)
				{
					// Skip if no node can be allocated.
					dtNode* neighbourNode = m_tinyNodePool->getNode(neis[k]);
					if (!neighbourNode)
						continue;
					// Skip if already visited.
					if (neighbourNode->flags & DT_NODE_CLOSED)
						continue;
					
					// Skip the link if it is too far from search constraint.
					// TODO: Maybe should use getPortalPoints(), but this one is way faster.
					const dtReal* vj = &verts[j*3];
					const dtReal* vi = &verts[i*3];
					dtReal tseg;
					dtReal distSqr = dtDistancePtSegSqr2D(searchPos, vj, vi, tseg);
					if (distSqr > searchRadSqr)
						continue;
					
					// Mark as the node as visited and push to queue.
					if (nstack < MAX_STACK)
					{
						neighbourNode->pidx = m_tinyNodePool->getNodeIdx(curNode);
						neighbourNode->flags |= DT_NODE_CLOSED;
						stack[nstack++] = neighbourNode;
					}
				}
			}
		}
	}
	
	int n = 0;
	if (bestNode)
	{
		// Reverse the path.
		dtNode* prev = 0;
		dtNode* node = bestNode;
		do
		{
			dtNode* next = m_tinyNodePool->getNodeAtIdx(node->pidx);
			node->pidx = m_tinyNodePool->getNodeIdx(prev);
			prev = node;
			node = next;
		}
		while (node);
		
		// Store result
		node = prev;
		check(node);
		do
		{
			visited[n++] = node->id;
			if (n >= maxVisitedSize)
			{
				status |= DT_BUFFER_TOO_SMALL;
				break;
			}
			node = m_tinyNodePool->getNodeAtIdx(node->pidx);
		}
		while (node);
	}
	
	dtVcopy(resultPos, bestPos);
	
	*visitedCount = n;
	
	if (n == 0)
	{
		status |= DT_FAILURE;
	}

	return status;
}


dtStatus dtNavMeshQuery::getPortalPoints(dtPolyRef from, dtPolyRef to, dtReal* left, dtReal* right,
										 unsigned char& fromType, unsigned char& toType) const
{
	dtAssert(m_nav);
	
	const dtMeshTile* fromTile = 0;
	const dtPoly* fromPoly = 0;
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(from, &fromTile, &fromPoly)))
		return DT_FAILURE | DT_INVALID_PARAM;
	fromType = fromPoly->getType();

	const dtMeshTile* toTile = 0;
	const dtPoly* toPoly = 0;
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(to, &toTile, &toPoly)))
		return DT_FAILURE | DT_INVALID_PARAM;
	toType = toPoly->getType();
		
	return getPortalPoints(from, fromPoly, fromTile, to, toPoly, toTile, left, right);
}

// Returns portal points between two polygons.
dtStatus dtNavMeshQuery::getPortalPoints(dtPolyRef from, const dtPoly* fromPoly, const dtMeshTile* fromTile,
										 dtPolyRef to, const dtPoly* toPoly, const dtMeshTile* toTile,
										 dtReal* left, dtReal* right) const
{
	// Find the link that points to the 'to' polygon.
	const dtLink* link = 0;
	unsigned int linkIndex = fromPoly->firstLink;
	while (linkIndex != DT_NULL_LINK)
	{
		const dtLink& testLink = m_nav->getLink(fromTile, linkIndex);
		linkIndex = testLink.next;
		
		if (testLink.ref == to)
		{
			link = &testLink;
			break;
		}
	}
	if (!link)
		return DT_FAILURE | DT_INVALID_PARAM;
	
	// Handle off-mesh connections.
	if (fromPoly->getType() == DT_POLYTYPE_OFFMESH_POINT)
	{
		// Find link that points to first vertex.
		unsigned int i = fromPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& testLink = m_nav->getLink(fromTile, i);
			i = testLink.next;

			if (testLink.ref == to)
			{
				const int v = testLink.edge;
				fromTile->verts[fromPoly->verts[v]].toWorld(fromTile->header->bmin, left); //@UE
				fromTile->verts[fromPoly->verts[v]].toWorld(fromTile->header->bmin, right); //@UE
				return DT_SUCCESS;
			}
		}
		return DT_FAILURE | DT_INVALID_PARAM;
	}
	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	else if (fromPoly->getType() == DT_POLYTYPE_OFFMESH_SEGMENT)
	{
		// Find link that points to first vertex.
		unsigned int i = fromPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& testLink = m_nav->getLink(fromTile, i);
			i = testLink.next;

			if (testLink.ref == to)
			{
				const int v = testLink.edge * 2;
				fromTile->verts[fromPoly->verts[v+0]].toWorld(fromTile->header->bmin, left); //@UE
				fromTile->verts[fromPoly->verts[v+1]].toWorld(fromTile->header->bmin, right); //@UE
				return DT_SUCCESS;
			}
		}

		return DT_FAILURE | DT_INVALID_PARAM;
	}
	//@UE END
#endif // WITH_NAVMESH_SEGMENT_LINKS

	if (toPoly->getType() == DT_POLYTYPE_OFFMESH_POINT)
	{
		unsigned int i = toPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& testLink = m_nav->getLink(toTile, i);
			i = testLink.next;

			if (testLink.ref == from)
			{
				const int v = testLink.edge;
				toTile->verts[toPoly->verts[v]].toWorld(toTile->header->bmin, left); //@UE
				toTile->verts[toPoly->verts[v]].toWorld(toTile->header->bmin, right); //@UE
				return DT_SUCCESS;
			}
		}
		return DT_FAILURE | DT_INVALID_PARAM;
	}
	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	else if (toPoly->getType() == DT_POLYTYPE_OFFMESH_SEGMENT)
	{
		unsigned int i = toPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& testLink = m_nav->getLink(toTile, i);
			i = testLink.next;

			if (testLink.ref == from)
			{
				const int v = testLink.edge * 2;
				toTile->verts[toPoly->verts[v+0]].toWorld(toTile->header->bmin, left); //@UE
				toTile->verts[toPoly->verts[v+1]].toWorld(toTile->header->bmin, right); //@UE
				return DT_SUCCESS;
			}
		}
		return DT_FAILURE | DT_INVALID_PARAM;
	}
#endif // WITH_NAVMESH_SEGMENT_LINKS
	//@UE END

	// Find portal vertices.
	const int v0 = fromPoly->verts[link->edge];
	const int v1 = fromPoly->verts[(link->edge+1) % (int)fromPoly->vertCount];
	dtReal v0pos[3], v1pos[3]; //@UE
	fromTile->verts[v0].toWorld(fromTile->header->bmin, v0pos); //@UE
	fromTile->verts[v1].toWorld(fromTile->header->bmin, v1pos); //@UE
	dtVcopy(left, v0pos); //@UE
	dtVcopy(right, v1pos); //@UE

	// If the link is at tile boundary, dtClamp the vertices to
	// the link width.
//@UE BEGIN
	if ((link->side & DT_CONNECTION_INTERNAL) == 0)
//@UE END
	{
		// Unpack portal limits.
		if (link->bmin != 0 || link->bmax != 255)
		{
			const dtReal s = dtReal(1.)/255.0f;
			const dtReal tmin = link->bmin*s;
			const dtReal tmax = link->bmax*s;
			dtVlerp(left, v0pos, v1pos, tmin); //@UE
			dtVlerp(right, v0pos, v1pos, tmax); //@UE
		}
	}
	
	return DT_SUCCESS;
}

// Returns edge mid point between two polygons.
dtStatus dtNavMeshQuery::getEdgeMidPoint(dtPolyRef from, dtPolyRef to, dtReal* mid) const
{
	dtReal left[3], right[3];
	unsigned char fromType, toType;
	if (dtStatusFailed(getPortalPoints(from, to, left,right, fromType, toType)))
		return DT_FAILURE | DT_INVALID_PARAM;
	mid[0] = (left[0]+right[0])*0.5f;
	mid[1] = (left[1]+right[1])*0.5f;
	mid[2] = (left[2]+right[2])*0.5f;
	return DT_SUCCESS;
}

dtStatus dtNavMeshQuery::getEdgeMidPoint(dtPolyRef from, const dtPoly* fromPoly, const dtMeshTile* fromTile,
										 dtPolyRef to, const dtPoly* toPoly, const dtMeshTile* toTile,
										 dtReal* mid) const
{
	dtReal left[3], right[3];
	if (dtStatusFailed(getPortalPoints(from, fromPoly, fromTile, to, toPoly, toTile, left, right)))
		return DT_FAILURE | DT_INVALID_PARAM;
	mid[0] = (left[0]+right[0])*0.5f;
	mid[1] = (left[1]+right[1])*0.5f;
	mid[2] = (left[2]+right[2])*0.5f;
	return DT_SUCCESS;
}

/// @par
///
/// This method is meant to be used for quick, short distance checks.
///
/// If the path array is too small to hold the result, it will be filled as 
/// far as possible from the start postion toward the end position.
///
/// <b>Using the Hit Parameter (t)</b>
/// 
/// If the hit parameter is a very high value (DT_REAL_MAX), then the ray has hit 
/// the end position. In this case the path represents a valid corridor to the 
/// end position and the value of @p hitNormal is undefined.
///
/// If the hit parameter is zero, then the start position is on the wall that 
/// was hit and the value of @p hitNormal is undefined.
///
/// If 0 < t < 1.0 then the following applies:
///
/// @code
/// distanceToHitBorder = distanceToEndPosition * t
/// hitPoint = startPos + (endPos - startPos) * t
/// @endcode
///
/// <b>Use Case Restriction</b>
///
/// The raycast ignores the y-value of the end position. (2D check.) This 
/// places significant limits on how it can be used. For example:
///
/// Consider a scene where there is a main floor with a second floor balcony 
/// that hangs over the main floor. So the first floor mesh extends below the 
/// balcony mesh. The start position is somewhere on the first floor. The end 
/// position is on the balcony.
///
/// The raycast will search toward the end position along the first floor mesh. 
/// If it reaches the end position's xz-coordinates it will indicate DT_REAL_MAX
/// (no wall hit), meaning it reached the end position. This is one example of why
/// this method is meant for short distance checks.
///
dtStatus dtNavMeshQuery::raycast(dtPolyRef startRef, const dtReal* startPos, const dtReal* endPos,
								 const dtQueryFilter* filter,
								 dtReal* t, dtReal* hitNormal, dtPolyRef* path, int* pathCount, const int maxPath) const
{
	dtAssert(m_nav);
	UE_CLOGF(m_nav == nullptr, LogDebugRaycastCrash, Fatal, "dtNavMeshQuery::raycast doesn't have valid navmesh!");
	
	*t = 0;
	if (pathCount)
		*pathCount = 0;
	
	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	dtPolyRef curRef = startRef;
	dtReal verts[DT_VERTS_PER_POLYGON*3];
	int n = 0;
	
	hitNormal[0] = 0;
	hitNormal[1] = 0;
	hitNormal[2] = 0;

	// [UE]: iteration limit, use the same value as findPath
	const int loopLimit = (m_nodePool->getMaxRuntimeNodes() + 1) * 4;
	int loopCounter = 0;

	dtStatus status = DT_SUCCESS;
	
	while (curRef)
	{
		// failsafe for cycles in navigation graph resulting in infinite loop 
		loopCounter++;
		if (loopCounter >= loopLimit)
		{
			return DT_FAILURE | DT_INVALID_CYCLE_PATH;
		}

		// Cast ray against current polygon.
		// The API input has been checked already, skip checking internal data.
		const dtMeshTile* tile = 0;
		const dtPoly* poly = 0;
		{
			unsigned int salt, it, ip;
			m_nav->decodePolyId(curRef, salt, it, ip);
			UE_CLOG(it >= (unsigned int)m_nav->getMaxTiles(), LogDebugRaycastCrash, Fatal, TEXT("dtNavMeshQuery::raycast tried to access invalid tile with ref:0x%" UINT64_X_FMT " (tileIdx:%d, maxTiles:%d) - out of bounds!"), curRef, it, m_nav->getMaxTiles());
			UE_CLOG(m_nav->getTile(it) == nullptr, LogDebugRaycastCrash, Fatal, TEXT("dtNavMeshQuery::raycast tried to access invalid tile with ref:0x%" UINT64_X_FMT " (tileIdx:%d, maxTiles:%d) - empty tile!"), curRef, it, m_nav->getMaxTiles());
			UE_CLOG(m_nav->getTile(it)->header == nullptr, LogDebugRaycastCrash, Fatal, TEXT("dtNavMeshQuery::raycast tried to access invalid tile with ref:0x%" UINT64_X_FMT " (tileIdx:%d, maxTiles:%d) - missing tile header!"), curRef, it, m_nav->getMaxTiles());
			UE_CLOG(ip >= (unsigned int)m_nav->getTile(it)->header->polyCount, LogDebugRaycastCrash, Fatal, TEXT("dtNavMeshQuery::raycast tried to access invalid poly with ref:0x%" UINT64_X_FMT " (polyIdx:%d, maxPolys:%d)!"), curRef, ip, m_nav->getTile(it)->header->polyCount);
		}
		m_nav->getTileAndPolyByRefUnsafe(curRef, &tile, &poly);
		
		// Check if poly has valid data, bail out otherwise
		if (poly == nullptr || poly->vertCount > DT_VERTS_PER_POLYGON || poly->vertCount == 0)
		{
			if (pathCount)
				*pathCount = n;
			return DT_FAILURE;
		}

		// Collect vertices.
		int nv = 0;
		for (int i = 0; i < (int)poly->vertCount; ++i)
		{
			tile->verts[poly->verts[i]].toWorld(tile->header->bmin, &verts[nv*3]); //@UE
			nv++;
		}
		
		dtReal tmin, tmax;
		int segMin, segMax;
		if (!dtIntersectSegmentPoly2D(startPos, endPos, verts, nv, tmin, tmax, segMin, segMax))
		{
			// Could not hit the polygon, keep the old t and report hit.
			if (pathCount)
				*pathCount = n;
			return status;
		}
		// Keep track of furthest t so far.
		if (tmax > *t)
			*t = tmax;
		
		// Store visited polygons.
		if (n < maxPath)
			path[n++] = curRef;
		else
			status |= DT_BUFFER_TOO_SMALL;
		
		// Ray end is completely inside the polygon.
		if (segMax == -1)
		{
			*t = DT_REAL_MAX;
			if (pathCount)
				*pathCount = n;
			return status;
		}
		
		// Follow neighbours.
		dtPolyRef nextRef = 0;
		
		unsigned int i = poly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(tile, i);
			i = link.next;
			
			// Find link which contains this edge.
			if ((int)link.edge != segMax)
				continue;
			
			// Get pointer to the next polygon.
			const dtMeshTile* nextTile = 0;
			const dtPoly* nextPoly = 0;
			{
				unsigned int salt, it, ip;
				m_nav->decodePolyId(link.ref, salt, it, ip);
				UE_CLOG(it >= (unsigned int)m_nav->getMaxTiles(), LogDebugRaycastCrash, Fatal, TEXT("dtNavMeshQuery::raycast tried to access invalid nei tile with ref:0x%" UINT64_X_FMT " (tileIdx:%d, maxTiles:%d) - out of bounds!"), link.ref, it, m_nav->getMaxTiles());
				UE_CLOG(m_nav->getTile(it) == nullptr, LogDebugRaycastCrash, Fatal, TEXT("dtNavMeshQuery::raycast tried to access invalid nei tile with ref:0x%" UINT64_X_FMT " (tileIdx:%d, maxTiles:%d) - empty tile!"), link.ref, it, m_nav->getMaxTiles());
				UE_CLOG(m_nav->getTile(it)->header == nullptr, LogDebugRaycastCrash, Fatal, TEXT("dtNavMeshQuery::raycast tried to access invalid nei tile with ref:0x%" UINT64_X_FMT " (tileIdx:%d, maxTiles:%d) - missing tile header!"), link.ref, it, m_nav->getMaxTiles());
				UE_CLOG(ip >= (unsigned int)m_nav->getTile(it)->header->polyCount, LogDebugRaycastCrash, Fatal, TEXT("dtNavMeshQuery::raycast tried to access invalid nei poly with ref:0x%" UINT64_X_FMT " (polyIdx:%d, maxPolys:%d)!"), link.ref, ip, m_nav->getTile(it)->header->polyCount);
			}
			m_nav->getTileAndPolyByRefUnsafe(link.ref, &nextTile, &nextPoly);

			// Skip off-mesh connections.
			if (nextPoly->getType() != DT_POLYTYPE_GROUND)
				continue;
			
			// Skip links based on filter.
			if (!filter->passFilter(link.ref, nextTile, nextPoly) || !passLinkFilterByRef(nextTile, link.ref))
				continue;
			
			// If the link is internal, just return the ref.
//@UE BEGIN
			if (link.side & DT_CONNECTION_INTERNAL)
//@UE END
			{
				nextRef = link.ref;
				break;
			}
			
			// If the link is at tile boundary,
			
			// Check if the link spans the whole edge, and accept.
			if (link.bmin == 0 && link.bmax == 255)
			{
				nextRef = link.ref;
				break;
			}
			
			// Check for partial edge links.
			const int v0 = poly->verts[link.edge];
			CA_SUPPRESS(6385);
			const int v1 = poly->verts[(link.edge+1) % poly->vertCount];
			dtReal left[3], right[3]; //@UE
			tile->verts[v0].toWorld(tile->header->bmin, left); //@UE
			tile->verts[v1].toWorld(tile->header->bmin, right); //@UE
			
//@UE BEGIN
			// strip off additional flags
			const unsigned char side = link.side & DT_LINK_FLAG_SIDE_MASK;

			// Check that the intersection lies inside the link portal.
			if (side == 0 || side == 4)
//@UE END
			{
				// Calculate link size.
				const dtReal s = dtReal(1.)/255.0f;
				dtReal lmin = left[2] + (right[2] - left[2])*(link.bmin*s);
				dtReal lmax = left[2] + (right[2] - left[2])*(link.bmax*s);
				if (lmin > lmax) dtSwap(lmin, lmax);
				
				// Find Z intersection.
				dtReal z = startPos[2] + (endPos[2]-startPos[2])*tmax;
				if (z >= lmin && z <= lmax)
				{
					nextRef = link.ref;
					break;
				}
			}
//@UE BEGIN
			else if (side == 2 || side == 6)
//@UE END
			{
				// Calculate link size.
				const dtReal s = dtReal(1.)/255.0f;
				dtReal lmin = left[0] + (right[0] - left[0])*(link.bmin*s);
				dtReal lmax = left[0] + (right[0] - left[0])*(link.bmax*s);
				if (lmin > lmax) dtSwap(lmin, lmax);
				
				// Find X intersection.
				dtReal x = startPos[0] + (endPos[0]-startPos[0])*tmax;
				if (x >= lmin && x <= lmax)
				{
					nextRef = link.ref;
					break;
				}
			}
		}
		
		if (!nextRef)
		{
			// No neighbour, we hit a wall.
			
			// Calculate hit normal.
			const int a = segMax;
			const int b = segMax+1 < nv ? segMax+1 : 0;
			const dtReal* va = &verts[a*3];
			const dtReal* vb = &verts[b*3];
			const dtReal dx = vb[0] - va[0];
			const dtReal dz = vb[2] - va[2];
			hitNormal[0] = dz;
			hitNormal[1] = 0;
			hitNormal[2] = -dx;
			dtVnormalize(hitNormal);
			
			if (pathCount)
				*pathCount = n;
			return status;
		}
		
		// No hit, advance to neighbour polygon.
		curRef = nextRef;
	}
	
	if (pathCount)
		*pathCount = n;
	
	return status;
}

/// @par
///
/// At least one result array must be provided.
///
/// The order of the result set is from least to highest cost to reach the polygon.
///
/// A common use case for this method is to perform Dijkstra searches. 
/// Candidate polygons are found by searching the graph beginning at the start polygon.
///
/// If a polygon is not found via the graph search, even if it intersects the 
/// search circle, it will not be included in the result set. For example:
///
/// polyA is the start polygon.
/// polyB shares an edge with polyA. (Is adjacent.)
/// polyC shares an edge with polyB, but not with polyA
/// Even if the search circle overlaps polyC, it will not be included in the 
/// result set unless polyB is also in the set.
/// 
/// The value of the center point is used as the start position for cost 
/// calculations. It is not projected onto the surface of the mesh, so its 
/// y-value will effect the costs.
///
/// Intersection tests occur in 2D. All polygons and the search circle are 
/// projected onto the xz-plane. So the y-value of the center point does not 
/// effect intersection tests.
///
/// If the result arrays are to small to hold the entire result set, they will be 
/// filled to capacity.
/// 
dtStatus dtNavMeshQuery::findPolysAroundCircle(dtPolyRef startRef, const dtReal* centerPos, const dtReal radius,
											   const dtQueryFilter* filter,
											   dtPolyRef* resultRef, dtPolyRef* resultParent, dtReal* resultCost,
											   int* resultCount, const int maxResult) const
{
	dtAssert(m_nav);
	dtAssert(m_nodePool);
	dtAssert(m_openList);

//@UE BEGIN
	if (!resultCount)
		return DT_FAILURE | DT_INVALID_PARAM;
//@UE END

	*resultCount = 0;
	
	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	m_nodePool->clear();
	m_openList->clear();
	
	dtNode* startNode = m_nodePool->getNode(startRef);
	dtVcopy(startNode->pos, centerPos);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_OPEN;
	m_openList->push(startNode);
	
	dtStatus status = DT_SUCCESS;
	
	int n = 0;
	if (n < maxResult)
	{
		if (resultRef)
			resultRef[n] = startNode->id;
		if (resultParent)
			resultParent[n] = 0;
		if (resultCost)
			resultCost[n] = 0;
		++n;
	}
	else
	{
		status |= DT_BUFFER_TOO_SMALL;
	}
	
	const dtReal radiusSqr = dtSqr(radius);
	
	while (!m_openList->empty())
	{
		dtNode* bestNode = m_openList->pop();
		bestNode->flags &= ~DT_NODE_OPEN;
		bestNode->flags |= DT_NODE_CLOSED;
		
		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef bestRef = bestNode->id;
		const dtMeshTile* bestTile = 0;
		const dtPoly* bestPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(bestRef, &bestTile, &bestPoly);
		
		// Get parent poly and tile.
		dtPolyRef parentRef = 0;
		const dtMeshTile* parentTile = 0;
		const dtPoly* parentPoly = 0;
		if (bestNode->pidx)
			parentRef = m_nodePool->getNodeAtIdx(bestNode->pidx)->id;
		if (parentRef)
			m_nav->getTileAndPolyByRefUnsafe(parentRef, &parentTile, &parentPoly);
		
		unsigned int i = bestPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(bestTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours and do not follow back to parent.
			if (!neighbourRef || neighbourRef == parentRef
//@UE BEGIN
				|| !filter->isValidLinkSide(link.side))
//@UE END
				continue;
			
			// Expand to neighbour
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);
		
			// Do not advance if the polygon is excluded by the filter.
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
				continue;
			
			// Find edge and calc distance to the edge.
			dtReal va[3], vb[3];
			if (!getPortalPoints(bestRef, bestPoly, bestTile, neighbourRef, neighbourPoly, neighbourTile, va, vb))
				continue;
			
			// If the circle is not touching the next polygon, skip it.
			dtReal tseg;
			dtReal distSqr = dtDistancePtSegSqr2D(centerPos, va, vb, tseg);
			if (distSqr > radiusSqr)
				continue;
			
			dtNode* neighbourNode = m_nodePool->getNode(neighbourRef);
			if (!neighbourNode)
			{
				status |= DT_OUT_OF_NODES;
				continue;
			}
				
			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;
			
			// Cost
			if (neighbourNode->flags == 0)
				dtVlerp(neighbourNode->pos, va, vb, 0.5f);
			
			const dtReal total = bestNode->total + dtVdist(bestNode->pos, neighbourNode->pos);
			
			// The node is already in open list and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_OPEN) && total >= neighbourNode->total)
				continue;
			
			neighbourNode->id = neighbourRef;
			neighbourNode->flags = (neighbourNode->flags & ~DT_NODE_CLOSED);
			neighbourNode->pidx = m_nodePool->getNodeIdx(bestNode);
			neighbourNode->total = total;
			
			if (neighbourNode->flags & DT_NODE_OPEN)
			{
				m_openList->modify(neighbourNode);
			}
			else
			{
				if (n < maxResult)
				{
					if (resultRef)
						resultRef[n] = neighbourNode->id;
					if (resultParent)
						resultParent[n] = m_nodePool->getNodeAtIdx(neighbourNode->pidx)->id;
					if (resultCost)
						resultCost[n] = neighbourNode->total;
					++n;
				}
				else
				{
					status |= DT_BUFFER_TOO_SMALL;
				}
				neighbourNode->flags = DT_NODE_OPEN;
				m_openList->push(neighbourNode);
			}
		}
	}
	
	*resultCount = n;
	
	return status;
}

/// @par
///
/// The order of the result set is from least to highest cost.
/// 
/// At least one result array must be provided.
///
/// A common use case for this method is to perform Dijkstra searches. 
/// Candidate polygons are found by searching the graph beginning at the start 
/// polygon.
/// 
/// The same intersection test restrictions that apply to findPolysAroundCircle()
/// method apply to this method.
/// 
/// The 3D centroid of the search polygon is used as the start position for cost 
/// calculations.
/// 
/// Intersection tests occur in 2D. All polygons are projected onto the 
/// xz-plane. So the y-values of the vertices do not effect intersection tests.
/// 
/// If the result arrays are is too small to hold the entire result set, they will 
/// be filled to capacity.
///
dtStatus dtNavMeshQuery::findPolysAroundShape(dtPolyRef startRef, const dtReal* verts, const int nverts,
											  const dtQueryFilter* filter,
											  dtPolyRef* resultRef, dtPolyRef* resultParent, dtReal* resultCost,
											  int* resultCount, const int maxResult) const
{
	dtAssert(m_nav);
	dtAssert(m_nodePool);
	dtAssert(m_openList);
	
	*resultCount = 0;
	
	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	m_nodePool->clear();
	m_openList->clear();
	
	dtReal centerPos[3] = {0,0,0};
	for (int i = 0; i < nverts; ++i)
		dtVadd(centerPos,centerPos,&verts[i*3]);
	dtVscale(centerPos,centerPos,dtReal(1.)/nverts);

	dtNode* startNode = m_nodePool->getNode(startRef);
	dtVcopy(startNode->pos, centerPos);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_OPEN;
	m_openList->push(startNode);
	
	dtStatus status = DT_SUCCESS;

	int n = 0;
	if (n < maxResult)
	{
		if (resultRef)
			resultRef[n] = startNode->id;
		if (resultParent)
			resultParent[n] = 0;
		if (resultCost)
			resultCost[n] = 0;
		++n;
	}
	else
	{
		status |= DT_BUFFER_TOO_SMALL;
	}
	
	while (!m_openList->empty())
	{
		dtNode* bestNode = m_openList->pop();
		bestNode->flags &= ~DT_NODE_OPEN;
		bestNode->flags |= DT_NODE_CLOSED;
		
		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef bestRef = bestNode->id;
		const dtMeshTile* bestTile = 0;
		const dtPoly* bestPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(bestRef, &bestTile, &bestPoly);
		
		// Get parent poly and tile.
		dtPolyRef parentRef = 0;
		const dtMeshTile* parentTile = 0;
		const dtPoly* parentPoly = 0;
		if (bestNode->pidx)
			parentRef = m_nodePool->getNodeAtIdx(bestNode->pidx)->id;
		if (parentRef)
			m_nav->getTileAndPolyByRefUnsafe(parentRef, &parentTile, &parentPoly);
		
		unsigned int i = bestPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(bestTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours and do not follow back to parent.
			if (!neighbourRef || neighbourRef == parentRef
//@UE BEGIN
				|| !filter->isValidLinkSide(link.side))
//@UE END
				continue;
			
			// Expand to neighbour
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);
			
			// Do not advance if the polygon is excluded by the filter.
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
				continue;
			
			// Find edge and calc distance to the edge.
			dtReal va[3], vb[3];
			if (!getPortalPoints(bestRef, bestPoly, bestTile, neighbourRef, neighbourPoly, neighbourTile, va, vb))
				continue;
			
			// If the poly is not touching the edge to the next polygon, skip the connection it.
			dtReal tmin, tmax;
			int segMin, segMax;
			if (!dtIntersectSegmentPoly2D(va, vb, verts, nverts, tmin, tmax, segMin, segMax))
				continue;
			if (tmin > 1.0f || tmax < 0.0f)
				continue;
			
			dtNode* neighbourNode = m_nodePool->getNode(neighbourRef);
			if (!neighbourNode)
			{
				status |= DT_OUT_OF_NODES;
				continue;
			}
			
			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;
			
			// Cost
			if (neighbourNode->flags == 0)
				dtVlerp(neighbourNode->pos, va, vb, 0.5f);
			
			const dtReal total = bestNode->total + dtVdist(bestNode->pos, neighbourNode->pos);
			
			// The node is already in open list and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_OPEN) && total >= neighbourNode->total)
				continue;
			
			neighbourNode->id = neighbourRef;
			neighbourNode->flags = (neighbourNode->flags & ~DT_NODE_CLOSED);
			neighbourNode->pidx = m_nodePool->getNodeIdx(bestNode);
			neighbourNode->total = total;
			
			if (neighbourNode->flags & DT_NODE_OPEN)
			{
				m_openList->modify(neighbourNode);
			}
			else
			{
				if (n < maxResult)
				{
					if (resultRef)
						resultRef[n] = neighbourNode->id;
					if (resultParent)
						resultParent[n] = m_nodePool->getNodeAtIdx(neighbourNode->pidx)->id;
					if (resultCost)
						resultCost[n] = neighbourNode->total;
					++n;
				}
				else
				{
					status |= DT_BUFFER_TOO_SMALL;
				}
				neighbourNode->flags = DT_NODE_OPEN;
				m_openList->push(neighbourNode);
			}
		}
	}
	
	*resultCount = n;
	
	return status;
}

//@UE BEGIN
//	based on dtNavMeshQuery::findPolysAroundCircle. Refer to its description for more details.
dtStatus dtNavMeshQuery::findPolysInPathDistance(dtPolyRef startRef, const dtReal* centerPos, const dtReal pathDistance,
												const dtQueryFilter* filter, dtPolyRef* resultRef,
												int* resultCount, const int maxResult) const
{
	dtAssert(m_nav);
	dtAssert(m_nodePool);
	dtAssert(m_openList);

	*resultCount = 0;

	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;

	m_nodePool->clear();
	m_openList->clear();

	dtNode* startNode = m_nodePool->getNode(startRef);
	dtVcopy(startNode->pos, centerPos);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_OPEN;
	m_openList->push(startNode);

	dtStatus status = DT_SUCCESS;

	int n = 0;
	if (n < maxResult)
	{
		if (resultRef)
			resultRef[n] = startNode->id;
		++n;
	}
	else
	{
		status |= DT_BUFFER_TOO_SMALL;
	}

	const dtReal pathDistSqr = dtSqr(pathDistance);

	while (!m_openList->empty())
	{
		dtNode* bestNode = m_openList->pop();
		bestNode->flags &= ~DT_NODE_OPEN;
		bestNode->flags |= DT_NODE_CLOSED;

		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef bestRef = bestNode->id;
		const dtMeshTile* bestTile = 0;
		const dtPoly* bestPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(bestRef, &bestTile, &bestPoly);

		// Get parent poly and tile.
		dtPolyRef parentRef = 0;
		const dtMeshTile* parentTile = 0;
		const dtPoly* parentPoly = 0;
		if (bestNode->pidx)
			parentRef = m_nodePool->getNodeAtIdx(bestNode->pidx)->id;
		if (parentRef)
			m_nav->getTileAndPolyByRefUnsafe(parentRef, &parentTile, &parentPoly);

		unsigned int i = bestPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(bestTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours and do not follow back to parent.
			if (!neighbourRef || neighbourRef == parentRef
//@UE BEGIN
				|| !filter->isValidLinkSide(link.side))
//@UE END
				continue;

			// Expand to neighbour
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);

			// Do not advance if the polygon is excluded by the filter.
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
				continue;

			// Find edge and calc distance to the edge.
			dtReal va[3], vb[3];
			if (!getPortalPoints(bestRef, bestPoly, bestTile, neighbourRef, neighbourPoly, neighbourTile, va, vb))
				continue;

			// If the circle is not touching the next polygon, skip it.
			dtReal tseg;
			dtReal distSqr = dtDistancePtSegSqr2D(centerPos, va, vb, tseg);
			if (distSqr > pathDistSqr)
				continue;

			dtNode* neighbourNode = m_nodePool->getNode(neighbourRef);
			if (!neighbourNode)
			{
				status |= DT_OUT_OF_NODES;
				continue;
			}

			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;

			// Cost
			if (neighbourNode->flags == 0)
				dtVlerp(neighbourNode->pos, va, vb, 0.5f);

			const dtReal total = bestNode->total + dtVdist(bestNode->pos, neighbourNode->pos);

			// The node is already in open list and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_OPEN) && total >= neighbourNode->total)
				continue;

			if (dtSqr(total) >= pathDistSqr)
				continue;

			neighbourNode->id = neighbourRef;
			neighbourNode->flags = (neighbourNode->flags & ~DT_NODE_CLOSED);
			neighbourNode->pidx = m_nodePool->getNodeIdx(bestNode);
			neighbourNode->total = total;

			if (neighbourNode->flags & DT_NODE_OPEN)
			{
				m_openList->modify(neighbourNode);
			}
			else
			{
				if (n < maxResult)
				{
					if (resultRef)
						resultRef[n] = neighbourNode->id;
					++n;
				}
				else
				{
					status |= DT_BUFFER_TOO_SMALL;
				}
				neighbourNode->flags = DT_NODE_OPEN;
				m_openList->push(neighbourNode);
			}
		}
	}

	*resultCount = n;

	return status;
}

static bool containsPolyRef(const dtPolyRef testRef, const dtPolyRef* path, const int npath)
{
	for (int i = 0; i < npath; i++)
	{
		if (path[i] == testRef)
			return true;
	}

	return false;
}
//@UE END

/// @par
///
/// This method is optimized for a small search radius and small number of result 
/// polygons.
///
/// Candidate polygons are found by searching the navigation graph beginning at 
/// the start polygon.
///
/// The same intersection test restrictions that apply to the findPolysAroundCircle 
/// method applies to this method.
///
/// The value of the center point is used as the start point for cost calculations. 
/// It is not projected onto the surface of the mesh, so its y-value will effect 
/// the costs.
/// 
/// Intersection tests occur in 2D. All polygons and the search circle are 
/// projected onto the xz-plane. So the y-value of the center point does not 
/// effect intersection tests.
/// 
/// If the result arrays are is too small to hold the entire result set, they will 
/// be filled to capacity.
/// 
dtStatus dtNavMeshQuery::findLocalNeighbourhood(dtPolyRef startRef, const dtReal* centerPos, const dtReal radius,
												const dtQueryFilter* filter,
												dtPolyRef* resultRef, dtPolyRef* resultParent,
												int* resultCount, const int maxResult) const
{
	dtAssert(m_nav);
	dtAssert(m_tinyNodePool);
	
	*resultCount = 0;

	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	static const int MAX_STACK = 48;
	dtNode* stack[MAX_STACK];
	int nstack = 0;
	
	m_tinyNodePool->clear();
	
	dtNode* startNode = m_tinyNodePool->getNode(startRef);
	startNode->pidx = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_CLOSED;
	stack[nstack++] = startNode;
	
	const dtReal radiusSqr = dtSqr(radius);
	
	dtReal pa[DT_VERTS_PER_POLYGON*3];
	dtReal pb[DT_VERTS_PER_POLYGON*3];
	
	dtStatus status = DT_SUCCESS;
	
	int n = 0;
	if (n < maxResult)
	{
		resultRef[n] = startNode->id;
		if (resultParent)
			resultParent[n] = 0;
		++n;
	}
	else
	{
		status |= DT_BUFFER_TOO_SMALL;
	}
	
	while (nstack)
	{
		// Pop front.
		dtNode* curNode = stack[0];
		for (int stackIndex = 0; stackIndex < nstack - 1; ++stackIndex)
			stack[stackIndex] = stack[stackIndex + 1];
		nstack--;
		
		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef curRef = curNode->id;
		const dtMeshTile* curTile = 0;
		const dtPoly* curPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(curRef, &curTile, &curPoly);
		
		unsigned int i = curPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(curTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours.
			if (!neighbourRef)
				continue;
			
			// Skip if cannot alloca more nodes.
			dtNode* neighbourNode = m_tinyNodePool->getNode(neighbourRef);
			if (!neighbourNode)
				continue;
			// Skip visited.
			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;
			
			// Expand to neighbour
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);
			
			// Skip off-mesh connections.
			if (neighbourPoly->getType() != DT_POLYTYPE_GROUND)
				continue;
			
			// Do not advance if the polygon is excluded by the filter.
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
			{
				continue;
			}
			
			// Find edge and calc distance to the edge.
			dtReal va[3], vb[3];
			if (!getPortalPoints(curRef, curPoly, curTile, neighbourRef, neighbourPoly, neighbourTile, va, vb))
				continue;
			
			// If the circle is not touching the next polygon, skip it.
			dtReal tseg;
			dtReal distSqr = dtDistancePtSegSqr2D(centerPos, va, vb, tseg);
			if (distSqr > radiusSqr)
				continue;
			
			// Mark node visited, this is done before the overlap test so that
			// we will not visit the poly again if the test fails.
			neighbourNode->flags |= DT_NODE_CLOSED;
			neighbourNode->pidx = m_tinyNodePool->getNodeIdx(curNode);
			
			// Check that the polygon does not collide with existing polygons.
			
			// Collect vertices of the neighbour poly.
			const int npa = neighbourPoly->vertCount;
			for (int neighbourPolyVertIndex = 0; neighbourPolyVertIndex < npa; ++neighbourPolyVertIndex)
				neighbourTile->verts[neighbourPoly->verts[neighbourPolyVertIndex]].toWorld(neighbourTile->header->bmin, &pa[neighbourPolyVertIndex * 3]); //@UE
			
			bool overlap = false;
			for (int j = 0; j < n; ++j)
			{
				dtPolyRef pastRef = resultRef[j];
				
				// Connected polys do not overlap.
				bool connected = false;
				unsigned int neighbourLinkIndex = neighbourPoly->firstLink;
				while (neighbourLinkIndex != DT_NULL_LINK)
				{
					const dtLink& link2 = m_nav->getLink(neighbourTile, neighbourLinkIndex);
					neighbourLinkIndex = link2.next;

					if (link2.ref == pastRef)
					{
						connected = true;
						break;
					}
				}
				if (connected)
					continue;
				
				// Potentially overlapping.
				const dtMeshTile* pastTile = 0;
				const dtPoly* pastPoly = 0;
				m_nav->getTileAndPolyByRefUnsafe(pastRef, &pastTile, &pastPoly);
				
				// Get vertices and test overlap
				const int npb = pastPoly->vertCount;
				for (int pastPolyVertIndex = 0; pastPolyVertIndex < npb; ++pastPolyVertIndex)
					pastTile->verts[pastPoly->verts[pastPolyVertIndex]].toWorld(pastTile->header->bmin, &pb[pastPolyVertIndex * 3]); //@UE
				
				if (dtOverlapPolyPoly2D(pa,npa, pb,npb))
				{
					overlap = true;
					break;
				}
			}
			if (overlap)
				continue;
			
			// This poly is fine, store and advance to the poly.
			if (n < maxResult)
			{
				resultRef[n] = neighbourRef;
				if (resultParent)
					resultParent[n] = curRef;
				++n;
			}
			else
			{
				status |= DT_BUFFER_TOO_SMALL;
			}
			
			if (nstack < MAX_STACK)
			{
				stack[nstack++] = neighbourNode;
			}
		}
	}
	
	*resultCount = n;
	
	return status;
}


struct dtSegInterval
{
	dtPolyRef ref;
	short tmin, tmax;
};

static void insertInterval(dtSegInterval* ints, int& nints, const int maxInts,
						   const short tmin, const short tmax, const dtPolyRef ref)
{
	if (nints+1 > maxInts) return;
	// Find insertion point.
	int idx = 0;
	while (idx < nints)
	{
		if (tmax <= ints[idx].tmin)
			break;
		idx++;
	}
	// Move current results.
	if (nints-idx)
		memmove(ints+idx+1, ints+idx, sizeof(dtSegInterval)*(nints-idx));
	// Store
	ints[idx].ref = ref;
	ints[idx].tmin = tmin;
	ints[idx].tmax = tmax;
	nints++;
}

/// @par
///
/// If the @p segmentRefs parameter is provided, then all polygon segments will be returned. 
/// Otherwise only the wall segments are returned.
/// 
/// A segment that is normally a portal will be included in the result set as a 
/// wall if the @p filter results in the neighbor polygon becoomming impassable.
/// 
/// The @p segmentVerts and @p segmentRefs buffers should normally be sized for the 
/// maximum segments per polygon of the source navigation mesh.
/// 
dtStatus dtNavMeshQuery::getPolyWallSegments(dtPolyRef ref, const dtQueryFilter* filter,
											 dtReal* segmentVerts, dtPolyRef* segmentRefs, int* segmentCount,
											 const int maxSegments) const
{
	dtAssert(m_nav);
	
	*segmentCount = 0;
	
	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	if (dtStatusFailed(m_nav->getTileAndPolyByRef(ref, &tile, &poly)))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	int n = 0;
	static const int MAX_INTERVAL = 16;
	dtSegInterval ints[MAX_INTERVAL];
	int nints;
	
	bool storePortals = false;// segmentRefs != 0;
	
	dtStatus status = DT_SUCCESS;
	
	for (int i = 0, j = (int)poly->vertCount-1; i < (int)poly->vertCount; j = i++)
	{
		// Skip non-solid edges.
		nints = 0;
		if (poly->neis[j] & DT_EXT_LINK)
		{
			// Tile border.
			unsigned int k = poly->firstLink;
			while (k != DT_NULL_LINK)
			{
				const dtLink& link = m_nav->getLink(tile, k);
				k = link.next;

				if (link.edge == j)
				{
					if (link.ref != 0)
					{
						const dtMeshTile* neiTile = 0;
						const dtPoly* neiPoly = 0;
						m_nav->getTileAndPolyByRefUnsafe(link.ref, &neiTile, &neiPoly);
						if (filter->passFilter(link.ref, neiTile, neiPoly) && passLinkFilterByRef(neiTile, link.ref))
						{
							insertInterval(ints, nints, MAX_INTERVAL, link.bmin, link.bmax, link.ref);
						}
					}
				}
			}
		}
		else
		{
			// Internal edge
			dtPolyRef neiRef = 0;
			if (poly->neis[j])
			{
				const unsigned int idx = (unsigned int)(poly->neis[j]-1);
				neiRef = m_nav->getPolyRefBase(tile) | idx;
				if (!filter->passFilter(neiRef, tile, &tile->polys[idx]) || !passLinkFilter(tile, idx))
				{
					neiRef = 0;
				}
			}

			// If the edge leads to another polygon and portals are not stored, skip.
			if (neiRef != 0 && !storePortals)
				continue;
			
			if (n < maxSegments)
			{
				dtReal* seg = &segmentVerts[n*6];
				tile->verts[poly->verts[j]].toWorld(tile->header->bmin, seg+0); //@UE
				tile->verts[poly->verts[i]].toWorld(tile->header->bmin, seg+3); //@UE
				if (segmentRefs)
					segmentRefs[n] = neiRef;
				n++;
			}
			else
			{
				status |= DT_BUFFER_TOO_SMALL;
			}
			
			continue;
		}
		
		// Add sentinels
		insertInterval(ints, nints, MAX_INTERVAL, -1, 0, 0);
		insertInterval(ints, nints, MAX_INTERVAL, 255, 256, 0);
		
		// Store segments.
		dtReal vj[3], vi[3]; //@UE
		tile->verts[poly->verts[j]].toWorld(tile->header->bmin, vj); //@UE
		tile->verts[poly->verts[i]].toWorld(tile->header->bmin, vi); //@UE
		for (int k = 1; k < nints; ++k)
		{
			// Portal segment.
			if (storePortals && ints[k].ref)
			{
				const dtReal tmin = ints[k].tmin/dtReal(255.); 
				const dtReal tmax = ints[k].tmax/dtReal(255.);
				if (n < maxSegments)
				{
					dtReal* seg = &segmentVerts[n*6];
					dtVlerp(seg+0, vj,vi, tmin);
					dtVlerp(seg+3, vj,vi, tmax);
					if (segmentRefs)
						segmentRefs[n] = ints[k].ref;
					n++;
				}
				else
				{
					status |= DT_BUFFER_TOO_SMALL;
				}
			}

			// Wall segment.
			const int imin = ints[k-1].tmax;
			const int imax = ints[k].tmin;
			if (imin != imax)
			{
				const dtReal tmin = imin/dtReal(255.);
				const dtReal tmax = imax/dtReal(255.);
				if (n < maxSegments)
				{
					dtReal* seg = &segmentVerts[n*6];
					dtVlerp(seg+0, vj,vi, tmin);
					dtVlerp(seg+3, vj,vi, tmax);
					if (segmentRefs)
						segmentRefs[n] = 0;
					n++;
				}
				else
				{
					status |= DT_BUFFER_TOO_SMALL;
				}
			}
		}
	}
	
	*segmentCount = n;
	
	return status;
}

static bool isEdgeInRadius(const dtMeshTile* tile, const dtPoly* poly, int edge, const dtReal* centerPos, const dtReal radiusSqr)
{
	dtReal va[3], vb[3]; //@UE
	tile->verts[poly->verts[edge]].toWorld(tile->header->bmin, va); //@UE
	tile->verts[poly->verts[(edge + 1) % poly->vertCount]].toWorld(tile->header->bmin, vb); //@UE

	dtReal tseg;
	const dtReal distSqr = dtDistancePtSegSqr2D(centerPos, va, vb, tseg);
	return distSqr <= radiusSqr;
}

static void storeWallSegment(const dtNavMesh* nav, const dtMeshTile* tile, const dtPoly* poly, int edge, dtPolyRef ref0, dtPolyRef ref1,
							 dtReal* resultWalls, dtPolyRef* resultRefs, int* resultCount, const int maxResult)
{
	if (*resultCount >= maxResult)
	{
		return;
	}

	// Skip for polys that don't have vertices (e.g. nav links)
	if (poly->vertCount == 0)
	{
		return;
	}

	if (!ensureMsgf(edge < DT_VERTS_PER_POLYGON, TEXT("Invalid edge index %i"), edge))
	{
		return;
	}

	const int tempEdgeIndexB = (edge + 1) % poly->vertCount;
	if (!ensureMsgf(tempEdgeIndexB < DT_VERTS_PER_POLYGON, TEXT("Invalid computed edge index %i"), tempEdgeIndexB))
	{
		return;
	}

	const int vertIndexA = poly->verts[edge]; //@UE
	const int vertIndexB = poly->verts[tempEdgeIndexB]; //@UE

	const int tileVertArraySize = tile->header->vertCount; //@UE
	if (!ensureMsgf(vertIndexA < tileVertArraySize && vertIndexB < tileVertArraySize,
		TEXT("Invalid vert index (vertIndexA=%i, vertIndexB=%i, vertCount=%i)"), vertIndexA, vertIndexB, tile->header->vertCount))
	{
		return;
	}

	dtReal va[3], vb[3]; //@UE
	tile->verts[vertIndexA].toWorld(tile->header->bmin, va); //@UE
	tile->verts[vertIndexB].toWorld(tile->header->bmin, vb); //@UE

	const int32 wall0Offset = (*resultCount * 6) + 0;
	const int32 wall1Offset = (*resultCount * 6) + 3;

	dtVcopy(&resultWalls[wall0Offset], va);
	dtVcopy(&resultWalls[wall1Offset], vb);
	resultRefs[*resultCount * 2 + 0] = ref0;
	resultRefs[*resultCount * 2 + 1] = ref1;

	*resultCount += 1;

	// If neighbour is valid, find the segment that both polygons share by projecting the neighbour segment to the current segment va-vb.
	if (ref1)
	{
		const dtMeshTile* neiTile = 0;
		const dtPoly* neiPoly = 0;
		nav->getTileAndPolyByRef(ref1, &neiTile, &neiPoly);

		// Find edge of the neighbour polygon.
		int neiEdge = -1;
		unsigned int neiLinkId = neiPoly ? neiPoly->firstLink : DT_NULL_LINK;
		while (neiLinkId != DT_NULL_LINK)
		{
			const dtLink& link = nav->getLink(neiTile, neiLinkId);
			neiLinkId = link.next;
			if (link.ref == ref0)
			{
				neiEdge = link.edge;
				break;
			}
		}

		if (neiEdge != -1)
		{
			dtReal va2[3], vb2[3]; //@UE
			neiTile->verts[neiPoly->verts[neiEdge]].toWorld(neiTile->header->bmin, va2); //@UE
			neiTile->verts[neiPoly->verts[(neiEdge + 1) % neiPoly->vertCount]].toWorld(neiTile->header->bmin, vb2); //@UE

			// Project and clip segment va2-vb2 on va-vb
			dtReal seg[3], diffA[3], diffB[3], clippedA[3], clippedB[3];
			dtVsub(seg, vb, va);

			dtVsub(diffA, va2, va);
			dtVsub(diffB, vb2, va);
			const dtReal da = dtVdot(diffA, seg);
			const dtReal db = dtVdot(diffB, seg);
			const dtReal ds = dtVdot(seg, seg);
			const dtReal dmin = dtMin(da, db);
			const dtReal dmax = dtMax(da, db);
			
			if (dmin <= 0)
			{
				dtVcopy(clippedA, va);
			}
			else if (dmin >= ds)
			{
				dtVcopy(clippedA, vb);
			}
			else
			{
				dtVmad(clippedA, va, seg, dmin / ds);
			}

			if (dmax <= 0)
			{
				dtVcopy(clippedB, va);
			}
			else if (dmax >= ds)
			{
				dtVcopy(clippedB, vb);
			}
			else
			{
				dtVmad(clippedB, va, seg, dmax / ds);
			}

			// Store projected segment (intersection of both edges)
			dtVcopy(&resultWalls[wall0Offset], clippedA);
			dtVcopy(&resultWalls[wall1Offset], clippedB);
		}
	}
}

dtStatus dtNavMeshQuery::findWallsInNeighbourhood(dtPolyRef startRef, const dtReal* centerPos, const dtReal radius,
												  const dtQueryFilter* filter,
												  dtPolyRef* neiRefs, int* neiCount, const int maxNei,
												  dtReal* resultWalls, dtPolyRef* resultRefs, int* resultCount, const int maxResult) const
{
	*resultCount = 0;
	*neiCount = 0;

	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;

	m_tinyNodePool->clear();

	static const int MAX_STACK = 48;
	dtNode* stack[MAX_STACK];
	int nstack = 0;

	dtNode* startNode = m_tinyNodePool->getNode(startRef);
	startNode->pidx = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_CLOSED;
	stack[nstack++] = startNode;

	dtStatus status = DT_SUCCESS;
	const dtReal radiusSqr = dtSqr(radius);

	int n = 0;
	if (n < maxNei)
	{
		neiRefs[n] = startNode->id;
		++n;
	}
	else
	{
		status |= DT_BUFFER_TOO_SMALL;
	}

	while (nstack)
	{
		// Pop front.
		dtNode* curNode = stack[0];
		for (int stackIndex = 0; stackIndex < nstack - 1; ++stackIndex)
			stack[stackIndex] = stack[stackIndex + 1];
		nstack--;
		
		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef curRef = curNode->id;
		const dtMeshTile* curTile = 0;
		const dtPoly* curPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(curRef, &curTile, &curPoly);

		unsigned int i = curPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(curTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours.
			if (!neighbourRef)
			{
				// store wall segment
				if (isEdgeInRadius(curTile, curPoly, link.edge, centerPos, radiusSqr))
				{
					storeWallSegment(m_nav, curTile, curPoly, link.edge,  curRef, 0, 
						resultWalls, resultRefs, resultCount, maxResult);
				}
				continue;
			}

			// Skip if cannot alloca more nodes.
			dtNode* neighbourNode = m_tinyNodePool->getNode(neighbourRef);
			if (!neighbourNode)
				continue;
			// Skip visited.
			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;

			// Expand to neighbour
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);

			// Skip off-mesh connections.
			if (neighbourPoly->getType() != DT_POLYTYPE_GROUND)
				continue;

			// Do not advance if the polygon is excluded by the filter.
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
			{
				// store wall segment
				if (isEdgeInRadius(curTile, curPoly, link.edge, centerPos, radiusSqr))
				{
					storeWallSegment(m_nav, curTile, curPoly, link.edge, curRef, neighbourRef,
						resultWalls, resultRefs, resultCount, maxResult);
				}
				continue;
			}

			// Find edge and calc distance to the edge.
			dtReal va[3], vb[3];
			if (!getPortalPoints(curRef, curPoly, curTile, neighbourRef, neighbourPoly, neighbourTile, va, vb))
				continue;

			// If the circle is not touching the next polygon, skip it.
			dtReal tseg;
			dtReal distSqr = dtDistancePtSegSqr2D(centerPos, va, vb, tseg);
			if (distSqr > radiusSqr)
				continue;

			// Mark node visited, this is done before the overlap test so that
			// we will not visit the poly again if the test fails.
			neighbourNode->flags |= DT_NODE_CLOSED;
			neighbourNode->pidx = m_tinyNodePool->getNodeIdx(curNode);			

			// This poly is fine, store and advance to the poly.
			if (n < maxNei)
			{
				neiRefs[n] = neighbourRef;
				++n;
			}
			else
			{
				status |= DT_BUFFER_TOO_SMALL;
			}

			if (nstack < MAX_STACK)
			{
				stack[nstack++] = neighbourNode;
			}
		}

		// add hard edges of poly
		for (int neighbourIndex = 0; neighbourIndex < curPoly->vertCount; neighbourIndex++)
		{
			bool bStoreEdge = (curPoly->neis[neighbourIndex] == 0);
			if (curPoly->neis[neighbourIndex] & DT_EXT_LINK)
			{
				// check if external edge has valid link
				bool bConnected = false;

				unsigned int linkIdx = curPoly->firstLink;
				while (linkIdx != DT_NULL_LINK)
				{
					const dtLink& link = m_nav->getLink(curTile, linkIdx);
					linkIdx = link.next;

					if (link.edge == neighbourIndex)
					{
						bConnected = true;
						break;
					}
				}

				bStoreEdge = !bConnected;
			}

			if (bStoreEdge)
			{
				if (isEdgeInRadius(curTile, curPoly, neighbourIndex, centerPos, radiusSqr))
				{
					storeWallSegment(m_nav, curTile, curPoly, neighbourIndex, curRef, 0, 
						resultWalls, resultRefs, resultCount, maxResult);
				}
			}
		}
	}

	*neiCount = n;
	return status;
}

dtStatus dtNavMeshQuery::findWallsOverlappingShape(dtPolyRef startRef, const dtReal* verts, const int nverts,
												   const dtQueryFilter* filter,
												   dtPolyRef* neiRefs, int* neiCount, const int maxNei,
												   dtReal* resultWalls, dtPolyRef* resultRefs, int* resultCount, const int maxResult) const
{
	*resultCount = 0;
	*neiCount = 0;

	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;

	m_tinyNodePool->clear();

	static const int MAX_STACK = 48;
	dtNode* stack[MAX_STACK];
	int nstack = 0;

	dtNode* startNode = m_tinyNodePool->getNode(startRef);
	startNode->pidx = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_CLOSED;
	stack[nstack++] = startNode;

	dtStatus status = DT_SUCCESS;

	int n = 0;
	if (n < maxNei)
	{
		neiRefs[n] = startNode->id;
		++n;
	}
	else
	{
		status |= DT_BUFFER_TOO_SMALL;
	}

	while (nstack)
	{
		// Pop front.
		dtNode* curNode = stack[0];
		for (int stackIndex = 0; stackIndex < nstack - 1; ++stackIndex)
			stack[stackIndex] = stack[stackIndex + 1];
		nstack--;
		
		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef curRef = curNode->id;
		const dtMeshTile* curTile = 0;
		const dtPoly* curPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(curRef, &curTile, &curPoly);

		unsigned int i = curPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(curTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours.
			if (!neighbourRef)
			{
				// store wall segment
				storeWallSegment(m_nav, curTile, curPoly, link.edge, curRef, 0, 
					resultWalls, resultRefs, resultCount, maxResult);
				continue;
			}

			// Skip if cannot alloca more nodes.
			dtNode* neighbourNode = m_tinyNodePool->getNode(neighbourRef);
			if (!neighbourNode)
				continue;
			// Skip visited.
			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;

			// Expand to neighbour
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);

			// Skip off-mesh connections.
			if (neighbourPoly->getType() != DT_POLYTYPE_GROUND)
				continue;

			// Do not advance if the polygon is excluded by the filter.
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
			{
				// store wall segment
				storeWallSegment(m_nav, curTile, curPoly, link.edge, curRef, neighbourRef,
					resultWalls, resultRefs, resultCount, maxResult);

				continue;
			}

			// Find edge and calc distance to the edge.
			dtReal va[3], vb[3];
			if (!getPortalPoints(curRef, curPoly, curTile, neighbourRef, neighbourPoly, neighbourTile, va, vb))
				continue;

			// If the poly is not touching the edge to the next polygon, skip the connection it.
			dtReal tmin, tmax;
			int segMin, segMax;
			if (!dtIntersectSegmentPoly2D(va, vb, verts, nverts, tmin, tmax, segMin, segMax))
				continue;
			if (tmin > 1.0f || tmax < 0.0f)
				continue;

			// Mark node visited, this is done before the overlap test so that
			// we will not visit the poly again if the test fails.
			neighbourNode->flags |= DT_NODE_CLOSED;
			neighbourNode->pidx = m_tinyNodePool->getNodeIdx(curNode);			

			// This poly is fine, store and advance to the poly.
			if (n < maxNei)
			{
				neiRefs[n] = neighbourRef;
				++n;
			}
			else
			{
				status |= DT_BUFFER_TOO_SMALL;
			}

			if (nstack < MAX_STACK)
			{
				stack[nstack++] = neighbourNode;
			}
		}

		// add hard edges of poly
		for (int neighbourIndex = 0; neighbourIndex < curPoly->vertCount; neighbourIndex++)
		{
			bool bStoreEdge = (curPoly->neis[neighbourIndex] == 0);
			if (curPoly->neis[neighbourIndex] & DT_EXT_LINK)
			{
				// check if external edge has valid link
				bool bConnected = false;

				unsigned int linkIdx = curPoly->firstLink;
				while (linkIdx != DT_NULL_LINK)
				{
					const dtLink& link = m_nav->getLink(curTile, linkIdx);
					linkIdx = link.next;

					if (link.edge == neighbourIndex)
					{
						bConnected = true;
						break;
					}
				}

				bStoreEdge = !bConnected;
			}

			if (bStoreEdge)
			{
				storeWallSegment(m_nav, curTile, curPoly, neighbourIndex, curRef, 0, 
					resultWalls, resultRefs, resultCount, maxResult);
			}
		}
	}

	*neiCount = n;
	return status;
}

dtStatus dtNavMeshQuery::findWallsAroundPath(const dtPolyRef* path, const int pathCount, const dtReal* searchAreaPoly, const int searchAreaPolyCount,
												   const dtReal maxAreaEnterCost, const dtQueryFilter* filter,
												   dtPolyRef* neiRefs, int* neiCount, const int maxNei,
												   dtReal* resultWalls, dtPolyRef* resultRefs, int* resultCount, const int maxResult) const
{
	*resultCount = 0;
	*neiCount = 0;

	// Validate input
	if (pathCount == 0)
		return DT_FAILURE | DT_INVALID_PARAM;

	dtStatus status = DT_SUCCESS;

	int n = 0;
	
	m_tinyNodePool->clear();

	static const int MAX_STACK = 48;
	dtNode* stack[MAX_STACK];
	int nstack = 0;

	// Add all seed polygons in the stack.
	for (int i = 0; i < dtMin(pathCount, MAX_STACK); i++)
	{
		const dtPolyRef polyRef = path[i];
		if (m_nav->isValidPolyRef(polyRef))
		{
			dtNode* node = m_tinyNodePool->getNode(polyRef);
			node->pidx = 0;
			node->id = polyRef;
			node->flags = DT_NODE_CLOSED;
			stack[nstack++] = node;

			if (n < maxNei)
			{
				neiRefs[n] = node->id;
				++n;
			}
			else
			{
				status |= DT_BUFFER_TOO_SMALL;
			}
		}
	}

	while (nstack)
	{
		// Pop front.
		dtNode* curNode = stack[0];
		for (int stackIndex = 0; stackIndex < nstack - 1; ++stackIndex)
			stack[stackIndex] = stack[stackIndex + 1];
		nstack--;
		
		// Get poly and tile.
		// The API input has been cheked already, skip checking internal data.
		const dtPolyRef curRef = curNode->id;
		const dtMeshTile* curTile = 0;
		const dtPoly* curPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(curRef, &curTile, &curPoly);

		unsigned int i = curPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(curTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours.
			if (!neighbourRef)
			{
				// store wall segment
				storeWallSegment(m_nav, curTile, curPoly, link.edge, curRef, 0, 
					resultWalls, resultRefs, resultCount, maxResult);
				continue;
			}

			// Skip if cannot alloca more nodes.
			dtNode* neighbourNode = m_tinyNodePool->getNode(neighbourRef);
			if (!neighbourNode)
				continue;
			// Skip visited.
			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;

			// Expand to neighbour
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);

			// Skip off-mesh connections.
			if (neighbourPoly->getType() != DT_POLYTYPE_GROUND)
				continue;

			// Do not advance if the polygon is excluded by the filter, or if the area enter cost is too high.
			const dtReal enterCost = filter->getAreaFixedCost(neighbourPoly->getArea());

			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly)
				|| !passLinkFilterByRef(neighbourTile, neighbourRef)
				|| enterCost > maxAreaEnterCost) 
			{
				// store wall segment
				storeWallSegment(m_nav, curTile, curPoly, link.edge, curRef, neighbourRef,
					resultWalls, resultRefs, resultCount, maxResult);

				continue;
			}

			// Find edge and calc distance to the edge.
			dtReal va[3], vb[3];
			if (!getPortalPoints(curRef, curPoly, curTile, neighbourRef, neighbourPoly, neighbourTile, va, vb))
				continue;

			// If the poly is not touching the edge to the next polygon, skip the connection it.
			dtReal tmin, tmax;
			int segMin, segMax;
			if (!dtIntersectSegmentPoly2D(va, vb, searchAreaPoly, searchAreaPolyCount, tmin, tmax, segMin, segMax))
				continue;
			if (tmin > 1.0f || tmax < 0.0f)
				continue;

			// Mark node visited, this is done before the overlap test so that
			// we will not visit the poly again if the test fails.
			neighbourNode->flags |= DT_NODE_CLOSED;
			neighbourNode->pidx = m_tinyNodePool->getNodeIdx(curNode);			

			// This poly is fine, store and advance to the poly.
			if (n < maxNei)
			{
				neiRefs[n] = neighbourRef;
				++n;
			}
			else
			{
				status |= DT_BUFFER_TOO_SMALL;
			}

			if (nstack < MAX_STACK)
			{
				stack[nstack++] = neighbourNode;
			}
		}

		// add hard edges of poly
		for (int neighbourIndex = 0; neighbourIndex < curPoly->vertCount; neighbourIndex++)
		{
			bool bStoreEdge = (curPoly->neis[neighbourIndex] == 0);
			if (curPoly->neis[neighbourIndex] & DT_EXT_LINK)
			{
				// check if external edge has valid link
				bool bConnected = false;

				unsigned int linkIdx = curPoly->firstLink;
				while (linkIdx != DT_NULL_LINK)
				{
					const dtLink& link = m_nav->getLink(curTile, linkIdx);
					linkIdx = link.next;

					if (link.edge == neighbourIndex)
					{
						bConnected = true;
						break;
					}
				}

				bStoreEdge = !bConnected;
			}

			if (bStoreEdge)
			{
				storeWallSegment(m_nav, curTile, curPoly, neighbourIndex, curRef, 0, 
					resultWalls, resultRefs, resultCount, maxResult);
			}
		}
	}

	*neiCount = n;
	return status;
}

/// @par
///
/// @p hitPos is not adjusted using the height detail data.
///
/// @p hitDist will equal the search radius if there is no wall within the 
/// radius. In this case the values of @p hitPos and @p hitNormal are
/// undefined.
///
/// The normal will become unpredicable if @p hitDist is a very small number.
///
dtStatus dtNavMeshQuery::findDistanceToWall(dtPolyRef startRef, const dtReal* centerPos, const dtReal maxRadius,
											const dtQueryFilter* filter,
											dtReal* hitDist, dtReal* hitPos, dtReal* hitNormal) const
{
	dtAssert(m_nav);
	dtAssert(m_nodePool);
	dtAssert(m_openList);
	
	// Validate input
	if (!startRef || !m_nav->isValidPolyRef(startRef))
		return DT_FAILURE | DT_INVALID_PARAM;
	
	m_nodePool->clear();
	m_openList->clear();
	
	dtNode* startNode = m_nodePool->getNode(startRef);
	dtVcopy(startNode->pos, centerPos);
	startNode->pidx = 0;
	startNode->cost = 0;
	startNode->total = 0;
	startNode->id = startRef;
	startNode->flags = DT_NODE_OPEN;
	m_openList->push(startNode);
	
	dtReal radiusSqr = dtSqr(maxRadius);
	
	dtStatus status = DT_SUCCESS;
	
	while (!m_openList->empty())
	{
		dtNode* bestNode = m_openList->pop();
		bestNode->flags &= ~DT_NODE_OPEN;
		bestNode->flags |= DT_NODE_CLOSED;
		
		// Get poly and tile.
		// The API input has been checked already, skip checking internal data.
		const dtPolyRef bestRef = bestNode->id;
		const dtMeshTile* bestTile = 0;
		const dtPoly* bestPoly = 0;
		m_nav->getTileAndPolyByRefUnsafe(bestRef, &bestTile, &bestPoly);
		if (bestPoly->vertCount == 0)
		{
			continue;
		}
		
		// Get parent poly and tile.
		dtPolyRef parentRef = 0;
		const dtMeshTile* parentTile = 0;
		const dtPoly* parentPoly = 0;
		if (bestNode->pidx)
			parentRef = m_nodePool->getNodeAtIdx(bestNode->pidx)->id;
		if (parentRef)
			m_nav->getTileAndPolyByRefUnsafe(parentRef, &parentTile, &parentPoly);
		
		// Hit test walls.
		for (int i = 0, j = (int)bestPoly->vertCount-1; i < (int)bestPoly->vertCount; j = i++)
		{
			bool isPortal = false;
			dtReal portalLeft[3];
			dtReal portalRight[3];
			
			// Skip non-solid edges.
			if (bestPoly->neis[j] & DT_EXT_LINK)
			{
				// Tile border.
				bool solid = true;
				unsigned int k = bestPoly->firstLink;
				while (k != DT_NULL_LINK)
				{
					const dtLink& link = m_nav->getLink(bestTile, k);
					k = link.next;

					if (link.edge == j)
					{
						if (link.ref != 0)
						{
							// Find neighbor tile and poly from the link.ref
							const dtMeshTile* neiTile = 0;
							const dtPoly* neiPoly = 0;
							m_nav->getTileAndPolyByRefUnsafe(link.ref, &neiTile, &neiPoly);
							
							if (filter->passFilter(link.ref, neiTile, neiPoly) && passLinkFilterByRef(neiTile, link.ref))
							{
								solid = false;
							}
							else
							{
								// Solid edge, unpack portal limits if needed.
								if (link.bmin != 0 || link.bmax != 255)
								{
									isPortal = true;
									constexpr dtReal s = 1.0/255.0;
									const dtReal tmin = link.bmin*s;
									const dtReal tmax = link.bmax*s;
									dtReal vjTmp[3], viTmp[3]; //@UE
									bestTile->verts[bestPoly->verts[j]].toWorld(bestTile->header->bmin, vjTmp); //@UE
									bestTile->verts[bestPoly->verts[i]].toWorld(bestTile->header->bmin, viTmp); //@UE
									dtVlerp(portalLeft,  vjTmp, viTmp, tmin); //@UE
									dtVlerp(portalRight, vjTmp, viTmp, tmax); //@UE
								}
							}
						}
						break;
					}
				}

				if (!solid)
					continue;
			}
			else if (bestPoly->neis[j])
			{
				// Internal edge
				const unsigned int idx = (unsigned int)(bestPoly->neis[j]-1);
				const dtPolyRef ref = m_nav->getPolyRefBase(bestTile) | idx;
				if (filter->passFilter(ref, bestTile, &bestTile->polys[idx]) && passLinkFilter(bestTile, idx))
					continue;
			}
			
			// Calc distance to the edge.
			dtReal vjBuf[3], viBuf[3]; //@UE
			const dtReal* vj; //@UE
			const dtReal* vi; //@UE
			if (!isPortal)
			{
				bestTile->verts[bestPoly->verts[j]].toWorld(bestTile->header->bmin, vjBuf); //@UE
				bestTile->verts[bestPoly->verts[i]].toWorld(bestTile->header->bmin, viBuf); //@UE
				vj = vjBuf; //@UE
				vi = viBuf; //@UE
			}
			else
			{
				vj = portalLeft;
				vi = portalRight;
			}

			dtReal tseg;
			const dtReal distSqr = dtDistancePtSegSqr2D(centerPos, vj, vi, tseg);
			
			// Edge is too far, skip.
			if (distSqr > radiusSqr)
				continue;
			
			// Hit wall, update radius.
			radiusSqr = distSqr;
			
			// Calculate hit pos.
			hitPos[0] = vj[0] + (vi[0] - vj[0])*tseg;
			hitPos[1] = vj[1] + (vi[1] - vj[1])*tseg;
			hitPos[2] = vj[2] + (vi[2] - vj[2])*tseg;
		}

		unsigned int i = bestPoly->firstLink;
		while (i != DT_NULL_LINK)
		{
			const dtLink& link = m_nav->getLink(bestTile, i);
			i = link.next;

			dtPolyRef neighbourRef = link.ref;
			// Skip invalid neighbours and do not follow back to parent.
			if (!neighbourRef || neighbourRef == parentRef)
				continue;
			
			// Expand to neighbour.
			const dtMeshTile* neighbourTile = 0;
			const dtPoly* neighbourPoly = 0;
			m_nav->getTileAndPolyByRefUnsafe(neighbourRef, &neighbourTile, &neighbourPoly);
			
			// Skip off-mesh connections.
			if (neighbourPoly->getType() != DT_POLYTYPE_GROUND)
				continue;
			
			// Calc distance to the edge.
			dtReal va[3], vb[3]; //@UE
			bestTile->verts[bestPoly->verts[link.edge]].toWorld(bestTile->header->bmin, va); //@UE
			CA_SUPPRESS(6385);
			bestTile->verts[bestPoly->verts[(link.edge+1) % bestPoly->vertCount]].toWorld(bestTile->header->bmin, vb); //@UE
			dtReal tseg;
			const dtReal distSqr = dtDistancePtSegSqr2D(centerPos, va, vb, tseg);
			
			// If the circle is not touching the next polygon, skip it.
			if (distSqr > radiusSqr)
				continue;
			
			if (!filter->passFilter(neighbourRef, neighbourTile, neighbourPoly) || !passLinkFilterByRef(neighbourTile, neighbourRef))
				continue;

			dtNode* neighbourNode = m_nodePool->getNode(neighbourRef);
			if (!neighbourNode)
			{
				status |= DT_OUT_OF_NODES;
				continue;
			}
			
			if (neighbourNode->flags & DT_NODE_CLOSED)
				continue;
			
			// Always calculate correct position on neighbor's edge,
			// skipping to wrong edge may greatly change path cost
			// (if area cost differences are more than 5x default)
			dtReal neiPos[3] = { 0.0f, 0.0f, 0.0f };
			getEdgeMidPoint(bestRef, bestPoly, bestTile,
				neighbourRef, neighbourPoly, neighbourTile,
				neiPos);
			
			const dtReal total = bestNode->total + dtVdist(bestNode->pos, neiPos);
			
			// The node is already in open list and the new result is worse, skip.
			if ((neighbourNode->flags & DT_NODE_OPEN) && total >= neighbourNode->total)
				continue;
			
			neighbourNode->id = neighbourRef;
			neighbourNode->flags = (neighbourNode->flags & ~DT_NODE_CLOSED);
			neighbourNode->pidx = m_nodePool->getNodeIdx(bestNode);
			neighbourNode->total = total;
			dtVcopy(neighbourNode->pos, neiPos);
				
			if (neighbourNode->flags & DT_NODE_OPEN)
			{
				m_openList->modify(neighbourNode);
			}
			else
			{
				neighbourNode->flags |= DT_NODE_OPEN;
				m_openList->push(neighbourNode);
			}
		}
	}
	
	// Calc hit normal.
	dtVsub(hitNormal, centerPos, hitPos);
	dtVnormalize(hitNormal);
	
	*hitDist = dtSqrt(radiusSqr);
	
	return status;
}

bool dtNavMeshQuery::isValidPolyRef(dtPolyRef ref, const dtQueryFilter* filter) const
{
	const dtMeshTile* tile = 0;
	const dtPoly* poly = 0;
	dtStatus status = m_nav->getTileAndPolyByRef(ref, &tile, &poly);

	// should be able to get the polygon if the boundary is valid
	return !dtStatusFailed(status)
		// and should pass all filters
		&& filter->passFilter(ref, tile, poly) && passLinkFilterByRef(tile, ref);
}

/// @par
///
/// The closed list is the list of polygons that were fully evaluated during 
/// the last navigation graph search. (A* or Dijkstra)
/// 
bool dtNavMeshQuery::isInClosedList(dtPolyRef ref) const
{
	if (!m_nodePool) return false;
	const dtNode* node = m_nodePool->findNode(ref);
	return node && node->flags & DT_NODE_CLOSED;
}