// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "Detour/DetourNode.h"
#include "Detour/DetourAssert.h"
#include "Detour/DetourCommon.h"

#if USE_64BIT_ADDRESS

inline unsigned int dtHashRef(dtPolyRef a)
{
	a += ~(a<<31);
	a ^=  (a>>20);
	a +=  (a<<6);
	a ^=  (a>>12);
	a += ~(a<<22);
	a ^=  (a>>32);
	return (unsigned int)a;
}

#else

inline unsigned int dtHashRef(dtPolyRef a)
{
	a += ~(a<<15);
	a ^=  (a>>10);
	a +=  (a<<3);
	a ^=  (a>>6);
	a += ~(a<<11);
	a ^=  (a>>16);
	return (unsigned int)a;
}

#endif // USE_64BIT_ADDRESS

//////////////////////////////////////////////////////////////////////////////////////////
dtNodePool::dtNodePool(int maxNodes, int hashSize) :
	m_nodes(0),
	m_first(0),
	m_next(0),
	m_maxNodes(maxNodes),
	m_hashSize(hashSize),
	//@UE BEGIN
	m_maxRuntimeNodes(maxNodes),
	//@UE END
	m_nodeCount(0)
{
	dtAssert(dtNextPow2(m_hashSize) == (unsigned int)m_hashSize);
	dtAssert(m_maxNodes > 0);

	m_nodes = (dtNode*)dtAlloc(sizeof(dtNode)*m_maxNodes, DT_ALLOC_PERM_NODE_POOL);
	m_next = (dtNodeIndex*)dtAlloc(sizeof(dtNodeIndex)*m_maxNodes, DT_ALLOC_PERM_NODE_POOL);
	m_first = (dtNodeIndex*)dtAlloc(sizeof(dtNodeIndex)*hashSize, DT_ALLOC_PERM_NODE_POOL);

	dtAssert(m_nodes);
	dtAssert(m_next);
	dtAssert(m_first);

	memset(m_first, 0xff, sizeof(dtNodeIndex)*m_hashSize);
	memset(m_next, 0xff, sizeof(dtNodeIndex)*m_maxNodes);
}

dtNodePool::~dtNodePool()
{
	dtFree(m_nodes, DT_ALLOC_PERM_NODE_POOL);
	dtFree(m_next, DT_ALLOC_PERM_NODE_POOL);
	dtFree(m_first, DT_ALLOC_PERM_NODE_POOL);
}

void dtNodePool::clear()
{
	memset(m_first, 0xff, sizeof(dtNodeIndex)*m_hashSize);
	m_nodeCount = 0;
}

dtNode* dtNodePool::findNode(dtPolyRef id)
{
	unsigned int bucket = dtHashRef(id) & (m_hashSize-1);
	dtNodeIndex i = m_first[bucket];
	while (i != DT_NULL_IDX)
	{
		if (m_nodes[i].id == id)
			return &m_nodes[i];
		i = m_next[i];
	}
	return 0;
}

dtNode* dtNodePool::getNode(dtPolyRef id)
{
	unsigned int bucket = dtHashRef(id) & (m_hashSize-1);
	dtNodeIndex i = m_first[bucket];
	dtNode* node = 0;
	while (i != DT_NULL_IDX)
	{
		if (m_nodes[i].id == id)
			return &m_nodes[i];
		i = m_next[i];
	}

	//@UE BEGIN
	if (m_nodeCount >= getMaxRuntimeNodes())
	//@UE END
		return 0;
	
	i = (dtNodeIndex)m_nodeCount;
	m_nodeCount++;
	
	// Init node
	node = &m_nodes[i];
	node->pidx = 0;
	node->cost = 0;
	node->total = 0;
	node->id = id;
	node->flags = 0;
	
	m_next[i] = m_first[bucket];
	m_first[bucket] = i;
	
	return node;
}


//////////////////////////////////////////////////////////////////////////////////////////
dtNodeQueue::dtNodeQueue(int n) :
	m_heap(0),
	m_capacity(n),
	m_size(0)
{
	dtAssert(m_capacity > 0);
	
	m_heap = (dtNode**)dtAlloc(sizeof(dtNode*)*(m_capacity+1), DT_ALLOC_PERM_NODE_POOL);
	dtAssert(m_heap);
}

dtNodeQueue::~dtNodeQueue()
{
	dtFree(m_heap, DT_ALLOC_PERM_NODE_POOL);
}

void dtNodeQueue::bubbleUp(int i, dtNode* node)
{
	int parent = (i-1)/2;
	// note: (index > 0) means there is a parent
	while ((i > 0) && (m_heap[parent]->total > node->total))
	{
		m_heap[i] = m_heap[parent];
		i = parent;
		parent = (i-1)/2;
	}
	m_heap[i] = node;
}

void dtNodeQueue::trickleDown(int i, dtNode* node)
{
	int child = (i*2)+1;
	while (child < m_size)
	{
		if (((child+1) < m_size) && 
			(m_heap[child]->total > m_heap[child+1]->total))
		{
			child++;
		}
		m_heap[i] = m_heap[child];
		i = child;
		child = (i*2)+1;
	}
	bubbleUp(i, node);
}
