// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DetourCrowd/DetourCrowd.h"
#include "DetourCrowd/DetourProximityGrid.h"
#define _USE_MATH_DEFINES
#include "Detour/DetourAssert.h"


dtCrowd* dtAllocCrowd()
{
	void* mem = dtAlloc(sizeof(dtCrowd), DT_ALLOC_PERM_CROWD);
	if (!mem) return 0;
	return new(mem) dtCrowd;
}

void dtFreeCrowd(dtCrowd* ptr)
{
	if (!ptr) return;
	ptr->~dtCrowd();
	dtFree(ptr, DT_ALLOC_PERM_CROWD);
}


static const int MAX_ITERS_PER_UPDATE = 100;

static const int MAX_PATHQUEUE_NODES = 4096;
static const int MAX_COMMON_NODES = 512;
static const int DT_WALKABLE_AREA = 63;

inline dtReal tween(const dtReal t, const dtReal t0, const dtReal t1)
{
	return dtClamp((t-t0) / (t1-t0), 0.0f, 1.0f);
}

static void integrate(dtCrowdAgent* ag, const dtReal dt)
{
	// Fake dynamic constraint.
	const dtReal maxDelta = ag->params.maxAcceleration * dt;
	dtReal dv[3];
	dtVsub(dv, ag->nvel, ag->vel);
	dtReal ds = dtVlen(dv);
	if (ds > maxDelta)
		dtVscale(dv, dv, maxDelta/ds);
	dtVadd(ag->vel, ag->vel, dv);
	
	// Integrate
	if (dtVlen(ag->vel) > 0.0001f)
		dtVmad(ag->npos, ag->npos, ag->vel, dt);
	else
		dtVset(ag->vel,0,0,0);
}

static bool overOffmeshConnection(const dtCrowdAgent* ag, const dtReal radius)
{
	if (!ag->ncorners)
		return false;
	
	const bool offMeshConnection = (ag->cornerFlags[ag->ncorners-1] & DT_STRAIGHTPATH_OFFMESH_CONNECTION) ? true : false;
	if (offMeshConnection)
	{
		const dtReal distSq = dtVdist2DSqr(ag->npos, &ag->cornerVerts[(ag->ncorners-1)*3]);
		if (distSq < radius*radius)
			return true;
	}
	
	return false;
}

static dtReal getDistanceToGoal(const dtCrowdAgent* ag, const dtReal range)
{
	if (!ag->ncorners)
		return range;
	
	const bool endOfPath = (ag->cornerFlags[ag->ncorners-1] & DT_STRAIGHTPATH_END) ? true : false;
	if (endOfPath)
		return dtMin(dtVdist2D(ag->npos, &ag->cornerVerts[(ag->ncorners-1)*3]), range);
	
	return range;
}

static void calcSmoothSteerDirection(const dtCrowdAgent* ag, dtReal* dir)
{
	if (!ag->ncorners)
	{
		dtVset(dir, 0,0,0);
		return;
	}
	
	const int ip0 = 0;
	const int ip1 = dtMin(1, ag->ncorners-1);
	const dtReal* p0 = &ag->cornerVerts[ip0*3];
	const dtReal* p1 = &ag->cornerVerts[ip1*3];
	
	dtReal dir0[3], dir1[3];
	dtVsub(dir0, p0, ag->npos);
	dtVsub(dir1, p1, ag->npos);
	dir0[1] = 0;
	dir1[1] = 0;
	
	dtReal len0 = dtVlen(dir0);
	dtReal len1 = dtVlen(dir1);
	if (len1 > 0.001f)
		dtVscale(dir1,dir1,1.0f/len1);
	
	dir[0] = dir0[0] - dir1[0]*len0*0.5f;
	dir[1] = 0;
	dir[2] = dir0[2] - dir1[2]*len0*0.5f;
	
	dtVnormalize(dir);
}

static void calcStraightSteerDirection(const dtCrowdAgent* ag, dtReal* dir)
{
	if (!ag->ncorners)
	{
		dtVset(dir, 0,0,0);
		return;
	}
	dtVsub(dir, &ag->cornerVerts[0], ag->npos);
	dir[1] = 0;
	dtVnormalize(dir);
}

static int addNeighbour(const int idx, const dtReal dist,
						dtCrowdNeighbour* neis, const int nneis, const int maxNeis)
{
	// Insert neighbour based on the distance.
	dtCrowdNeighbour* nei = 0;
	if (!nneis)
	{
		nei = &neis[nneis];
	}
	else if (dist >= neis[nneis-1].dist)
	{
		if (nneis >= maxNeis)
			return nneis;
		nei = &neis[nneis];
	}
	else
	{
		int i;
		for (i = 0; i < nneis; ++i)
			if (dist <= neis[i].dist)
				break;
		
		const int tgt = i+1;
		const int n = dtMin(nneis-i, maxNeis-tgt);
		
		dtAssert(tgt+n <= maxNeis);
		
		if (n > 0)
			memmove(&neis[tgt], &neis[i], sizeof(dtCrowdNeighbour)*n);
		nei = &neis[i];
	}
	
	memset(nei, 0, sizeof(dtCrowdNeighbour));
	
	nei->idx = idx;
	nei->dist = dist;
	
	return dtMin(nneis+1, maxNeis);
}

static int getNeighbours(const dtReal* pos, const dtReal height, const dtReal range,
						 const dtCrowdAgent* skip, dtCrowdNeighbour* result, const int maxResult,
						 dtCrowdAgent** agents, const int /*nagents*/, dtProximityGrid* grid)
{
	int n = 0;
	
	static const int MAX_NEIS = 32;
	unsigned short ids[MAX_NEIS];
	int nids = grid->queryItems(pos[0]-range, pos[2]-range,
								pos[0]+range, pos[2]+range,
								ids, MAX_NEIS);
	
	for (int i = 0; i < nids; ++i)
	{
		const dtCrowdAgent* ag = agents[ids[i]];
		
		if (ag == skip) continue;
		
		// Check for overlap.
		dtReal diff[3];
		dtVsub(diff, pos, ag->npos);
		if (dtAbs(diff[1]) >= (height+ag->params.height)/2.0f)
			continue;
		diff[1] = 0;
		const dtReal distSqr = dtVlenSqr(diff);
		if (distSqr > dtSqr(range))
			continue;

		// [UE] add only when avoidance group allows it
		const bool bDontAvoid = (skip->params.groupsToIgnore & ag->params.avoidanceGroup) || !(skip->params.groupsToAvoid & ag->params.avoidanceGroup);
		if (bDontAvoid)
			continue;
		
		n = addNeighbour(ids[i], distSqr, result, n, maxResult);
	}
	return n;
}

static int addToOptQueue(dtCrowdAgent* newag, dtCrowdAgent** agents, const int nagents, const int maxAgents)
{
	// Insert neighbour based on greatest time.
	int slot = 0;
	if (!nagents)
	{
		slot = nagents;
	}
	else if (newag->topologyOptTime <= agents[nagents - 1]->topologyOptTime)
	{
		if (nagents >= maxAgents)
			return nagents;
		slot = nagents;
	}
	else
	{
		int i;
		for (i = 0; i < nagents; ++i)
		if (newag->topologyOptTime >= agents[i]->topologyOptTime)
				break;
		
		const int tgt = i+1;
		const int n = dtMin(nagents - i, maxAgents - tgt);
		
		dtAssert(tgt+n <= maxAgents);
		
		if (n > 0)
			memmove(&agents[tgt], &agents[i], sizeof(dtCrowdAgent*)*n);
		slot = i;
	}
	
	agents[slot] = newag;
	
	return dtMin(nagents + 1, maxAgents);
}

static int addToPathQueue(dtCrowdAgent* newag, dtCrowdAgent** agents, const int nagents, const int maxAgents)
{
	// Insert neighbour based on greatest time.
	int slot = 0;
	if (!nagents)
	{
		slot = nagents;
	}
	else if (newag->targetReplanTime <= agents[nagents - 1]->targetReplanTime)
	{
		if (nagents >= maxAgents)
			return nagents;
		slot = nagents;
	}
	else
	{
		int i;
		for (i = 0; i < nagents; ++i)
		if (newag->targetReplanTime >= agents[i]->targetReplanTime)
				break;
		
		const int tgt = i+1;
		const int n = dtMin(nagents - i, maxAgents - tgt);
		
		dtAssert(tgt+n <= maxAgents);
		
		if (n > 0)
			memmove(&agents[tgt], &agents[i], sizeof(dtCrowdAgent*)*n);
		slot = i;
	}
	
	agents[slot] = newag;
	
	return dtMin(nagents + 1, maxAgents);
}

/**
@class dtCrowd
@par

This is the core class of the @ref crowd module.  See the @ref crowd documentation for a summary
of the crowd features.

A common method for setting up the crowd is as follows:

-# Allocate the crowd using #dtAllocCrowd.
-# Initialize the crowd using #init().
-# Set the avoidance configurations using #setObstacleAvoidanceParams().
-# Add agents using #addAgent() and make an initial movement request using #requestMoveTarget().

A common process for managing the crowd is as follows:

-# Call #update() to allow the crowd to manage its agents.
-# Retrieve agent information using #getActiveAgents().
-# Make movement requests using #requestMoveTarget() when movement goal changes.
-# Repeat every frame.

Some agent configuration settings can be updated using #updateAgentParameters().  But the crowd owns the
agent position.  So it is not possible to update an active agent's position.  If agent position
must be fed back into the crowd, the agent must be removed and re-added.

Notes: 

- Path related information is available for newly added agents only after an #update() has been
  performed.
- Agent objects are kept in a pool and re-used.  So it is important when using agent objects to check the value of
  #dtCrowdAgent::active to determine if the agent is actually in use or not.
- This class is meant to provide 'local' movement. There is a limit of 256 polygons in the path corridor.  
  So it is not meant to provide automatic pathfinding services over long distances.

@see dtAllocCrowd(), dtFreeCrowd(), init(), dtCrowdAgent

*/

dtCrowd::dtCrowd() :
	m_maxAgents(0),
	m_numActiveAgents(0),
	m_agents(0),
	m_activeAgents(0),
	m_agentAnims(0),
	m_obstacleQuery(0),
	m_grid(0),
	m_pathResult(0),
	m_maxAgentRadius(0),
	m_agentStateCheckInterval(1.0f),
	m_pathOffsetRadiusMultiplier(1.0f),
	m_maxPathResult(0),
	m_velocitySampleCount(0),
	m_navquery(0),
	m_raycastSingleArea(0),
	m_keepOffmeshConnections(0),
	m_earlyReachTest(0)
{
}

dtCrowd::~dtCrowd()
{
	purge();
}

void dtCrowd::purge()
{
	for (int i = 0; i < m_maxAgents; ++i)
		m_agents[i].~dtCrowdAgent();
	dtFree(m_agents, DT_ALLOC_PERM_CROWD);
	m_agents = 0;
	m_maxAgents = 0;

	dtFree(m_activeAgents, DT_ALLOC_PERM_CROWD);
	m_activeAgents = 0;
	m_numActiveAgents = 0;

	dtFree(m_agentAnims, DT_ALLOC_PERM_CROWD);
	m_agentAnims = 0;
	
	dtFree(m_pathResult, DT_ALLOC_PERM_CROWD);
	m_pathResult = 0;
	
	dtFreeProximityGrid(m_grid);
	m_grid = 0;

	dtFreeObstacleAvoidanceQuery(m_obstacleQuery);
	m_obstacleQuery = 0;
	
	dtFreeNavMeshQuery(m_navquery);
	m_navquery = 0;
}

/// @par
///
/// May be called more than once to purge and re-initialize the crowd.
bool dtCrowd::init(const int maxAgents, const dtReal maxAgentRadius, dtNavMesh* nav)
{
	purge();
	
	m_maxAgents = maxAgents;
	m_maxAgentRadius = maxAgentRadius;
	m_numActiveAgents = 0;

	dtVset(m_ext, m_maxAgentRadius*2.0f,m_maxAgentRadius*1.5f,m_maxAgentRadius*2.0f);
	
	m_grid = dtAllocProximityGrid();
	if (!m_grid)
		return false;
	if (!m_grid->init(m_maxAgents*4, maxAgentRadius*3))
		return false;

	// [UE] moved avoidance query init to separate function

	// Allocate temp buffer for merging paths.
	m_maxPathResult = 256;
	m_pathResult = (dtPolyRef*)dtAlloc(sizeof(dtPolyRef)*m_maxPathResult, DT_ALLOC_PERM_CROWD);
	if (!m_pathResult)
		return false;
	
	if (!m_pathq.init(m_maxPathResult, MAX_PATHQUEUE_NODES, nav))
		return false;
	
	m_agents = (dtCrowdAgent*)dtAlloc(sizeof(dtCrowdAgent)*m_maxAgents, DT_ALLOC_PERM_CROWD);
	if (!m_agents)
		return false;
	
	m_activeAgents = (dtCrowdAgent**)dtAlloc(sizeof(dtCrowdAgent*)*m_maxAgents, DT_ALLOC_PERM_CROWD);
	if (!m_activeAgents)
		return false;

	m_agentAnims = (dtCrowdAgentAnimation*)dtAlloc(sizeof(dtCrowdAgentAnimation)*m_maxAgents, DT_ALLOC_PERM_CROWD);
	if (!m_agentAnims)
		return false;
	
	for (int i = 0; i < m_maxAgents; ++i)
	{
		new(&m_agents[i]) dtCrowdAgent();
		m_agents[i].active = 0;
		if (!m_agents[i].corridor.init(m_maxPathResult))
			return false;
	}

	for (int i = 0; i < m_maxAgents; ++i)
	{
		m_agentAnims[i].active = 0;
	}

	for (int i = 0; i < DT_MAX_AREAS; i++)
	{
		m_raycastFilter.setAreaCost(i, DT_UNWALKABLE_POLY_COST);
	}

	// The navquery is mostly used for local searches, no need for large node pool.
	m_navquery = dtAllocNavMeshQuery();
	if (!m_navquery)
		return false;
	if (dtStatusFailed(m_navquery->init(nav, MAX_COMMON_NODES)))
		return false;
	
	m_sharedBoundary.Initialize();
	m_separationDirFilter = -1.0f;

	return true;
}

bool dtCrowd::initAvoidance(const int maxNeighbors, const int maxWalls, const int maxCustomPatterns)
{
	m_obstacleQuery = dtAllocObstacleAvoidanceQuery();
	if (!m_obstacleQuery)
		return false;
	if (!m_obstacleQuery->init(maxNeighbors, maxWalls, maxCustomPatterns))
		return false;

	// Init obstacle query params.
	memset(m_obstacleQueryParams, 0, sizeof(m_obstacleQueryParams));
	for (int i = 0; i < DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS; ++i)
	{
		dtObstacleAvoidanceParams* params = &m_obstacleQueryParams[i];
		params->velBias = 0.4f;
		params->weightDesVel = 2.0f;
		params->weightCurVel = 0.75f;
		params->weightSide = 0.75f;
		params->weightToi = 2.5f;
		params->horizTime = 2.5f;
		params->patternIdx = 0xff;
		params->adaptiveDivs = 7;
		params->adaptiveRings = 2;
		params->adaptiveDepth = 5;
	}

	return true;
}

void dtCrowd::setObstacleAvoidanceParams(const int idx, const dtObstacleAvoidanceParams* params)
{
	if (idx >= 0 && idx < DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS)
		memcpy(&m_obstacleQueryParams[idx], params, sizeof(dtObstacleAvoidanceParams));
}

const dtObstacleAvoidanceParams* dtCrowd::getObstacleAvoidanceParams(const int idx) const
{
	if (idx >= 0 && idx < DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS)
		return &m_obstacleQueryParams[idx];
	return 0;
}

void dtCrowd::setObstacleAvoidancePattern(int idx, const dtReal* angles, const dtReal* radii, int nsamples)
{
	m_obstacleQuery->setCustomSamplingPattern(idx, angles, radii, nsamples);
}

bool dtCrowd::getObstacleAvoidancePattern(int idx, dtReal* angles, dtReal* radii, int* nsamples)
{
	return m_obstacleQuery->getCustomSamplingPattern(idx, angles, radii, nsamples);
}

const int dtCrowd::getAgentCount() const
{
	return m_maxAgents;
}

/// @par
/// 
/// agents in the pool may not be in use.  Check #dtCrowdAgent.active before using the returned object.
const dtCrowdAgent* dtCrowd::getAgent(const int idx)
{
	return &m_agents[idx];
}

void dtCrowd::updateAgentParameters(const int idx, const dtCrowdAgentParams& params)
{
	if (idx < 0 || idx > m_maxAgents)
		return;
	
	// this line used to be a memcopy call, but it was breaking logic of TSharedPtr that dtCrowdAgentParams has
	m_agents[idx].params = params;
}

// [UE] multiple filter support
bool dtCrowd::updateAgentFilter(const int idx, const dtQueryFilter* filter)
{
	if (idx < 0 || idx > m_maxAgents)
	{
		return false;
	}

	// try to match with existing
	for (int i = 0; i < DT_CROWD_MAX_FILTERS; i++)
	{
		const bool bMatching = filter->equals(m_filters[i]);
		if (bMatching)
		{
			m_agents[idx].params.filter = (unsigned char)i;
			return true;
		}
	}

	// need to add new filter, find free slot
	unsigned char used[DT_CROWD_MAX_FILTERS] = { 0 };
	for (int i = 0; i < m_maxAgents; i++)
	{
		if (m_agents[i].active && i != idx)
		{
			used[m_agents[i].params.filter] = 1;
		}
	}

	for (int i = 0; i < DT_CROWD_MAX_FILTERS; i++)
	{
		if (used[i] == 0)
		{
			m_filters[i].copyFrom(filter);
			m_agents[idx].params.filter = (unsigned char)i;
			return true;
		}
	}

	return false;
}

/// @par
///
/// The agent's position will be constrained to the surface of the navigation mesh.
int dtCrowd::addAgent(const dtReal* pos, const dtCrowdAgentParams& params, const dtQueryFilter* filter)
{
	// Find empty slot.
	int idx = -1;
	for (int i = 0; i < m_maxAgents; ++i)
	{
		if (!m_agents[i].active)
		{
			idx = i;
			break;
		}
	}
	if (idx == -1)
		return -1;
	
	dtCrowdAgent* ag = &m_agents[idx];
	
	// [UE] multiple filter support
	const bool bStoredFilter = updateAgentFilter(idx, filter);
	if (!bStoredFilter)
	{
		return -1;
	}

	// Find nearest position on navmesh and place the agent there.
	dtReal nearest[3];
	dtPolyRef ref;
	m_navquery->updateLinkFilter(params.linkFilter.Get());
	m_navquery->findNearestPoly(pos, m_ext, &m_filters[ag->params.filter], &ref, nearest);
	
	ag->corridor.reset(ref, nearest);
	ag->boundary.reset();

	updateAgentParameters(idx, params);
	
	ag->topologyOptTime = 0;
	ag->targetReplanTime = 0;
	ag->nneis = 0;
	ag->ncorners = 0;
	
	dtVset(ag->dvel, 0,0,0);
	dtVset(ag->nvel, 0,0,0);
	dtVset(ag->vel, 0,0,0);
	dtVcopy(ag->npos, nearest);
	
	ag->desiredSpeed = 0;

	if (ref)
		ag->state = DT_CROWDAGENT_STATE_WALKING;
	else
		ag->state = DT_CROWDAGENT_STATE_INVALID;
	
	ag->targetState = DT_CROWDAGENT_TARGET_NONE;
	
	ag->active = 1;

	return idx;
}

void dtCrowd::updateAgentState(const int idx, bool repath)
{
	if (idx >= 0 && idx < m_maxAgents)
	{
		dtCrowdAgent* ag = &m_agents[idx];
		dtCrowdAgentAnimation* anim = &m_agentAnims[idx];

		if (anim->active)
		{
			anim->active = 0;

			if (m_keepOffmeshConnections)
			{
				const dtReal distToStartSq = dtVdistSqr(ag->npos, anim->startPos);
				const dtReal distToEndSq = dtVdistSqr(ag->npos, anim->endPos);
				if (distToEndSq < distToStartSq)
				{
					ag->corridor.pruneOffmeshConenction(anim->polyRef);
				}
			}
		}

		if (ag->active)
		{
			if (repath)
			{
				// switch to invalid state and force update in next tick
				ag->state = DT_CROWDAGENT_STATE_INVALID;
				ag->targetReplanTime = m_agentStateCheckInterval;
			}
			else
			{
				ag->state = DT_CROWDAGENT_STATE_WALKING;
			}
		}
	}
}

/// @par
///
/// The agent is deactivated and will no longer be processed.  Its #dtCrowdAgent object
/// is not removed from the pool.  It is marked as inactive so that it is available for reuse.
void dtCrowd::removeAgent(const int idx)
{
	if (idx >= 0 && idx < m_maxAgents)
	{
		m_agents[idx].active = 0;
	}
}

bool dtCrowd::requestMoveTargetReplan(const int idx, dtPolyRef ref, const dtReal* pos)
{
	if (idx < 0 || idx > m_maxAgents)
		return false;
	
	dtCrowdAgent* ag = &m_agents[idx];
	
	// Initialize request.
	ag->targetRef = ref;
	dtVcopy(ag->targetPos, pos);
	ag->targetPathqRef = DT_PATHQ_INVALID;
	ag->targetReplan = true;
	if (ag->targetRef)
		ag->targetState = DT_CROWDAGENT_TARGET_REQUESTING;
	else
		ag->targetState = DT_CROWDAGENT_TARGET_FAILED;
	
	return true;
}

/// @par
/// 
/// This method is used when a new target is set.
/// 
/// The position will be constrained to the surface of the navigation mesh.
///
/// The request will be processed during the next #update().
bool dtCrowd::requestMoveTarget(const int idx, dtPolyRef ref, const dtReal* pos)
{
	if (idx < 0 || idx > m_maxAgents)
		return false;
	if (!ref)
		return false;

	dtCrowdAgent* ag = &m_agents[idx];
	
	// Initialize request.
	ag->targetRef = ref;
	dtVcopy(ag->targetPos, pos);
	ag->targetPathqRef = DT_PATHQ_INVALID;
	ag->targetReplan = false;
	if (ag->targetRef)
		ag->targetState = DT_CROWDAGENT_TARGET_REQUESTING;
	else
		ag->targetState = DT_CROWDAGENT_TARGET_FAILED;

	return true;
}

bool dtCrowd::requestMoveVelocity(const int idx, const dtReal* vel)
{
	if (idx < 0 || idx > m_maxAgents)
		return false;
	
	dtCrowdAgent* ag = &m_agents[idx];
	
	// Initialize request.
	ag->targetRef = 0;
	dtVcopy(ag->targetPos, vel);
	ag->targetPathqRef = DT_PATHQ_INVALID;
	ag->targetReplan = false;
	ag->targetState = DT_CROWDAGENT_TARGET_VELOCITY;
	
	return true;
}

bool dtCrowd::resetMoveTarget(const int idx)
{
	if (idx < 0 || idx > m_maxAgents)
		return false;
	
	dtCrowdAgent* ag = &m_agents[idx];
	
	// Initialize request.
	ag->targetRef = 0;
	dtVset(ag->targetPos, 0,0,0);
	ag->targetPathqRef = DT_PATHQ_INVALID;
	ag->targetReplan = false;
	ag->targetState = DT_CROWDAGENT_TARGET_NONE;
	
	return true;
}

bool dtCrowd::setAgentWaiting(const int idx)
{
	if (idx < 0 || idx > m_maxAgents)
		return false;

	dtCrowdAgent* ag = &m_agents[idx];
	ag->state = DT_CROWDAGENT_STATE_WAITING;

	return true;
}

bool dtCrowd::setAgentBackOnLink(const int idx)
{
	if (idx < 0 || idx > m_maxAgents)
		return false;

	dtCrowdAgent* ag = &m_agents[idx];
	dtCrowdAgentAnimation* anim = &m_agentAnims[idx];
	
	if (anim->active)
	{
		ag->state = DT_CROWDAGENT_STATE_OFFMESH;
		return true;
	}

	return false;
}

bool dtCrowd::resetAgentVelocity(const int idx)
{
	if (idx < 0 || idx > m_maxAgents)
		return false;

	dtCrowdAgent* ag = &m_agents[idx];
	dtVset(ag->nvel, 0, 0, 0);
	dtVset(ag->vel, 0, 0, 0);
	dtVset(ag->dvel, 0, 0, 0);
	return true;
}

int dtCrowd::getActiveAgents(dtCrowdAgent** agents, const int maxAgents)
{
	int n = 0;
	for (int i = 0; i < m_maxAgents; ++i)
	{
		if (!m_agents[i].active) continue;
		if (n < maxAgents)
			agents[n++] = &m_agents[i];
	}
	return n;
}

int dtCrowd::cacheActiveAgents()
{
	m_numActiveAgents = getActiveAgents(m_activeAgents, m_maxAgents);
	return m_numActiveAgents;
}

void dtCrowd::updateMoveRequest(const dtReal /*dt*/)
{
	const int PATH_MAX_AGENTS = 8;
	dtCrowdAgent* queue[PATH_MAX_AGENTS];
	int nqueue = 0;
	
	// Fire off new requests.
	for (int i = 0; i < m_maxAgents; ++i)
	{
		dtCrowdAgent* ag = &m_agents[i];
		if (!ag->active)
			continue;
		if (ag->state == DT_CROWDAGENT_STATE_INVALID)
			continue;
		if (ag->targetState == DT_CROWDAGENT_TARGET_NONE || ag->targetState == DT_CROWDAGENT_TARGET_VELOCITY)
			continue;

		if (ag->targetState == DT_CROWDAGENT_TARGET_REQUESTING)
		{
			const dtPolyRef* path = ag->corridor.getPath();
			const int npath = ag->corridor.getPathCount();
			dtAssert(npath);

			static const int MAX_RES = 32;
			dtReal reqPos[3];
			dtPolyRef reqPath[MAX_RES];	// The path to the request location
			int reqPathCount = 0;

			// Quick seach towards the goal.
			static const int MAX_ITER = 20;
			m_navquery->updateLinkFilter(ag->params.linkFilter.Get());

			const dtReal costLimit = DT_REAL_MAX; //@UE
			const bool requireNavigableEndLocation = true; //@UE
			m_navquery->initSlicedFindPath(path[0], ag->targetRef, ag->npos, ag->targetPos, costLimit, requireNavigableEndLocation, &m_filters[ag->params.filter]); //@UE
			m_navquery->updateSlicedFindPath(MAX_ITER, 0);
			dtStatus status = 0;
			if (ag->targetReplan) // && npath > 10)
			{
				// Try to use existing steady path during replan if possible.
				status = m_navquery->finalizeSlicedFindPathPartial(path, npath, reqPath, &reqPathCount, MAX_RES);
			}
			else
			{
				// Try to move towards target when goal changes.
				status = m_navquery->finalizeSlicedFindPath(reqPath, &reqPathCount, MAX_RES);
			}

			if (!dtStatusFailed(status) && reqPathCount > 0)
			{
				// In progress or succeed.
				if (reqPath[reqPathCount-1] != ag->targetRef)
				{
					// Partial path, constrain target position inside the last polygon.
					status = m_navquery->closestPointOnPoly(reqPath[reqPathCount-1], ag->targetPos, reqPos);
					if (dtStatusFailed(status))
						reqPathCount = 0;
				}
				else
				{
					dtVcopy(reqPos, ag->targetPos);
				}
			}
			else
			{
				reqPathCount = 0;
			}
				
			if (!reqPathCount)
			{
				// Could not find path, start the request from current location.
				dtVcopy(reqPos, ag->npos);
				reqPath[0] = path[0];
				reqPathCount = 1;
			}

			ag->corridor.setCorridor(reqPos, reqPath, reqPathCount);
			ag->boundary.reset();

			if (reqPath[reqPathCount-1] == ag->targetRef)
			{
				ag->targetState = DT_CROWDAGENT_TARGET_VALID;
				ag->targetReplanTime = 0.0;
			}
			else
			{
				// The path is longer or potentially unreachable, full plan.
				ag->targetState = DT_CROWDAGENT_TARGET_WAITING_FOR_QUEUE;
			}
		}
		
		if (ag->targetState == DT_CROWDAGENT_TARGET_WAITING_FOR_QUEUE)
		{
			nqueue = addToPathQueue(ag, queue, nqueue, PATH_MAX_AGENTS);
		}
	}

	for (int i = 0; i < nqueue; ++i)
	{
		dtCrowdAgent* ag = queue[i];
		const dtReal costLimit = DT_REAL_MAX; //@UE
		const bool requireNavigableEndLocation = true; //@UE
		ag->targetPathqRef = m_pathq.request(ag->corridor.getLastPoly(), ag->targetRef,
			ag->corridor.getTarget(), ag->targetPos, costLimit, requireNavigableEndLocation, &m_filters[ag->params.filter], ag->params.linkFilter); //@UE
		if (ag->targetPathqRef != DT_PATHQ_INVALID)
			ag->targetState = DT_CROWDAGENT_TARGET_WAITING_FOR_PATH;
	}

	
	// Update requests.
	m_pathq.update(MAX_ITERS_PER_UPDATE);

	dtStatus status;

	// Process path results.
	for (int i = 0; i < m_maxAgents; ++i)
	{
		dtCrowdAgent* ag = &m_agents[i];
		if (!ag->active)
			continue;
		if (ag->targetState == DT_CROWDAGENT_TARGET_NONE || ag->targetState == DT_CROWDAGENT_TARGET_VELOCITY)
			continue;
		
		if (ag->targetState == DT_CROWDAGENT_TARGET_WAITING_FOR_PATH)
		{
			// Poll path queue.
			status = m_pathq.getRequestStatus(ag->targetPathqRef);
			if (dtStatusFailed(status))
			{
				// Path find failed, retry if the target location is still valid.
				ag->targetPathqRef = DT_PATHQ_INVALID;
				if (ag->targetRef)
					ag->targetState = DT_CROWDAGENT_TARGET_REQUESTING;
				else
					ag->targetState = DT_CROWDAGENT_TARGET_FAILED;
				ag->targetReplanTime = 0.0;
			}
			else if (dtStatusSucceed(status))
			{
				const dtPolyRef* path = ag->corridor.getPath();
				const int npath = ag->corridor.getPathCount();
				dtAssert(npath);
				
				// Apply results.
				dtReal targetPos[3];
				dtVcopy(targetPos, ag->targetPos);
				
				dtPolyRef* res = m_pathResult;
				bool valid = true;
				int nres = 0;
				status = m_pathq.getPathResult(ag->targetPathqRef, res, &nres, m_maxPathResult);
				if (dtStatusFailed(status) || !nres)
					valid = false;
				
				// Merge result and existing path.
				// The agent might have moved whilst the request is
				// being processed, so the path may have changed.
				// We assume that the end of the path is at the same location
				// where the request was issued.
				
				// The last ref in the old path should be the same as
				// the location where the request was issued..
				if (valid && path[npath-1] != res[0])
					valid = false;
				
				if (valid)
				{
					// Put the old path infront of the old path.
					if (npath > 1)
					{
						// Make space for the old path.
						if ((npath-1)+nres > m_maxPathResult)
							nres = m_maxPathResult - (npath-1);
						
						memmove(res+npath-1, res, sizeof(dtPolyRef)*nres);
						// Copy old path in the beginning.
						memcpy(res, path, sizeof(dtPolyRef)*(npath-1));
						nres += npath-1;
						
						// Remove trackbacks
						for (int j = 0; j < nres; ++j)
						{
							if (j-1 >= 0 && j+1 < nres)
							{
								if (res[j-1] == res[j+1])
								{
									memmove(res+(j-1), res+(j+1), sizeof(dtPolyRef)*(nres-(j+1)));
									nres -= 2;
									j -= 2;
								}
							}
						}
						
					}
					
					// Check for partial path.
					CA_SUPPRESS(6385);
					if (res[nres-1] != ag->targetRef)
					{
						// Partial path, constrain target position inside the last polygon.
						dtReal nearest[3];
						m_navquery->updateLinkFilter(ag->params.linkFilter.Get());
						status = m_navquery->closestPointOnPoly(res[nres - 1], targetPos, nearest);
						if (dtStatusSucceed(status))
							dtVcopy(targetPos, nearest);
						else
							valid = false;
					}
				}
				
				if (valid)
				{
					// Set current corridor.
					ag->corridor.setCorridor(targetPos, res, nres);
					// Force to update boundary.
					ag->boundary.reset();
					ag->targetState = DT_CROWDAGENT_TARGET_VALID;
				}
				else
				{
					// Something went wrong.
					ag->targetState = DT_CROWDAGENT_TARGET_FAILED;
				}

				ag->targetReplanTime = 0.0;
			}
		}
	}
	
}


void dtCrowd::updateTopologyOptimization(dtCrowdAgent** agents, const int nagents, const dtReal dt)
{
	if (!nagents)
		return;
	
	const dtReal OPT_TIME_THR = 0.5f; // seconds
	const int OPT_MAX_AGENTS = 1;
	dtCrowdAgent* queue[OPT_MAX_AGENTS];
	int nqueue = 0;
	
	for (int i = 0; i < nagents; ++i)
	{
		dtCrowdAgent* ag = agents[i];
		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;
		if (ag->targetState == DT_CROWDAGENT_TARGET_NONE || ag->targetState == DT_CROWDAGENT_TARGET_VELOCITY)
			continue;
		if ((ag->params.updateFlags & DT_CROWD_OPTIMIZE_TOPO) == 0)
			continue;
		ag->topologyOptTime += dt;
		if (ag->topologyOptTime >= OPT_TIME_THR)
			nqueue = addToOptQueue(ag, queue, nqueue, OPT_MAX_AGENTS);
	}

	for (int i = 0; i < nqueue; ++i)
	{
		dtCrowdAgent* ag = queue[i];
		m_navquery->updateLinkFilter(ag->params.linkFilter.Get());
		ag->corridor.optimizePathTopology(m_navquery, &m_filters[ag->params.filter]);
		ag->topologyOptTime = 0;
	}

}

void dtCrowd::checkPathValidity(dtCrowdAgent** agents, const int nagents, const dtReal dt)
{
	static const int CHECK_LOOKAHEAD = 10;
	static const dtReal TARGET_REPLAN_DELAY = 1.0; // seconds
	
	for (int i = 0; i < nagents; ++i)
	{
		dtCrowdAgent* ag = agents[i];
		bool replan = false;

		// [UE] try to place agent back on navmesh
		if (ag->state == DT_CROWDAGENT_STATE_INVALID)
		{
			ag->targetReplanTime += dt;
			
			if (ag->targetReplanTime > m_agentStateCheckInterval)
			{
				dtReal nearest[3];
				dtPolyRef ref;
				m_navquery->updateLinkFilter(ag->params.linkFilter.Get());
				m_navquery->findNearestPoly(ag->npos, m_ext, &m_filters[ag->params.filter], &ref, nearest);

				if (ref)
				{
					ag->state = DT_CROWDAGENT_STATE_WALKING;
					ag->targetReplanTime = 0;

					ag->corridor.reset(ref, nearest);
					ag->boundary.reset();
					replan = true;
				}
			}
		}

		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;

		if (ag->targetState == DT_CROWDAGENT_TARGET_NONE || ag->targetState == DT_CROWDAGENT_TARGET_VELOCITY)
			continue;
			
		ag->targetReplanTime += dt;

		// First check that the current location is valid.
		const int idx = getAgentIndex(ag);
		dtReal agentPos[3];
		dtPolyRef agentRef = ag->corridor.getFirstPoly();
		dtVcopy(agentPos, ag->npos);
		m_navquery->updateLinkFilter(ag->params.linkFilter.Get());
		if (!m_navquery->isValidPolyRef(agentRef, &m_filters[ag->params.filter]))
		{
			// Current location is not valid, try to reposition.
			// TODO: this can snap agents, how to handle that?
			dtReal nearest[3];
			agentRef = 0;
			m_navquery->findNearestPoly(ag->npos, m_ext, &m_filters[ag->params.filter], &agentRef, nearest);
			dtVcopy(agentPos, nearest);

			if (!agentRef)
			{
				// Could not find location in navmesh, set state to invalid.
				ag->corridor.reset(0, agentPos);
				ag->boundary.reset();
				ag->state = DT_CROWDAGENT_STATE_INVALID;

				// [UE] reset replan timer, it will be used to place agent back on navmesh
				ag->targetReplanTime = 0;
				continue;
			}

			// Make sure the first polygon is valid, but leave other valid
			// polygons in the path so that replanner can adjust the path better.
			ag->corridor.fixPathStart(agentRef, agentPos);
//			ag->corridor.trimInvalidPath(agentRef, agentPos, m_navquery, &m_filters[ag->params.filter]);
			ag->boundary.reset();
			dtVcopy(ag->npos, agentPos);

			replan = true;
		}

		// Try to recover move request position.
		if (ag->targetState != DT_CROWDAGENT_TARGET_NONE && ag->targetState != DT_CROWDAGENT_TARGET_FAILED)
		{
			if (!m_navquery->isValidPolyRef(ag->targetRef, &m_filters[ag->params.filter]))
			{
				// Current target is not valid, try to reposition.
				dtReal nearest[3];
				m_navquery->findNearestPoly(ag->targetPos, m_ext, &m_filters[ag->params.filter], &ag->targetRef, nearest);
				dtVcopy(ag->targetPos, nearest);
				replan = true;
			}
			if (!ag->targetRef)
			{
				// Failed to reposition target, fail moverequest.
				ag->corridor.reset(agentRef, agentPos);
				ag->targetState = DT_CROWDAGENT_TARGET_NONE;
			}
		}

		// If nearby corridor is not valid, replan.
		if (!ag->corridor.isValid(CHECK_LOOKAHEAD, m_navquery, &m_filters[ag->params.filter]))
		{
			// Fix current path.
//			ag->corridor.trimInvalidPath(agentRef, agentPos, m_navquery, &m_filters[ag->params.filter]);
//			ag->boundary.reset();
			replan = true;
		}
		
		// If the end of the path is near and it is not the requested location, replan.
		if (ag->targetState == DT_CROWDAGENT_TARGET_VALID)
		{
			if (ag->targetReplanTime > TARGET_REPLAN_DELAY &&
				ag->corridor.getPathCount() < CHECK_LOOKAHEAD &&
				ag->corridor.getLastPoly() != ag->targetRef)
				replan = true;
		}

		// Try to replan path to goal.
		if (replan)
		{
			if (ag->targetState != DT_CROWDAGENT_TARGET_NONE)
			{
				requestMoveTargetReplan(idx, ag->targetRef, ag->targetPos);
			}
		}
	}
}

void dtCrowd::update(const dtReal dt, dtCrowdAgentDebugInfo* debug)
{
	int numActive = cacheActiveAgents();
	if (numActive > 0)
	{
		updateStepPaths(dt, debug);
		updateStepProximityData(dt, debug);
		updateStepNextMovePoint(dt, debug);
		updateStepSteering(dt, debug);
		updateStepAvoidance(dt, debug);
		updateStepMove(dt, debug);
		updateStepCorridor(dt, debug);
		updateStepOffMeshAnim(dt, debug);
	}
}

void dtCrowd::updateStepPaths(const dtReal dt, dtCrowdAgentDebugInfo*)
{
	// Check that all agents still have valid paths.
	checkPathValidity(m_activeAgents, m_numActiveAgents, dt);

	// Update async move request and path finder.
	updateMoveRequest(dt);

	// Optimize path topology.
	updateTopologyOptimization(m_activeAgents, m_numActiveAgents, dt);
}

void dtCrowd::updateStepProximityData(const dtReal dt, dtCrowdAgentDebugInfo* debug)
{
	// Register agents to proximity grid.
	m_grid->clear();
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];
		const dtReal* p = ag->npos;
		const dtReal r = ag->params.radius;
		m_grid->addItem((unsigned short)i, p[0] - r, p[2] - r, p[0] + r, p[2] + r);
	}

	m_sharedBoundary.Tick(dt);

	// Get nearby navmesh segments and agents to collide with.
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];
		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;

		m_navquery->updateLinkFilter(ag->params.linkFilter.Get());
		int sharedDataIdx = -1;
		if (m_raycastSingleArea)
		{
			unsigned char allowedArea = DT_WALKABLE_AREA;
			m_navquery->getAttachedNavMesh()->getPolyArea(ag->corridor.getFirstPoly(), &allowedArea);

			sharedDataIdx = m_sharedBoundary.CacheData(ag->npos, ag->params.collisionQueryRange, ag->corridor.getFirstPoly(), m_navquery, allowedArea);
		}
		else
		{
			sharedDataIdx = m_sharedBoundary.CacheData(ag->npos, ag->params.collisionQueryRange, ag->corridor.getFirstPoly(), m_navquery, &m_filters[ag->params.filter]);
		}

		// Update the collision boundary after certain distance has been passed or
		// if it has become invalid.
		const dtReal updateThr = ag->params.collisionQueryRange*0.25f;
		if (dtVdist2DSqr(ag->npos, ag->boundary.getCenter()) > dtSqr(updateThr) ||
			!ag->boundary.isValid(m_navquery, &m_filters[ag->params.filter]))
		{
			const bool bIgnoreEdgesNearLastCorner = ag->ncorners && (ag->cornerFlags[ag->ncorners - 1] & (DT_STRAIGHTPATH_OFFMESH_CONNECTION | DT_STRAIGHTPATH_END));

			// UE: move dir for segment scoring
			dtReal moveDir[3] = { 0.0f };
			if (ag->ncorners)
			{
				dtVsub(moveDir, &ag->cornerVerts[0], ag->npos);
			}
			else
			{
				dtVcopy(moveDir, ag->vel);
			}
			dtVnormalize(moveDir);

			ag->boundary.update(&m_sharedBoundary, sharedDataIdx, ag->npos, ag->params.collisionQueryRange,
				bIgnoreEdgesNearLastCorner, &ag->cornerVerts[(ag->ncorners - 1) * 3],
				ag->corridor.getPath(), m_raycastSingleArea ? ag->corridor.getPathCount() : 0,
				moveDir, m_navquery, &m_filters[ag->params.filter]);
		}
		// Query neighbour agents
		ag->nneis = getNeighbours(ag->npos, ag->params.height, ag->params.collisionQueryRange,
			ag, ag->neis, DT_CROWDAGENT_MAX_NEIGHBOURS,
			m_activeAgents, m_numActiveAgents, m_grid);
		for (int j = 0; j < ag->nneis; j++)
			ag->neis[j].idx = getAgentIndex(m_activeAgents[ag->neis[j].idx]);
	}
}

void dtCrowd::updateStepNextMovePoint(const dtReal dt, dtCrowdAgentDebugInfo* debug)
{
	const int debugIdx = debug ? debug->idx : -1;

	// Find next corner to steer to.
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];

		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;
		if (ag->targetState == DT_CROWDAGENT_TARGET_NONE || ag->targetState == DT_CROWDAGENT_TARGET_VELOCITY)
			continue;

		// UE: corridor.earlyReach support
		const bool bAllowCuttingCorners = (ag->boundary.getSegmentCount() == 0);

		// Find corners for steering
		m_navquery->updateLinkFilter(ag->params.linkFilter.Get());
		ag->ncorners = ag->corridor.findCorners(ag->cornerVerts, ag->cornerFlags, ag->cornerPolys,
			DT_CROWDAGENT_MAX_CORNERS, m_navquery, &m_filters[ag->params.filter],
			ag->params.radius * m_pathOffsetRadiusMultiplier, ag->params.radius * 4.0f, bAllowCuttingCorners);

		const int agIndex = getAgentIndex(ag);
		if (debugIdx == agIndex)
		{
			dtVset(debug->optStart, 0, 0, 0);
			dtVset(debug->optEnd, 0, 0, 0);
		}

		// Check to see if the corner after the next corner is directly visible,
		// and short cut to there.
		if ((ag->params.updateFlags & DT_CROWD_OPTIMIZE_VIS) && ag->ncorners > 1)
		{
			unsigned char allowedArea = DT_WALKABLE_AREA;
			if (m_raycastSingleArea)
			{
				m_navquery->getAttachedNavMesh()->getPolyArea(ag->corridor.getFirstPoly(), &allowedArea);
				m_raycastFilter.setAreaCost(allowedArea, 1.0f);
			}

			const int firstCheckedIdx = ag->ncorners - 1;
			const int lastCheckedIdx = (ag->params.updateFlags & DT_CROWD_OPTIMIZE_VIS_MULTI) ? 1 : firstCheckedIdx;

			for (int cornerIdx = firstCheckedIdx; cornerIdx >= lastCheckedIdx; cornerIdx--)
			{
				dtReal* target = &ag->cornerVerts[cornerIdx * 3];

				const bool bOptimized = ag->corridor.optimizePathVisibility(target, ag->params.pathOptimizationRange, m_navquery, 
					m_raycastSingleArea ? &m_raycastFilter : &m_filters[ag->params.filter]);

				if (bOptimized)
				{
					// Copy data for debug purposes.
					if (debugIdx == agIndex)
					{
						dtVcopy(debug->optStart, ag->corridor.getPos());
						dtVcopy(debug->optEnd, target);
					}

					break;
				}
			}

			m_raycastFilter.setAreaCost(allowedArea, DT_UNWALKABLE_POLY_COST);
		}
	}

	// Trigger off-mesh connections (depends on corners).
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];

		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;
		if (ag->targetState == DT_CROWDAGENT_TARGET_NONE || ag->targetState == DT_CROWDAGENT_TARGET_VELOCITY)
			continue;

		// Check 
		const dtReal triggerRadius = ag->params.radius*2.25f;
		if (overOffmeshConnection(ag, triggerRadius))
		{
			// Prepare to off-mesh connection.
			const int idx = (int)(ag - m_agents);
			dtCrowdAgentAnimation* anim = &m_agentAnims[idx];
			m_navquery->updateLinkFilter(ag->params.linkFilter.Get());

			// Adjust the path over the off-mesh connection.
			dtPolyRef refs[2];

			// UE: m_keepOffmeshConnections support
			const bool bCanStart = m_keepOffmeshConnections ?
				ag->corridor.canMoveOverOffmeshConnection(ag->cornerPolys[ag->ncorners - 1], refs, ag->npos, anim->startPos, anim->endPos, m_navquery) :
				ag->corridor.moveOverOffmeshConnection(ag->cornerPolys[ag->ncorners - 1], refs, ag->npos, anim->startPos, anim->endPos, m_navquery);

			if (bCanStart)
			{
				dtVcopy(anim->initPos, ag->npos);
				anim->polyRef = refs[1];
				anim->active = 1;
				anim->t = 0.0f;
				anim->tmax = (dtVdist2D(anim->startPos, anim->endPos) / ag->params.maxSpeed) * 0.5f;

				ag->state = DT_CROWDAGENT_STATE_OFFMESH;
				ag->ncorners = 0;
				ag->nneis = 0;
				continue;
			}
			else
			{
				// Path validity check will ensure that bad/blocked connections will be replanned.
			}
		}
	}
}

void dtCrowd::updateStepSteering(const dtReal dt, dtCrowdAgentDebugInfo*)
{
	// Calculate steering.
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];

		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;
		if (ag->targetState == DT_CROWDAGENT_TARGET_NONE)
			continue;

		dtReal dvel[3] = { 0, 0, 0 };

		if (ag->targetState == DT_CROWDAGENT_TARGET_VELOCITY)
		{
			dtVcopy(dvel, ag->targetPos);
			ag->desiredSpeed = dtVlen(ag->targetPos);
		}
		else
		{
			// Calculate steering direction.
			if (ag->params.updateFlags & DT_CROWD_ANTICIPATE_TURNS)
				calcSmoothSteerDirection(ag, dvel);
			else
				calcStraightSteerDirection(ag, dvel);

			dtReal speedScale = 1.0f;

			if (ag->params.updateFlags & DT_CROWD_SLOWDOWN_AT_GOAL)
			{
				// Calculate speed scale, which tells the agent to slowdown at the end of the path.
				const dtReal slowDownRadius = ag->params.radius * 2;	// TODO: make less hacky.
				speedScale = getDistanceToGoal(ag, slowDownRadius) / slowDownRadius;
			}

			ag->desiredSpeed = ag->params.maxSpeed;
			dtVscale(dvel, dvel, ag->desiredSpeed * speedScale);
		}

		// Separation
		if (ag->params.updateFlags & DT_CROWD_SEPARATION)
		{
			const dtReal separationDist = ag->params.collisionQueryRange;
			const dtReal invSeparationDist = 1.0f / separationDist;
			const dtReal separationWeight = ag->params.separationWeight;
			const dtReal upDir[3] = { 0, 1.0f, 0 };

			dtReal w = 0;
			dtReal disp[3] = { 0, 0, 0 };

			for (int j = 0; j < ag->nneis; ++j)
			{
				const dtCrowdAgent* nei = &m_agents[ag->neis[j].idx];

				dtReal diff[3];
				dtVsub(diff, ag->npos, nei->npos);
				diff[1] = 0;
				
				const dtReal distSqr = dtVlenSqr(diff);
				if (distSqr < 0.00001f)
					continue;
				if (distSqr > dtSqr(separationDist))
					continue;
				const dtReal dist = dtSqrt(distSqr);
				const dtReal weight = separationWeight * (1.0f - dtSqr(dist*invSeparationDist));

				dtReal sepDot = dtVdot(diff, dvel);
				if (sepDot < m_separationDirFilter)
				{
					// [UE]: clamp to right/left vector, depending on which side nei is
					dtReal testDir[3] = { 0, 0, 0 };
					dtVcross(testDir, dvel, diff);
					const bool bRightSide = (testDir[1] > 0);

					dtVcross(diff, upDir, dvel);
					dtVnormalize(diff);
					dtVscale(diff, diff, bRightSide ? dist : -dist);
				}

				dtVmad(disp, disp, diff, weight / dist);
				w += 1.0f;
			}

			if (w > 0.0001f)
			{
				// Adjust desired velocity.
				dtVmad(dvel, dvel, disp, 1.0f / w);
				// Clamp desired velocity to desired speed.
				const dtReal speedSqr = dtVlenSqr(dvel);
				const dtReal desiredSqr = dtSqr(ag->desiredSpeed);
				if (speedSqr > desiredSqr)
					dtVscale(dvel, dvel, desiredSqr / speedSqr);
			}
		}

		// Set the desired velocity.
		dtVcopy(ag->dvel, dvel);
	}
}

void dtCrowd::updateStepAvoidance(const dtReal dt, dtCrowdAgentDebugInfo* debug)
{
	const int debugIdx = debug ? debug->idx : -1;
	m_velocitySampleCount = 0;

	// Velocity planning.	
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];

		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;

		if (ag->params.updateFlags & DT_CROWD_OBSTACLE_AVOIDANCE)
		{
			m_obstacleQuery->reset();

			// Add neighbours as obstacles.
			for (int j = 0; j < ag->nneis; ++j)
			{
				const dtCrowdAgent* nei = &m_agents[ag->neis[j].idx];
				m_obstacleQuery->addCircle(nei->npos, nei->params.radius, nei->vel, nei->dvel);
			}

			// Append neighbour segments as obstacles.
			for (int j = 0; j < ag->boundary.getSegmentCount(); ++j)
			{
				const dtReal* s = ag->boundary.getSegment(j);
				if (dtTriArea2D(ag->npos, s, s + 3) < 0.0f)
					continue;
				m_obstacleQuery->addSegment(s, s + 3, ag->boundary.getSegmentFlags(j));
			}

			dtObstacleAvoidanceDebugData* vod = 0;
			const int agIndex = getAgentIndex(ag);
			if (debug && debugIdx == agIndex)
				vod = debug->vod;

			// Sample new safe velocity.
			const dtObstacleAvoidanceParams* params = &m_obstacleQueryParams[ag->params.obstacleAvoidanceType];
			const int ns = m_obstacleQuery->sampleVelocity(ag->npos, ag->params.radius,
					ag->desiredSpeed, ag->params.avoidanceQueryMultiplier,
					ag->vel, ag->dvel, ag->nvel, params, vod);

			m_velocitySampleCount += ns;
		}
		else
		{
			// If not using velocity planning, new velocity is directly the desired velocity.
			dtVcopy(ag->nvel, ag->dvel);
		}
	}
}

void dtCrowd::updateStepMove(const dtReal dt, dtCrowdAgentDebugInfo*)
{
	// Integrate.
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];
		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;
		integrate(ag, dt);
	}

	// Handle collisions.
	static const dtReal COLLISION_RESOLVE_FACTOR = 0.7f;

	for (int iter = 0; iter < 4; ++iter)
	{
		for (int i = 0; i < m_numActiveAgents; ++i)
		{
			dtCrowdAgent* ag = m_activeAgents[i];
			const int idx0 = getAgentIndex(ag);

			if (ag->state != DT_CROWDAGENT_STATE_WALKING)
				continue;

			dtVset(ag->disp, 0, 0, 0);

			dtReal w = 0;

			for (int j = 0; j < ag->nneis; ++j)
			{
				const dtCrowdAgent* nei = &m_agents[ag->neis[j].idx];
				const int idx1 = getAgentIndex(nei);

				dtReal diff[3];
				dtVsub(diff, ag->npos, nei->npos);
				diff[1] = 0;

				dtReal dist = dtVlenSqr(diff);
				if (dist > dtSqr(ag->params.radius + nei->params.radius))
					continue;
				dist = dtSqrt(dist);
				dtReal pen = (ag->params.radius + nei->params.radius) - dist;
				if (dist < 0.0001f)
				{
					// m_activeAgents on top of each other, try to choose diverging separation directions.
					if (idx0 > idx1)
						dtVset(diff, -ag->dvel[2], 0, ag->dvel[0]);
					else
						dtVset(diff, ag->dvel[2], 0, -ag->dvel[0]);
					pen = 0.01f;
				}
				else
				{
					pen = (1.0f / dist) * (pen*0.5f) * COLLISION_RESOLVE_FACTOR;
				}

				dtVmad(ag->disp, ag->disp, diff, pen);

				w += 1.0f;
			}

			if (w > 0.0001f)
			{
				const dtReal iw = 1.0f / w;
				dtVscale(ag->disp, ag->disp, iw);
			}
		}

		for (int i = 0; i < m_numActiveAgents; ++i)
		{
			dtCrowdAgent* ag = m_activeAgents[i];
			if (ag->state != DT_CROWDAGENT_STATE_WALKING)
				continue;

			dtVadd(ag->npos, ag->npos, ag->disp);
		}
	}
}

void dtCrowd::updateStepCorridor(const dtReal dt, dtCrowdAgentDebugInfo*)
{
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];
		if (ag->state != DT_CROWDAGENT_STATE_WALKING)
			continue;

		// Move along navmesh.
		m_navquery->updateLinkFilter(ag->params.linkFilter.Get());
		const bool bMoved = ag->corridor.movePosition(ag->npos, m_navquery, &m_filters[ag->params.filter]);
		if (bMoved)
		{
			// Get valid constrained position back.
			dtVcopy(ag->npos, ag->corridor.getPos());
		}

		// If not using path, truncate the corridor to just one poly.
		if (ag->targetState == DT_CROWDAGENT_TARGET_NONE || ag->targetState == DT_CROWDAGENT_TARGET_VELOCITY)
		{
			ag->corridor.reset(ag->corridor.getFirstPoly(), ag->npos);
		}
	}
}

void dtCrowd::updateStepOffMeshAnim(const dtReal dt, dtCrowdAgentDebugInfo*)
{
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];
		const int agentIndex = getAgentIndex(ag);
		dtCrowdAgentAnimation* anim = &m_agentAnims[agentIndex];

		if (!anim->active)
			continue;

		anim->t += dt;
		if (anim->t > anim->tmax)
		{
			// Reset animation
			anim->active = 0;
			// Prepare agent for walking.
			ag->state = DT_CROWDAGENT_STATE_WALKING;

			// UE: m_keepOffmeshConnections support
			if (m_keepOffmeshConnections)
			{
				ag->corridor.pruneOffmeshConenction(anim->polyRef);
			}

			continue;
		}

		// Update position
		const dtReal ta = anim->tmax*0.15f;
		const dtReal tb = anim->tmax;
		if (anim->t < ta)
		{
			const dtReal u = tween(anim->t, 0.0, ta);
			dtVlerp(ag->npos, anim->initPos, anim->startPos, u);
		}
		else
		{
			const dtReal u = tween(anim->t, ta, tb);
			dtVlerp(ag->npos, anim->startPos, anim->endPos, u);
		}

		// Update velocity.
		dtVset(ag->vel, 0, 0, 0);
		dtVset(ag->dvel, 0, 0, 0);
	}
}

void dtCrowd::updateStepOffMeshVelocity(const dtReal dt, dtCrowdAgentDebugInfo*)
{
	dtReal DirLink[3];
	dtReal DirToEnd[3];

	// UE version of offmesh anims, updates velocity and checks distance instead of fixed time
	for (int i = 0; i < m_numActiveAgents; ++i)
	{
		dtCrowdAgent* ag = m_activeAgents[i];
		const int agentIndex = getAgentIndex(ag);
		dtCrowdAgentAnimation* anim = &m_agentAnims[agentIndex];

		if (!anim->active)
			continue;

		anim->t += dt;
		
		dtVsub(DirLink, anim->endPos, anim->startPos);
		dtVsub(DirToEnd, anim->endPos, ag->npos);
		const dtReal dirDot = dtVdot2D(DirLink, DirToEnd);
		const dtReal dist = dtVdist2DSqr(ag->npos, anim->endPos);
		const dtReal distThres = dtSqr(dtReal(5.));
		const dtReal heightDiff = dtAbs(ag->npos[1] - anim->endPos[1]);
		const dtReal heightThres = ag->params.height * 0.5f;

		// check height difference + distance or overshooting 
		if ((dist < distThres || dirDot < 0.0f) && heightDiff < heightThres)
		{
			// Reset animation
			anim->active = 0;
			// Prepare agent for walking.
			ag->state = DT_CROWDAGENT_STATE_WALKING;

			// UE: m_keepOffmeshConnections support
			if (m_keepOffmeshConnections)
			{
				ag->corridor.pruneOffmeshConenction(anim->polyRef);
			}
		}

		dtReal MoveDir[3] = { 0 };
		dtVsub(MoveDir, anim->endPos, anim->initPos);

		// check if it's moving along the line: initPos -> endPos
		const dtReal distFromLinkSq = dtDistancePtSegSqr(ag->npos, anim->initPos, anim->endPos);
		const dtReal maxDistFromLinkSq = dtSqr(ag->params.radius * 2.0f);
		if (distFromLinkSq > maxDistFromLinkSq)
		{
			dtVsub(MoveDir, anim->endPos, ag->npos);
		}

		if (ag->state == DT_CROWDAGENT_STATE_OFFMESH)
		{
			MoveDir[1] = 0.0f;
			dtVnormalize(MoveDir);
			dtVscale(ag->nvel, MoveDir, ag->params.maxSpeed);
			dtVcopy(ag->vel, ag->nvel);
			dtVset(ag->dvel, 0, 0, 0);
		}
	}
}

void dtCrowd::setAgentCheckInterval(const dtReal t)
{
	m_agentStateCheckInterval = t;
}

void dtCrowd::setSingleAreaVisibilityOptimization(bool bEnable)
{
	m_raycastSingleArea = bEnable;
}

void dtCrowd::setPruneStartedOffmeshConnections(bool bRemoveFromCorridor)
{
	m_keepOffmeshConnections = !bRemoveFromCorridor;
}

void dtCrowd::setEarlyReachTestOptimization(bool bEnable)
{
	m_earlyReachTest = bEnable;
}

bool dtCrowd::isOutsideCorridor(const int idx) const
{
	if (idx < 0 || idx > m_maxAgents)
		return false;

	dtCrowdAgent* ag = &m_agents[idx];
	if (ag->state != DT_CROWDAGENT_STATE_WALKING)
		return false;

	dtReal nearest[3];
	dtPolyRef polyRef = 0;
	
	const dtStatus status = m_navquery->findNearestPoly(ag->npos, m_ext, &m_filters[ag->params.filter], &polyRef, nearest);	
	if (dtStatusSucceed(status))
	{
		const dtPolyRef* path = ag->corridor.getPath();
		for (int i = 0; i < ag->corridor.getPathCount(); i++)
		{
			if (path[i] == polyRef)
				return false;
		}
	}

	return true;
}

bool dtCrowd::setAgentCorridor(const int idx, const dtPolyRef* path, const int npath)
{
	if (idx < 0 || idx > m_maxAgents || npath <= 0)
		return false;

	dtCrowdAgent* ag = &m_agents[idx];
	if (ag->targetState != DT_CROWDAGENT_TARGET_REQUESTING || ag->targetRef != path[npath - 1])
		return false;

	ag->corridor.setCorridor(ag->targetPos, path, npath);
	ag->corridor.setEarlyReachTest(m_earlyReachTest);
	ag->boundary.reset();
	ag->targetState = DT_CROWDAGENT_TARGET_VALID;
	ag->targetReplanTime = 0.0;
    
    return true;
}

const dtQueryFilter* dtCrowd::getFilter(const int idx) const
{
	return (idx >= 0) && (idx < m_maxAgents) && (m_agents[idx].params.filter < DT_CROWD_MAX_FILTERS) ?
		&m_filters[m_agents[idx].params.filter] : NULL;
}

dtQueryFilter* dtCrowd::getEditableFilter(const int idx)
{
	return (idx >= 0) && (idx < m_maxAgents) && (m_agents[idx].params.filter < DT_CROWD_MAX_FILTERS) ?
		&m_filters[m_agents[idx].params.filter] : NULL;
}

void dtCrowd::setSeparationFilter(dtReal InFilter)
{
	m_separationDirFilter = InFilter;
}

void dtCrowd::setPathOffsetRadiusMultiplier(dtReal RadiusMultiplier)
{
	m_pathOffsetRadiusMultiplier = RadiusMultiplier;
}
