// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DetourCrowd/DetourLocalBoundary.h"
#include "DetourCrowd/DetourCrowd.h"
#include "Detour/DetourAssert.h"


dtLocalBoundary::dtLocalBoundary() :
	m_nsegs(0),
	m_npolys(0)
{
	dtVset(m_center, DT_REAL_MAX, DT_REAL_MAX, DT_REAL_MAX);
}

dtLocalBoundary::~dtLocalBoundary()
{
}

void dtLocalBoundary::reset()
{
	dtVset(m_center, DT_REAL_MAX, DT_REAL_MAX, DT_REAL_MAX);
	m_npolys = 0;
	m_nsegs = 0;
}

void dtLocalBoundary::addSegment(const dtReal dist, const dtReal* s, int flags)
{
	// Insert neighbour based on the distance.
	Segment* seg = 0;
	if (!m_nsegs)
	{
		// First, trivial accept.
		seg = &m_segs[0];
	}
	else if (dist >= m_segs[m_nsegs-1].d)
	{
		// Further than the last segment, skip.
		if (m_nsegs >= MAX_LOCAL_SEGS)
			return;
		// Last, trivial accept.
		seg = &m_segs[m_nsegs];
	}
	else
	{
		// Insert inbetween.
		int i;
		for (i = 0; i < m_nsegs; ++i)
			if (dist <= m_segs[i].d)
				break;
		const int tgt = i+1;
		const int n = dtMin(m_nsegs-i, MAX_LOCAL_SEGS-tgt);
		dtAssert(tgt+n <= MAX_LOCAL_SEGS);
		if (n > 0)
			memmove(&m_segs[tgt], &m_segs[i], sizeof(Segment)*n);
		seg = &m_segs[i];
	}
	
	seg->d = dist;
	seg->flags = flags;
	memcpy(seg->s, s, sizeof(dtReal)*6);
	
	if (m_nsegs + 1 < MAX_LOCAL_SEGS)
	{
		++m_nsegs;
	}
}

namespace LocalBoundaryHelpers
{
	inline bool IsOutsideHeightLimit(const dtReal* pos, const dtReal* p, const dtReal* q, dtReal tseg)
	{
		const dtReal closestPtHeight = p[1] + (q[1] - p[1]) * tseg;
		const dtReal segHeightDiff = dtAbs(closestPtHeight - pos[1]);
		const dtReal maxHeightDiff = 50.0f;

		return segHeightDiff > maxHeightDiff;
	}

	inline int GetSegmentFlags(const dtReal* endPos, const dtReal* p, const dtReal* q, dtReal QueryRange)
	{
		const dtReal ignoreDistancePct = 0.25f;
		const dtReal maxDistSq = dtSqr(QueryRange * ignoreDistancePct);

		dtReal tsegEnd = 0.0f;
		const dtReal distEndSqr = dtDistancePtSegSqr2D(endPos, p, q, tsegEnd);
		return (distEndSqr < maxDistSq) ? DT_CROWD_BOUNDARY_IGNORE : 0;
	}
}

void dtLocalBoundary::update(dtPolyRef ref, const dtReal* pos, const dtReal collisionQueryRange,
	const bool bIgnoreAtEnd, const dtReal* endPos,
	const dtPolyRef* path, const int npath,
	const dtReal* moveDir,
	dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	static const int MAX_SEGS_PER_POLY = DT_VERTS_PER_POLYGON*3;
	
	if (!ref)
	{
		dtVset(m_center, DT_REAL_MAX, DT_REAL_MAX, DT_REAL_MAX);
		m_nsegs = 0;
		m_npolys = 0;
		return;
	}
	
	dtVcopy(m_center, pos);
	
	// First query non-overlapping polygons.
	navquery->findLocalNeighbourhood(ref, pos, collisionQueryRange,
									 filter, m_polys, 0, &m_npolys, MAX_LOCAL_POLYS);
	
	// [UE] include direction to segment in score
	dtReal closestPt[3] = { 0.0f };
	dtReal dirToSeg[3] = { 0.0f };

	// Secondly, store all polygon edges.
	m_nsegs = 0;
	dtReal segs[MAX_SEGS_PER_POLY*6];
	int nsegs = 0;
	for (int j = 0; j < m_npolys; ++j)
	{
		navquery->getPolyWallSegments(m_polys[j], filter, segs, 0, &nsegs, MAX_SEGS_PER_POLY);
		for (int k = 0; k < nsegs; ++k)
		{
			const dtReal* s = &segs[k*6];
			// Skip too distant segments.
			dtReal tseg = 0.0f;
			const dtReal distSqr = dtDistancePtSegSqr2D(pos, s, s+3, tseg);
			if (distSqr > dtSqr(collisionQueryRange))
				continue;

			// [UE] handle segments too far in Y (height diff)
			const bool bOutsideHeightLimit = LocalBoundaryHelpers::IsOutsideHeightLimit(pos, s, s + 3, tseg);
			if (bOutsideHeightLimit)
			{
				continue;
			}

			// [UE] handle segments too close to requested position
			const int segFlags = bIgnoreAtEnd ? LocalBoundaryHelpers::GetSegmentFlags(endPos, s, s + 3, collisionQueryRange) : 0;

			// [UE] include direction to segment in score
			dtVlerp(closestPt, s, s + 3, tseg);
			dtVsub(dirToSeg, closestPt, pos);
			dtVnormalize(dirToSeg);
			const dtReal dseg = dtVdot2D(dirToSeg, moveDir);
			const dtReal score = distSqr * ((1.0f - dseg) * 0.5f);

			addSegment(score, s, segFlags);
		}
	}
}

void dtLocalBoundary::update(const dtSharedBoundary* sharedData, const int sharedIdx,
	const dtReal* pos, const dtReal collisionQueryRange,
	const bool bIgnoreAtEnd, const dtReal* endPos,
	const dtPolyRef* path, const int npath, const dtReal* moveDir,
	dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	if (!sharedData || !sharedData->HasSample(sharedIdx))
	{
		return;
	}

	const dtSharedBoundaryData& Data = sharedData->Data[sharedIdx];
	m_npolys = FMath::Min(Data.Polys.Num(), MAX_LOCAL_POLYS);
	int32 PolyIdx = 0;
	for (auto It = Data.Polys.CreateConstIterator(); It; ++It)
	{
		m_polys[PolyIdx] = *It;
		
		PolyIdx++;
		if (PolyIdx >= m_npolys)
		{
			break;
		}
	}

	dtReal closestPt[3] = { 0.0f };
	dtReal dirToSeg[3] = { 0.0f };
	dtReal s[6];

	TSet<dtPolyRef> PathLookup;
	for (int32 Idx = 0; Idx < npath; Idx++)
	{
		PathLookup.Add(path[Idx]);
	}

	m_nsegs = 0;
	for (int32 Idx = 0; Idx < Data.Edges.Num(); Idx++)
	{
		dtReal tseg = 0.0f;
		const dtReal distSqr = dtDistancePtSegSqr2D(pos, Data.Edges[Idx].v0, Data.Edges[Idx].v1, tseg);
		if (distSqr > dtSqr(collisionQueryRange))
			continue;

		// handle segments too far in Y (height diff)
		const bool bOutsideHeightLimit = LocalBoundaryHelpers::IsOutsideHeightLimit(pos, Data.Edges[Idx].v0, Data.Edges[Idx].v1, tseg);
		if (bOutsideHeightLimit)
		{
			continue;
		}

		// handle segments too close to requested position
		const int segFlags = bIgnoreAtEnd ? LocalBoundaryHelpers::GetSegmentFlags(endPos, Data.Edges[Idx].v0, Data.Edges[Idx].v1, collisionQueryRange) : 0;

		// remove segments when both sides are on path (single area trace)
		if (PathLookup.Contains(Data.Edges[Idx].p0) && PathLookup.Contains(Data.Edges[Idx].p1))
		{
			continue;
		}

		// include direction to segment in score
		dtVlerp(closestPt, Data.Edges[Idx].v0, Data.Edges[Idx].v1, tseg);
		dtVsub(dirToSeg, closestPt, pos);
		dtVnormalize(dirToSeg);
		const dtReal dseg = dtVdot2D(dirToSeg, moveDir);
		const dtReal score = distSqr *((1.0f - dseg) * 0.5f);

		dtVcopy(s, Data.Edges[Idx].v0);
		dtVcopy(s + 3, Data.Edges[Idx].v1);
		addSegment(score, s, segFlags);
	}
}

bool dtLocalBoundary::isValid(dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	if (!m_npolys)
		return false;
	
	// Check that all polygons still pass query filter.
	for (int i = 0; i < m_npolys; ++i)
	{
		if (!navquery->isValidPolyRef(m_polys[i], filter))
			return false;
	}

	return true;
}

