// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DetourCrowd/DetourObstacleAvoidance.h"
#include "Detour/DetourAssert.h"
#include "DetourCrowd/DetourCrowd.h"

static const dtReal DT_PI = 3.14159265358979323846;

static int sweepCircleCircle(const dtReal* c0, const dtReal r0, const dtReal* v,
							 const dtReal* c1, const dtReal r1,
							 dtReal& tmin, dtReal& tmax)
{
	static const dtReal EPS = 0.0001f;
	dtReal s[3];
	dtVsub(s,c1,c0);
	dtReal r = r0+r1;
	dtReal c = dtVdot2D(s,s) - r*r;
	dtReal a = dtVdot2D(v,v);
	if (a < EPS) return 0;	// not moving
	
	// Overlap, calc time to exit.
	dtReal b = dtVdot2D(v,s);
	dtReal d = b*b - a*c;
	if (d < 0.0f) return 0; // no intersection.
	a = 1.0f / a;
	const dtReal rd = dtSqrt(d);
	tmin = (b - rd) * a;
	tmax = (b + rd) * a;
	return 1;
}

static int isectRaySeg(const dtReal* ap, const dtReal* u,
					   const dtReal* bp, const dtReal* bq,
					   dtReal& t)
{
	dtReal v[3], w[3];
	dtVsub(v,bq,bp);
	dtVsub(w,ap,bp);
	dtReal d = dtVperp2D(u,v);
	if (dtAbs(d) < 1e-6f) return 0;
	d = 1.0f/d;
	t = dtVperp2D(v,w) * d;
	if (t < 0 || t > 1) return 0;
	dtReal s = dtVperp2D(u,w) * d;
	if (s < 0 || s > 1) return 0;
	return 1;
}



dtObstacleAvoidanceDebugData* dtAllocObstacleAvoidanceDebugData()
{
	void* mem = dtAlloc(sizeof(dtObstacleAvoidanceDebugData), DT_ALLOC_PERM_AVOIDANCE);
	if (!mem) return 0;
	return new(mem) dtObstacleAvoidanceDebugData;
}

void dtFreeObstacleAvoidanceDebugData(dtObstacleAvoidanceDebugData* ptr)
{
	if (!ptr) return;
	ptr->~dtObstacleAvoidanceDebugData();
	dtFree(ptr, DT_ALLOC_PERM_AVOIDANCE);
}


dtObstacleAvoidanceDebugData::dtObstacleAvoidanceDebugData() :
	m_nsamples(0),
	m_maxSamples(0),
	m_vel(0),
	m_ssize(0),
	m_pen(0),
	m_vpen(0),
	m_vcpen(0),
	m_spen(0),
	m_tpen(0)
{
}

dtObstacleAvoidanceDebugData::~dtObstacleAvoidanceDebugData()
{
	dtFree(m_vel, DT_ALLOC_PERM_AVOIDANCE);
	dtFree(m_ssize, DT_ALLOC_PERM_AVOIDANCE);
	dtFree(m_pen, DT_ALLOC_PERM_AVOIDANCE);
	dtFree(m_vpen, DT_ALLOC_PERM_AVOIDANCE);
	dtFree(m_vcpen, DT_ALLOC_PERM_AVOIDANCE);
	dtFree(m_spen, DT_ALLOC_PERM_AVOIDANCE);
	dtFree(m_tpen, DT_ALLOC_PERM_AVOIDANCE);
}
		
bool dtObstacleAvoidanceDebugData::init(const int maxSamples)
{
	dtAssert(maxSamples);
	m_maxSamples = maxSamples;

	m_vel = (dtReal*)dtAlloc(sizeof(dtReal)*3*m_maxSamples, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_vel)
		return false;
	m_pen = (dtReal*)dtAlloc(sizeof(dtReal)*m_maxSamples, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_pen)
		return false;
	m_ssize = (dtReal*)dtAlloc(sizeof(dtReal)*m_maxSamples, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_ssize)
		return false;
	m_vpen = (dtReal*)dtAlloc(sizeof(dtReal)*m_maxSamples, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_vpen)
		return false;
	m_vcpen = (dtReal*)dtAlloc(sizeof(dtReal)*m_maxSamples, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_vcpen)
		return false;
	m_spen = (dtReal*)dtAlloc(sizeof(dtReal)*m_maxSamples, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_spen)
		return false;
	m_tpen = (dtReal*)dtAlloc(sizeof(dtReal)*m_maxSamples, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_tpen)
		return false;
	
	return true;
}

void dtObstacleAvoidanceDebugData::reset()
{
	m_nsamples = 0;
}

void dtObstacleAvoidanceDebugData::addSample(const dtReal* vel, const dtReal ssize, const dtReal pen,
											 const dtReal vpen, const dtReal vcpen, const dtReal spen, const dtReal tpen)
{
	if (m_nsamples >= m_maxSamples)
		return;
	dtAssert(m_vel);
	dtAssert(m_ssize);
	dtAssert(m_pen);
	dtAssert(m_vpen);
	dtAssert(m_vcpen);
	dtAssert(m_spen);
	dtAssert(m_tpen);
	dtVcopy(&m_vel[m_nsamples*3], vel);
	m_ssize[m_nsamples] = ssize;
	m_pen[m_nsamples] = pen;
	m_vpen[m_nsamples] = vpen;
	m_vcpen[m_nsamples] = vcpen;
	m_spen[m_nsamples] = spen;
	m_tpen[m_nsamples] = tpen;
	m_nsamples++;
}

static void normalizeArray(dtReal* arr, const int n)
{
	// Normalize penaly range.
	dtReal minPen = DT_REAL_MAX;
	dtReal maxPen = -DT_REAL_MAX;
	for (int i = 0; i < n; ++i)
	{
		minPen = dtMin(minPen, arr[i]);
		maxPen = dtMax(maxPen, arr[i]);
	}
	const dtReal penRange = maxPen-minPen;
	const dtReal s = penRange > 0.001f ? (1.0f / penRange) : 1;
	for (int i = 0; i < n; ++i)
		arr[i] = dtClamp((arr[i]-minPen)*s, 0.0f, 1.0f);
}

void dtObstacleAvoidanceDebugData::normalizeSamples()
{
	normalizeArray(m_pen, m_nsamples);
	normalizeArray(m_vpen, m_nsamples);
	normalizeArray(m_vcpen, m_nsamples);
	normalizeArray(m_spen, m_nsamples);
	normalizeArray(m_tpen, m_nsamples);
}


dtObstacleAvoidanceQuery* dtAllocObstacleAvoidanceQuery()
{
	void* mem = dtAlloc(sizeof(dtObstacleAvoidanceQuery), DT_ALLOC_PERM_AVOIDANCE);
	if (!mem) return 0;
	return new(mem) dtObstacleAvoidanceQuery;
}

void dtFreeObstacleAvoidanceQuery(dtObstacleAvoidanceQuery* ptr)
{
	if (!ptr) return;
	ptr->~dtObstacleAvoidanceQuery();
	dtFree(ptr, DT_ALLOC_PERM_AVOIDANCE);
}


dtObstacleAvoidanceQuery::dtObstacleAvoidanceQuery() :
	m_customPatterns(0),
	m_circles(0),
	m_segments(0),
	m_maxPatterns(0),
	m_maxCircles(0),
	m_ncircles(0),
	m_maxSegments(0),
	m_nsegments(0)
{
}

dtObstacleAvoidanceQuery::~dtObstacleAvoidanceQuery()
{
	dtFree(m_circles, DT_ALLOC_PERM_AVOIDANCE);
	dtFree(m_segments, DT_ALLOC_PERM_AVOIDANCE);
	dtFree(m_customPatterns, DT_ALLOC_PERM_AVOIDANCE);
}

bool dtObstacleAvoidanceQuery::init(const int maxCircles, const int maxSegments, const int maxCustomPatterns)
{
	m_maxCircles = maxCircles;
	m_ncircles = 0;
	m_circles = (dtObstacleCircle*)dtAlloc(sizeof(dtObstacleCircle)*m_maxCircles, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_circles)
		return false;
	memset(m_circles, 0, sizeof(dtObstacleCircle)*m_maxCircles);

	m_maxSegments = maxSegments;
	m_nsegments = 0;
	m_segments = (dtObstacleSegment*)dtAlloc(sizeof(dtObstacleSegment)*m_maxSegments, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_segments)
		return false;
	memset(m_segments, 0, sizeof(dtObstacleSegment)*m_maxSegments);

	m_maxPatterns = maxCustomPatterns;
	m_customPatterns = (dtObstacleAvoidancePattern*)dtAlloc(sizeof(dtObstacleAvoidancePattern)*m_maxPatterns, DT_ALLOC_PERM_AVOIDANCE);
	if (!m_customPatterns)
		return false;
	memset(m_customPatterns, 0, sizeof(dtObstacleAvoidancePattern)*m_maxPatterns);

	return true;
}

void dtObstacleAvoidanceQuery::reset()
{
	m_ncircles = 0;
	m_nsegments = 0;
}

void dtObstacleAvoidanceQuery::addCircle(const dtReal* pos, const dtReal rad,
										 const dtReal* vel, const dtReal* dvel)
{
	if (m_ncircles >= m_maxCircles)
		return;
		
	dtObstacleCircle* cir = &m_circles[m_ncircles++];
	dtVcopy(cir->p, pos);
	cir->rad = rad;
	dtVcopy(cir->vel, vel);
	dtVcopy(cir->dvel, dvel);
}

void dtObstacleAvoidanceQuery::addSegment(const dtReal* p, const dtReal* q, int flags)
{
	// [UE] fixed condition below. Used to be strict > comparison
	if (m_nsegments >= m_maxSegments)
		return;
	
	dtObstacleSegment* seg = &m_segments[m_nsegments++];
	dtVcopy(seg->p, p);
	dtVcopy(seg->q, q);
	seg->canIgnore = (flags & DT_CROWD_BOUNDARY_IGNORE) != 0;
}

void dtObstacleAvoidanceQuery::prepare(const dtReal* pos, const dtReal* dvel)
{
	// Prepare obstacles
	for (int i = 0; i < m_ncircles; ++i)
	{
		dtObstacleCircle* cir = &m_circles[i];
		
		// Side
		const dtReal* pa = pos;
		const dtReal* pb = cir->p;
		
		const dtReal orig[3] = {0,0};
		dtReal dv[3];
		dtVsub(cir->dp,pb,pa);
		dtVnormalize(cir->dp);
		dtVsub(dv, cir->dvel, dvel);
		
		const dtReal a = dtTriArea2D(orig, cir->dp,dv);
		if (a < m_params.sideBiasThreshold) //@UE configurable threshold (was 0.01f)
		{
			cir->np[0] = -cir->dp[2];
			cir->np[2] = cir->dp[0];
		}
		else
		{
			cir->np[0] = cir->dp[2];
			cir->np[2] = -cir->dp[0];
		}
	}	

	for (int i = 0; i < m_nsegments; ++i)
	{
		dtObstacleSegment* seg = &m_segments[i];
		
		// Precalc if the agent is really close to the segment.
		const dtReal r = 0.01f;
		dtReal t;
		seg->touch = dtDistancePtSegSqr2D(pos, seg->p, seg->q, t) < dtSqr(r);
	}	
}

dtReal dtObstacleAvoidanceQuery::processSample(const dtReal* vcand, const dtReal cs,
											  const dtReal* pos, const dtReal rad,
											  const dtReal* vel, const dtReal* dvel,
											  dtObstacleAvoidanceDebugData* debug)
{
	// Find min time of impact and exit amongst all obstacles.
	dtReal tmin = m_params.horizTime;
	dtReal side = 0;
	dtReal rightBias = 0; //@UE right-side passing preference
	int nside = 0;
	
	for (int i = 0; i < m_ncircles; ++i)
	{
		const dtObstacleCircle* cir = &m_circles[i];
			
		// RVO
		dtReal vab[3];
		dtVscale(vab, vcand, 2);
		dtVsub(vab, vab, vel);
		dtVsub(vab, vab, cir->vel);
		
		// Side
		side += dtClamp(dtMin(dtVdot2D(cir->dp,vab)*0.5f+0.5f, dtVdot2D(cir->np,vab)*2), 0.0f, 1.0f);
		nside++;

		//@UE BEGIN - Right-side passing bias: penalize left-side passing to reduce head-on oscillation.
		// Use a fixed right-side normal (-dp[2], dp[0]) independent of the np flip in prepare().
		// Both agents in a head-on pair compute opposite dp vectors, so this consistently
		// makes them agree to pass on their respective right sides.
		if (m_params.weightRightBias > 0.0f)
		{
			const dtReal rightDot = -cir->dp[2] * vab[0] + cir->dp[0] * vab[2];
			rightBias += dtClamp(rightDot * 0.5f + 0.5f, 0.0f, 1.0f);
		}
		//@UE END
		
		dtReal htmin = 0, htmax = 0;
		if (!sweepCircleCircle(pos,rad, vab, cir->p,cir->rad, htmin, htmax))
			continue;
		
		// Handle overlapping obstacles.
		if (htmin < 0.0f && htmax > 0.0f)
		{
			// Avoid more when overlapped.
			htmin = -htmin * 0.5f;
		}
		
		if (htmin >= 0.0f)
		{
			// The closest obstacle is somewhere ahead of us, keep track of nearest obstacle.
			if (htmin < tmin)
				tmin = htmin;
		}
	}

	const dtReal TooCloseToSegmentDistPct = 0.1f;
	for (int i = 0; i < m_nsegments; ++i)
	{
		const dtObstacleSegment* seg = &m_segments[i];
		dtReal htmin = 0;
		
		if (seg->touch)
		{
			// Special case when the agent is very close to the segment.
			dtReal sdir[3], snorm[3];
			dtVsub(sdir, seg->q, seg->p);
			snorm[0] = -sdir[2];
			snorm[2] = sdir[0];
			// If the velocity is pointing towards the segment, no collision.
			if (dtVdot2D(snorm, vcand) < 0.0f)
				continue;
			// Else immediate collision.
			htmin = 0.0f;
		}
		else
		{
			if (!isectRaySeg(pos, vcand, seg->p, seg->q, htmin))
				continue;

			if (seg->canIgnore && htmin > TooCloseToSegmentDistPct)
			{
				htmin = 1.0f;
			}
		}
		
		// UE: when sample is too close to segment (navmesh wall) - disable it completely
		if (htmin < TooCloseToSegmentDistPct)
		{
			return -1.0f;
		}

		// Avoid less when facing walls.
		htmin *= 2.0f;
		
		// The closest obstacle is somewhere ahead of us, keep track of nearest obstacle.
		if (htmin < tmin)
			tmin = htmin;
	}
	
	// Normalize side bias, to prevent it dominating too much.
	if (nside)
	{
		side /= nside;
		rightBias /= nside; //@UE
	}

	const dtReal vpen = m_params.weightDesVel * (dtVdist2D(vcand, dvel) * m_invVmax);
	const dtReal vcpen = m_params.weightCurVel * (dtVdist2D(vcand, vel) * m_invVmax);
	const dtReal spen = m_params.weightSide * side;
	const dtReal tpen = m_params.weightToi * (1.0f/(0.1f+tmin*m_invHorizTime));
	const dtReal rpen = m_params.weightRightBias * rightBias; //@UE right-side passing penalty
	
	const dtReal penalty = vpen + vcpen + spen + tpen + rpen; //@UE added rpen
	
	// Store different penalties for debug viewing
	if (debug)
		debug->addSample(vcand, cs, penalty, vpen, vcpen, spen, tpen);
	
	return penalty;
}

bool dtObstacleAvoidanceQuery::setCustomSamplingPattern(int idx, const dtReal* angles, const dtReal* radii, int nsamples)
{
	if (nsamples < 0 || nsamples >= DT_MAX_CUSTOM_SAMPLES)
		return false;

	if (idx < 0 || idx >= m_maxPatterns)
		return false;

	memcpy(m_customPatterns[idx].angles, angles, sizeof(dtReal)* nsamples);
	memcpy(m_customPatterns[idx].radii, radii, sizeof(dtReal)* nsamples);
	m_customPatterns[idx].nsamples = nsamples;

	return true;
}

bool dtObstacleAvoidanceQuery::getCustomSamplingPattern(int idx, dtReal* angles, dtReal* radii, int* nsamples)
{
	if (idx < 0 || idx >= m_maxPatterns)
		return false;

	memcpy(angles, m_customPatterns[idx].angles, sizeof(dtReal)* m_customPatterns[idx].nsamples);
	memcpy(radii, m_customPatterns[idx].radii, sizeof(dtReal)* m_customPatterns[idx].nsamples);
	*nsamples = m_customPatterns[idx].nsamples;

	return true;
}

int dtObstacleAvoidanceQuery::sampleVelocityCustom(const dtReal* pos, const dtReal rad,
												   const dtReal vmax, const dtReal vmult,
												   const dtReal* vel, const dtReal* dvel, dtReal* nvel,
												   const dtObstacleAvoidanceParams* params,
											  	   dtObstacleAvoidanceDebugData* debug)
{
	memcpy(&m_params, params, sizeof(dtObstacleAvoidanceParams)); //@UE moved before prepare() so it can access m_params
	prepare(pos, dvel);

	m_invHorizTime = 1.0f / m_params.horizTime;
	m_vmax = vmax;
	m_invVmax = 1.0f / vmax;
	
	dtVset(nvel, 0,0,0);
	
	if (debug)
		debug->reset();

	const dtObstacleAvoidancePattern& pattern = m_customPatterns[m_params.patternIdx];
	const dtReal dang = dtAtan2(dvel[2], dvel[0]);

	dtReal pat[DT_MAX_CUSTOM_SAMPLES * 2];
	for (int i = 0; i < pattern.nsamples; i++)
	{
		dtReal a = dang + pattern.angles[i];
		pat[i * 2 + 0] = dtCos(a) * pattern.radii[i];
		pat[i * 2 + 1] = dtSin(a) * pattern.radii[i];
	}

	dtReal minPenalty = DT_REAL_MAX;
	dtReal cr = vmax * vmult * (1.0f - m_params.velBias);
	dtReal res[3];
	bool bFoundSample = false;
	dtVset(res, dvel[0] * m_params.velBias, 0, dvel[2] * m_params.velBias);

	for (int i = 0; i < pattern.nsamples; ++i)
	{
		dtReal vcand[3];
		vcand[0] = res[0] + pat[i * 2 + 0] * cr;
		vcand[1] = 0;
		vcand[2] = res[2] + pat[i * 2 + 1] * cr;

		if (dtSqr(vcand[0]) + dtSqr(vcand[2]) > dtSqr((vmax * vmult) + 0.001f)) continue;

		const dtReal penalty = processSample(vcand, 20.0f, pos, rad, vel, dvel, debug);
		if (penalty < minPenalty && penalty >= 0.0f)
		{
			bFoundSample = true;
			minPenalty = penalty;
			dtVcopy(nvel, vcand);
		}
	}

	if (!bFoundSample)
	{
		dtVcopy(nvel, dvel);
	}
	else
	{
		dtVscale(nvel, nvel, 1.0f / vmult);
	}

	return pattern.nsamples;
}

int dtObstacleAvoidanceQuery::sampleVelocityAdaptive(const dtReal* pos, const dtReal rad,
													 const dtReal vmax, const dtReal vmult,
													 const dtReal* vel, const dtReal* dvel, dtReal* nvel,
													 const dtObstacleAvoidanceParams* params,
													 dtObstacleAvoidanceDebugData* debug)
{
	memcpy(&m_params, params, sizeof(dtObstacleAvoidanceParams)); //@UE moved before prepare() so it can access m_params
	prepare(pos, dvel);

	m_invHorizTime = 1.0f / m_params.horizTime;
	m_vmax = vmax * vmult;
	m_invVmax = 1.0f / (vmax * vmult);
	
	dtVset(nvel, 0,0,0);
	
	if (debug)
		debug->reset();

	// Build sampling pattern aligned to desired velocity.
	dtReal pat[(DT_MAX_PATTERN_DIVS*DT_MAX_PATTERN_RINGS+1)*2];
	int npat = 0;

	const int ndivs = (int)m_params.adaptiveDivs;
	const int nrings= (int)m_params.adaptiveRings;
	const int depth = (int)m_params.adaptiveDepth;
	
	const int nd = dtClamp(ndivs, 1, DT_MAX_PATTERN_DIVS);
	const int nr = dtClamp(nrings, 1, DT_MAX_PATTERN_RINGS);
	const dtReal da = (dtReal(1.)/nd) * DT_PI*2;
	const dtReal dang = dtAtan2(dvel[2], dvel[0]);
	
	// Always add sample at zero
	pat[npat*2+0] = 0;
	pat[npat*2+1] = 0;
	npat++;
	
	for (int j = 0; j < nr; ++j)
	{
		const dtReal r = (dtReal)(nr-j)/(dtReal)nr;
		dtReal a = dang + dtReal(j&1)*0.5f*da;
		for (int i = 0; i < nd; ++i)
		{
			pat[npat*2+0] = dtCos(a)*r;
			pat[npat*2+1] = dtSin(a)*r;
			npat++;
			a += da;
		}
	}

	// Start sampling.
	dtReal cr = vmax * vmult * (1.0f - m_params.velBias);
	dtReal res[3];
	dtVset(res, dvel[0] * m_params.velBias, 0, dvel[2] * m_params.velBias);
	int ns = 0;

	const dtReal invVmult = 1.0f / vmult;
	for (int k = 0; k < depth; ++k)
	{
		dtReal minPenalty = DT_REAL_MAX;
		dtReal bvel[3];
		dtVset(bvel, 0,0,0);
		
		for (int i = 0; i < npat; ++i)
		{
			dtReal vcand[3];
			vcand[0] = res[0] + pat[i*2+0]*cr;
			vcand[1] = 0;
			vcand[2] = res[2] + pat[i*2+1]*cr;
			
			if (dtSqr(vcand[0])+dtSqr(vcand[2]) > dtSqr((vmax * vmult)+0.001f)) continue;
			
			const dtReal penalty = processSample(vcand,cr/10, pos,rad,vel,dvel, debug);
			ns++;
			if (penalty < minPenalty && penalty >= 0.0f)
			{
				minPenalty = penalty;
				dtVcopy(bvel, vcand);
			}
		}

		dtVscale(res, bvel, invVmult);

		cr *= 0.5f;
	}	
	
	dtVcopy(nvel, res);
	
	return ns;
}

