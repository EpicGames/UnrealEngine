// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DetourCrowd/DetourPathCorridor.h"
#include "Detour/DetourNavMeshQuery.h"
#include "Detour/DetourAssert.h"


int dtMergeCorridorStartMoved(dtPolyRef* path, const int npath, const int maxPath,
							  const dtPolyRef* visited, const int nvisited)
{
	int furthestPath = -1;
	int furthestVisited = -1;
	
	// Find furthest common polygon.
	for (int i = npath-1; i >= 0; --i)
	{
		bool found = false;
		for (int j = nvisited-1; j >= 0; --j)
		{
			if (path[i] == visited[j])
			{
				furthestPath = i;
				furthestVisited = j;
				found = true;
			}
		}
		if (found)
			break;
	}
	
	// If no intersection found just return current path. 
	if (furthestPath == -1 || furthestVisited == -1)
		return npath;
	
	// Concatenate paths.	
	
	// Adjust beginning of the buffer to include the visited.
	const int req = nvisited - furthestVisited;
	const int orig = dtMin(furthestPath+1, npath);
	int size = dtMax(0, npath-orig);
	if (req+size > maxPath)
		size = maxPath-req;
	if (size)
		memmove(path+req, path+orig, size*sizeof(dtPolyRef));
	
	// Store visited
	for (int i = 0; i < req; ++i)
		path[i] = visited[(nvisited-1)-i];				
	
	return req+size;
}

int dtMergeCorridorEndMoved(dtPolyRef* path, const int npath, const int maxPath,
							const dtPolyRef* visited, const int nvisited)
{
	int furthestPath = -1;
	int furthestVisited = -1;
	
	// Find furthest common polygon.
	for (int i = 0; i < npath; ++i)
	{
		bool found = false;
		for (int j = nvisited-1; j >= 0; --j)
		{
			if (path[i] == visited[j])
			{
				furthestPath = i;
				furthestVisited = j;
				found = true;
			}
		}
		if (found)
			break;
	}
	
	// If no intersection found just return current path. 
	if (furthestPath == -1 || furthestVisited == -1)
		return npath;
	
	// Concatenate paths.
	const int ppos = furthestPath+1;
	const int vpos = furthestVisited+1;
	const int count = dtMin(nvisited-vpos, maxPath-ppos);
	dtAssert(ppos+count <= maxPath);
	if (count)
		memcpy(path+ppos, visited+vpos, sizeof(dtPolyRef)*count);
	
	return ppos+count;
}

int dtMergeCorridorStartShortcut(dtPolyRef* path, const int npath, const int maxPath,
								 const dtPolyRef* visited, const int nvisited)
{
	int furthestPath = -1;
	int furthestVisited = -1;
	
	// Find furthest common polygon.
	for (int i = npath-1; i >= 0; --i)
	{
		bool found = false;
		for (int j = nvisited-1; j >= 0; --j)
		{
			if (path[i] == visited[j])
			{
				furthestPath = i;
				furthestVisited = j;
				found = true;
			}
		}
		if (found)
			break;
	}
	
	// If no intersection found just return current path. 
	if (furthestPath == -1 || furthestVisited == -1)
		return npath;
	
	// Concatenate paths.	
	
	// Adjust beginning of the buffer to include the visited.
	const int req = furthestVisited;
	if (req <= 0)
		return npath;
	
	const int orig = furthestPath;
	int size = dtMax(0, npath-orig);
	if (req+size > maxPath)
		size = maxPath-req;
	if (size)
		memmove(path+req, path+orig, size*sizeof(dtPolyRef));
	
	// Store visited
	for (int i = 0; i < req; ++i)
		path[i] = visited[i];
	
	return req+size;
}

/**
@class dtPathCorridor
@par

The corridor is loaded with a path, usually obtained from a #dtNavMeshQuery::findPath() query. The corridor
is then used to plan local movement, with the corridor automatically updating as needed to deal with inaccurate 
agent locomotion.

Example of a common use case:

-# Construct the corridor object and call #init() to allocate its path buffer.
-# Obtain a path from a #dtNavMeshQuery object.
-# Use #reset() to set the agent's current position. (At the beginning of the path.)
-# Use #setCorridor() to load the path and target.
-# Use #findCorners() to plan movement. (This handles dynamic path straightening.)
-# Use #movePosition() to feed agent movement back into the corridor. (The corridor will automatically adjust as needed.)
-# If the target is moving, use #moveTargetPosition() to update the end of the corridor. 
   (The corridor will automatically adjust as needed.)
-# Repeat the previous 3 steps to continue to move the agent.

The corridor position and target are always constrained to the navigation mesh.

One of the difficulties in maintaining a path is that floating point errors, locomotion inaccuracies, and/or local 
steering can result in the agent crossing the boundary of the path corridor, temporarily invalidating the path. 
This class uses local mesh queries to detect and update the corridor as needed to handle these types of issues. 

The fact that local mesh queries are used to move the position and target locations results in two beahviors that 
need to be considered:

Every time a move function is used there is a chance that the path will become non-optimial. Basically, the further 
the target is moved from its original location, and the further the position is moved outside the original corridor, 
the more likely the path will become non-optimal. This issue can be addressed by periodically running the 
#optimizePathTopology() and #optimizePathVisibility() methods.

All local mesh queries have distance limitations. (Review the #dtNavMeshQuery methods for details.) So the most accurate 
use case is to move the position and target in small increments. If a large increment is used, then the corridor 
may not be able to accurately find the new location.  Because of this limiation, if a position is moved in a large
increment, then compare the desired and resulting polygon references. If the two do not match, then path replanning 
may be needed.  E.g. If you move the target, check #getLastPoly() to see if it is the expected polygon.

*/

dtPathCorridor::dtPathCorridor() :
	m_moveSegAngle(0.0f),
	m_path(0),
	m_npath(0),
	m_maxPath(0)
{
}

dtPathCorridor::~dtPathCorridor()
{
	dtFree(m_path, DT_ALLOC_PERM_PATH_CORRIDOR);
}

/// @par
///
/// @warning Cannot be called more than once.
bool dtPathCorridor::init(const int maxPath)
{
	dtAssert(!m_path);
	m_path = (dtPolyRef*)dtAlloc(sizeof(dtPolyRef)*maxPath, DT_ALLOC_PERM_PATH_CORRIDOR);
	if (!m_path)
		return false;
	m_npath = 0;
	m_maxPath = maxPath;
	return true;
}

/// @par
///
/// Essentially, the corridor is set of one polygon in size with the target
/// equal to the position.
void dtPathCorridor::reset(dtPolyRef ref, const dtReal* pos)
{
	dtAssert(m_path);
	dtVcopy(m_pos, pos);
	dtVcopy(m_target, pos);
	m_path[0] = ref;
	m_npath = 1;
}

/**
@par

This is the function used to plan local movement within the corridor. One or more corners can be 
detected in order to plan movement. It performs essentially the same function as #dtNavMeshQuery::findStraightPath.

Due to internal optimizations, the maximum number of corners returned will be (@p maxCorners - 1) 
For example: If the buffers are sized to hold 10 corners, the function will never return more than 9 corners. 
So if 10 corners are needed, the buffers should be sized for 11 corners.

If the target is within range, it will be the last corner and have a polygon reference id of zero.
*/
int dtPathCorridor::findCorners(dtReal* cornerVerts, unsigned char* cornerFlags,
							  dtPolyRef* cornerPolys, const int maxCorners,
							  dtNavMeshQuery* navquery, const dtQueryFilter* filter,
							  dtReal pathOffsetDistance, dtReal earlyReachDistance, bool bAllowEarlyReach)
{
	dtAssert(m_path);
	dtAssert(m_npath);
	
	static const dtReal MIN_TARGET_DIST = 0.01f;

	dtQueryResult result;
	navquery->findStraightPath(m_pos, m_target, m_path, m_npath, result);
	
	int ncorners = dtMin(result.size(), maxCorners);
	result.copyRefs(cornerPolys, maxCorners);
	result.copyFlags(cornerFlags, maxCorners);
	result.copyPos(cornerVerts, maxCorners);

	// Prune points in the beginning of the path which are too close.
	while (ncorners)
	{
		if ((cornerFlags[0] & DT_STRAIGHTPATH_OFFMESH_CONNECTION) ||
			dtVdist2DSqr(&cornerVerts[0], m_pos) > dtSqr(MIN_TARGET_DIST))
			break;
		ncorners--;
		if (ncorners)
		{
			memmove(cornerFlags, cornerFlags+1, sizeof(unsigned char)*ncorners);
			memmove(cornerPolys, cornerPolys+1, sizeof(dtPolyRef)*ncorners);
			memmove(cornerVerts, cornerVerts+3, sizeof(dtReal)*3*ncorners);
		}
	}

	// Prune points after an off-mesh connection.
	for (int i = 0; i < ncorners; ++i)
	{
		if (cornerFlags[i] & DT_STRAIGHTPATH_OFFMESH_CONNECTION)
		{
			ncorners = i+1;
			break;
		}
	}

	// UE BEGIN
	// [UE] fixed corner for early reach detection
	if (m_hasNextExpectedCorner)
	{
		int foundIdx = -1;
		for (int i = 0; i < ncorners; i++)
		{
			// can't use equal here, expected coords are offset from corner
			const dtReal dist = dtVdistSqr(m_nextExpectedCorner, &cornerVerts[i * 3]);
			if (dist <= dtSqr(pathOffsetDistance))
			{
				foundIdx = i;
				break;
			}
		}

		if (m_hasNextExpectedCorner2 && m_isInSkipRange && foundIdx == 0)
		{
			m_hasNextExpectedCorner2 = false;
			dtVcopy(m_nextExpectedCorner, m_nextExpectedCorner2);

			foundIdx = -1;
			for (int i = 1; i < ncorners; i++)
			{
				// can't use equal here, expected coords are offset from corner
				const dtReal dist = dtVdistSqr(m_nextExpectedCorner, &cornerVerts[i * 3]);
				if (dist <= dtSqr(pathOffsetDistance))
				{
					foundIdx = i;
					break;
				}
			}
		}

		if (foundIdx > 0)
		{
			ncorners -= foundIdx;

			memmove(cornerFlags, cornerFlags + foundIdx, sizeof(unsigned char)*ncorners);
			memmove(cornerPolys, cornerPolys + foundIdx, sizeof(dtPolyRef)*ncorners);
			memmove(cornerVerts, cornerVerts + 3 * foundIdx, sizeof(dtReal)* 3 * ncorners);
		}

		m_hasNextExpectedCorner = (foundIdx >= 0);
		if (!m_hasNextExpectedCorner && m_hasNextExpectedCorner2)
		{
			m_hasNextExpectedCorner2 = false;
			dtVcopy(m_nextExpectedCorner, m_nextExpectedCorner2);
		}
	}

	// [UE] Offset path points from corners
	const dtNavMesh* nav = navquery->getAttachedNavMesh();
	dtReal v1[3], v2[3], dir[3];

	for (int i = 0; i < ncorners - 1; i++)
	{
		int fromIdx = 0;
		CA_SUPPRESS(6385);
		while (fromIdx < m_npath && m_path[fromIdx] != cornerPolys[i])
		{
			fromIdx++;
		}

		if (m_path[fromIdx] != cornerPolys[i] || fromIdx == 0)
			continue;

		const dtMeshTile* tile0 = 0;
		const dtMeshTile* tile1 = 0;
		const dtPoly* poly0 = 0;
		const dtPoly* poly1 = 0;

		nav->getTileAndPolyByRefUnsafe(m_path[fromIdx - 1], &tile0, &poly0);
		nav->getTileAndPolyByRefUnsafe(m_path[fromIdx], &tile1, &poly1);

		if (tile0 != tile1)
			continue;

		dtReal* corner = &cornerVerts[i * 3];

		unsigned char dummyT1, dummyT2;
		navquery->getPortalPoints(m_path[fromIdx - 1], m_path[fromIdx], v1, v2, dummyT1, dummyT2);

		const dtReal edgeLen = dtVdist(v1, v2);
		if (edgeLen > 0.001f)
		{
			const dtReal edgeOffset = dtMin(pathOffsetDistance, edgeLen * 0.75f) / edgeLen;

			if (dtVequal(corner, v1))
			{
				dtVsub(dir, v2, v1);
				dtVmad(corner, corner, dir, edgeOffset);
			}
			else
			{
				dtVsub(dir, v1, v2);
				dtVmad(corner, corner, dir, edgeOffset);
			}
		}
	}

	// [UE] Dynamic acceptance radius, depending on angle between first two segments of path
	if (m_enableEarlyReach)
	{
		const bool bSame = dtVequal(m_prevMovePoint, cornerVerts);
		dtVcopy(m_prevMovePoint, cornerVerts);
		m_isInSkipRange = false;

		const dtReal segAngleThr = 0.8f;
		if (!bSame && ncorners > 1)
		{
			dtReal seg1[3], seg2[3];
			dtVsub(seg1, &cornerVerts[0], m_pos);
			dtVsub(seg2, &cornerVerts[3], &cornerVerts[0]);
			dtVnormalize(seg1);
			dtVnormalize(seg2);

			m_moveSegAngle = dtVdot2D(seg1, seg2);
			if (m_moveSegAngle > segAngleThr)
			{
				// prepare for skipping to forced corner
				if (m_hasNextExpectedCorner)
				{
					dtVcopy(m_nextExpectedCorner2, &cornerVerts[3]);
					m_hasNextExpectedCorner2 = true;
				}
				else
				{
					dtVcopy(m_nextExpectedCorner, &cornerVerts[3]);
				}
			}
		}

		if (m_moveSegAngle > segAngleThr && bAllowEarlyReach)
		{
			dtReal seg[3];
			dtVsub(seg, &cornerVerts[0], m_pos);
			const dtReal distToFirstVert = dtVlenSqr(seg);

			const dtReal skipThreshold = dtSqr(earlyReachDistance);
			if (distToFirstVert < skipThreshold)
			{
				// skip to known corner (in next tick)
				m_hasNextExpectedCorner = true;
				m_isInSkipRange = true;
			}
		}
	}
	// UE END

	return ncorners;
}

/** 
@par

Inaccurate locomotion or dynamic obstacle avoidance can force the argent position significantly outside the 
original corridor. Over time this can result in the formation of a non-optimal corridor. Non-optimal paths can 
also form near the corners of tiles.

This function uses an efficient local visibility search to try to optimize the corridor 
between the current position and @p next.

The corridor will change only if @p next is visible from the current position and moving directly toward the point 
is better than following the existing path.

The more inaccurate the agent movement, the more beneficial this function becomes. Simply adjust the frequency 
of the call to match the needs to the agent.

This function is not suitable for long distance searches.
*/
bool dtPathCorridor::optimizePathVisibility(const dtReal* next, const dtReal pathOptimizationRange,
										  dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	dtAssert(m_path);
	
	// Clamp the ray to max distance.
	dtReal goal[3];
	dtVcopy(goal, next);
	dtReal dist = dtVdist2D(m_pos, goal);
	
	// If too close to the goal, do not try to optimize.
	if (dist < 0.01f)
		return true;
	
	// Overshoot a little. This helps to optimize open fields in tiled meshes.
	// UE: changes to ray adjustment - make sure it's not going further than newDist
	dtReal newDist = dtMin(dist+0.01f, pathOptimizationRange);
	
	// Adjust ray length.
	dtReal delta[3];
	dtVsub(delta, goal, m_pos);
	dtVmad(goal, m_pos, delta, newDist / dist);
	
	static const int MAX_RES = 32;
	dtPolyRef res[MAX_RES];
	dtReal t, norm[3];
	int nres = 0;
	navquery->raycast(m_path[0], m_pos, goal, filter, &t, norm, res, &nres, MAX_RES);
	if (nres > 1 && t > 0.99f)
	{
		m_npath = dtMergeCorridorStartShortcut(m_path, m_npath, m_maxPath, res, nres);
		return true;
	}

	return false;
}

/**
@par

Inaccurate locomotion or dynamic obstacle avoidance can force the agent position significantly outside the 
original corridor. Over time this can result in the formation of a non-optimal corridor. This function will use a 
local area path search to try to re-optimize the corridor.

The more inaccurate the agent movement, the more beneficial this function becomes. Simply adjust the frequency of 
the call to match the needs to the agent.
*/
bool dtPathCorridor::optimizePathTopology(dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	dtAssert(navquery);
	dtAssert(filter);
	dtAssert(m_path);
	
	if (m_npath < 3)
		return false;
	
	static const int MAX_ITER = 32;
	static const int MAX_RES = 32;
	
	dtPolyRef res[MAX_RES];
	int nres = 0;
	const dtReal costLimit = DT_REAL_MAX; //@UE
	const bool requireNavigableEndLocation = true; //@UE
	navquery->initSlicedFindPath(m_path[0], m_path[m_npath-1], m_pos, m_target, costLimit, requireNavigableEndLocation, filter); //@UE
	navquery->updateSlicedFindPath(MAX_ITER, 0);
	dtStatus status = navquery->finalizeSlicedFindPathPartial(m_path, m_npath, res, &nres, MAX_RES);
	
	if (dtStatusSucceed(status) && nres > 0)
	{
		m_npath = dtMergeCorridorStartShortcut(m_path, m_npath, m_maxPath, res, nres);
		return true;
	}
	
	return false;
}

bool dtPathCorridor::moveOverOffmeshConnection(dtPolyRef offMeshConRef, dtPolyRef* refs,
											   const dtReal* agentPos, dtReal* startPos, dtReal* endPos,
											   dtNavMeshQuery* navquery)
{
	dtAssert(navquery);
	dtAssert(m_path);
	dtAssert(m_npath);

	// Advance the path up to and over the off-mesh connection.
	dtPolyRef prevRef = 0, polyRef = m_path[0];
	int npos = 0;
	while (npos < m_npath && polyRef != offMeshConRef)
	{
		prevRef = polyRef;
		polyRef = m_path[npos];
		npos++;
	}
	if (npos == m_npath)
	{
		// Could not find offMeshConRef
		return false;
	}
	
	// Prune path
	for (int i = npos; i < m_npath; ++i)
		m_path[i-npos] = m_path[i];
	m_npath -= npos;

	refs[0] = prevRef;
	refs[1] = polyRef;
	
	const dtNavMesh* nav = navquery->getAttachedNavMesh();
	dtAssert(nav);

	dtStatus status = nav->getOffMeshConnectionPolyEndPoints(refs[0], refs[1], agentPos, startPos, endPos);
	if (dtStatusSucceed(status))
	{
		dtVcopy(m_pos, endPos);
		return true;
	}

	return false;
}

bool dtPathCorridor::canMoveOverOffmeshConnection(dtPolyRef offMeshConRef, dtPolyRef* refs,
	const dtReal* agentPos, dtReal* startPos, dtReal* endPos,
	dtNavMeshQuery* navquery) const
{
	dtAssert(navquery);
	dtAssert(m_path);
	dtAssert(m_npath);

	// Advance the path up to and over the off-mesh connection.
	dtPolyRef prevRef = 0, polyRef = m_path[0];
	int npos = 0;
	while (npos < m_npath && polyRef != offMeshConRef)
	{
		prevRef = polyRef;
		polyRef = m_path[npos];
		npos++;
	}
	if (npos == m_npath)
	{
		// Could not find offMeshConRef
		return false;
	}

	refs[0] = prevRef;
	refs[1] = polyRef;

	const dtNavMesh* nav = navquery->getAttachedNavMesh();
	dtAssert(nav);

	dtStatus status = nav->getOffMeshConnectionPolyEndPoints(refs[0], refs[1], agentPos, startPos, endPos);
	if (dtStatusSucceed(status))
	{
		return true;
	}

	return true;
}

void dtPathCorridor::pruneOffmeshConenction(dtPolyRef offMeshConRef)
{
	dtAssert(m_path);
	dtAssert(m_npath);

	// Advance the path up to and over the off-mesh connection.
	dtPolyRef polyRef = m_path[0];
	int npos = 0;
	while (npos < m_npath && polyRef != offMeshConRef)
	{
		polyRef = m_path[npos];
		npos++;
	}
	
	// Prune path
	if (npos != m_npath)
	{
		for (int i = npos; i < m_npath; ++i)
			m_path[i - npos] = m_path[i];
		m_npath -= npos;
	}
}


/**
@par

Behavior:

- The movement is constrained to the surface of the navigation mesh. 
- The corridor is automatically adjusted (shorted or lengthened) in order to remain valid. 
- The new position will be located in the adjusted corridor's first polygon.

The expected use case is that the desired position will be 'near' the current corridor. What is considered 'near' 
depends on local polygon density, query search extents, etc.

The resulting position will differ from the desired position if the desired position is not on the navigation mesh, 
or it can't be reached using a local search.
*/
bool dtPathCorridor::movePosition(const dtReal* npos, dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	dtAssert(m_path);
	dtAssert(m_npath);
	
	// Move along navmesh and update new position.
	dtReal result[3];
	static const int MAX_VISITED = 16;
	dtPolyRef visited[MAX_VISITED];
	int nvisited = 0;

	// [UE: check status, it may fail due to runtime navmesh rebuild]
	const dtStatus status = navquery->moveAlongSurface(m_path[0], m_pos, npos, filter, result, visited, &nvisited, MAX_VISITED);
	if (dtStatusFailed(status))
	{
		return false;
	}

	m_npath = dtMergeCorridorStartMoved(m_path, m_npath, m_maxPath, visited, nvisited);
	
	// Adjust the position to stay on top of the navmesh.
	dtReal h = m_pos[1];
	navquery->getPolyHeight(m_path[0], result, &h);
	result[1] = h;
	dtVcopy(m_pos, result);

	return true;
}

/**
@par

Behavior:

- The movement is constrained to the surface of the navigation mesh. 
- The corridor is automatically adjusted (shorted or lengthened) in order to remain valid. 
- The new target will be located in the adjusted corridor's last polygon.

The expected use case is that the desired target will be 'near' the current corridor. What is considered 'near' depends on local polygon density, query search extents, etc.

The resulting target will differ from the desired target if the desired target is not on the navigation mesh, or it can't be reached using a local search.
*/
void dtPathCorridor::moveTargetPosition(const dtReal* npos, dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	dtAssert(m_path);
	dtAssert(m_npath);
	
	// Move along navmesh and update new position.
	dtReal result[3];
	static const int MAX_VISITED = 16;
	dtPolyRef visited[MAX_VISITED];
	int nvisited = 0;
	navquery->moveAlongSurface(m_path[m_npath-1], m_target, npos, filter,
							   result, visited, &nvisited, MAX_VISITED);
	m_npath = dtMergeCorridorEndMoved(m_path, m_npath, m_maxPath, visited, nvisited);
	
	// TODO: should we do that?
	// Adjust the position to stay on top of the navmesh.
	/*	dtReal h = m_target[1];
	 navquery->getPolyHeight(m_path[m_npath-1], result, &h);
	 result[1] = h;*/
	
	dtVcopy(m_target, result);
}

/// @par
///
/// The current corridor position is expected to be within the first polygon in the path. The target 
/// is expected to be in the last polygon. 
/// 
/// @warning The size of the path must not exceed the size of corridor's path buffer set during #init().
void dtPathCorridor::setCorridor(const dtReal* target, const dtPolyRef* path, const int npath)
{
	dtAssert(m_path);
	dtAssert(npath > 0);
	dtAssert(npath < m_maxPath);
	
	dtVcopy(m_target, target);
	memcpy(m_path, path, sizeof(dtPolyRef)*npath);
	m_npath = npath;
	m_hasNextExpectedCorner = false;
	m_hasNextExpectedCorner2 = false;
	m_isInSkipRange = false;
}

bool dtPathCorridor::fixPathStart(dtPolyRef safeRef, const dtReal* safePos)
{
	dtAssert(m_path);

	dtVcopy(m_pos, safePos);
	if (m_npath < 3 && m_npath > 0)
	{
		m_path[2] = m_path[m_npath-1];
		m_path[0] = safeRef;
		m_path[1] = 0;
		m_npath = 3;
	}
	else
	{
		m_path[0] = safeRef;
		m_path[1] = 0;
	}
	
	return true;
}

bool dtPathCorridor::trimInvalidPath(dtPolyRef safeRef, const dtReal* safePos,
									 dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	dtAssert(navquery);
	dtAssert(filter);
	dtAssert(m_path);
	
	// Keep valid path as far as possible.
	int n = 0;
	while (n < m_npath && navquery->isValidPolyRef(m_path[n], filter)) {
		n++;
	}
	
	if (n == m_npath)
	{
		// All valid, no need to fix.
		return true;
	}
	else if (n == 0)
	{
		// The first polyref is bad, use current safe values.
		dtVcopy(m_pos, safePos);
		m_path[0] = safeRef;
		m_npath = 1;
	}
	else
	{
		// The path is partially usable.
		m_npath = n;
	}
	
	// Clamp target pos to last poly
	dtReal tgt[3];
	dtVcopy(tgt, m_target);
	navquery->closestPointOnPolyBoundary(m_path[m_npath-1], tgt, m_target);
	
	return true;
}

/// @par
///
/// The path can be invalidated if there are structural changes to the underlying navigation mesh, or the state of 
/// a polygon within the path changes resulting in it being filtered out. (E.g. An exclusion or inclusion flag changes.)
bool dtPathCorridor::isValid(const int maxLookAhead, dtNavMeshQuery* navquery, const dtQueryFilter* filter)
{
	// Check that all polygons still pass query filter.
	const int n = dtMin(m_npath, maxLookAhead);
	for (int i = 0; i < n; ++i)
	{
		if (!navquery->isValidPolyRef(m_path[i], filter))
			return false;
	}

	return true;
}
