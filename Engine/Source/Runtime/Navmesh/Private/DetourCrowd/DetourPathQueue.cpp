// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DetourCrowd/DetourPathQueue.h"


dtPathQueue::dtPathQueue() :
	m_navquery(0),
	m_nextHandle(1),
	m_maxPathSize(0),
	m_queueHead(0)
{
	for (int i = 0; i < MAX_QUEUE; ++i)
		m_queue[i].path = 0;
}

dtPathQueue::~dtPathQueue()
{
	purge();
}

void dtPathQueue::purge()
{
	dtFreeNavMeshQuery(m_navquery);
	m_navquery = 0;
	for (int i = 0; i < MAX_QUEUE; ++i)
	{
		dtFree(m_queue[i].path, DT_ALLOC_PERM_PATH_QUEUE);
		m_queue[i].path = 0;
	}
}

bool dtPathQueue::init(const int maxPathSize, const int maxSearchNodeCount, dtNavMesh* nav)
{
	purge();

	m_navquery = dtAllocNavMeshQuery();
	if (!m_navquery)
		return false;
	if (dtStatusFailed(m_navquery->init(nav, maxSearchNodeCount)))
		return false;
	
	m_maxPathSize = maxPathSize;
	for (int i = 0; i < MAX_QUEUE; ++i)
	{
		m_queue[i].ref = DT_PATHQ_INVALID;
		m_queue[i].path = (dtPolyRef*)dtAlloc(sizeof(dtPolyRef)*m_maxPathSize, DT_ALLOC_PERM_PATH_QUEUE);
		if (!m_queue[i].path)
			return false;
	}
	
	m_queueHead = 0;
	
	return true;
}

void dtPathQueue::update(const int maxIters)
{
	static const int MAX_KEEP_ALIVE = 2; // in update ticks.

	// Update path request until there is nothing to update
	// or upto maxIters pathfinder iterations has been consumed.
	int iterCount = maxIters;
	
	for (int i = 0; i < MAX_QUEUE; ++i)
	{
		PathQuery& q = m_queue[m_queueHead % MAX_QUEUE];
		
		// Skip inactive requests.
		if (q.ref == DT_PATHQ_INVALID)
		{
			m_queueHead++;
			continue;
		}
		
		// Handle completed request.
		if (dtStatusSucceed(q.status) || dtStatusFailed(q.status))
		{
			// If the path result has not been read in few frames, free the slot.
			q.keepAlive++;
			if (q.keepAlive > MAX_KEEP_ALIVE)
			{
				q.ref = DT_PATHQ_INVALID;
				q.status = 0;
			}
			
			m_queueHead++;
			continue;
		}

		m_navquery->updateLinkFilter(q.linkFilter.Get());

		// Handle query start.
		if (q.status == 0)
		{
			q.status = m_navquery->initSlicedFindPath(q.startRef, q.endRef, q.startPos, q.endPos, q.costLimit, q.requireNavigableEndLocation, q.filter); //@UE
		}		
		// Handle query in progress.
		if (dtStatusInProgress(q.status))
		{
			int iters = 0;
			q.status = m_navquery->updateSlicedFindPath(iterCount, &iters);
			iterCount -= iters;
		}
		if (dtStatusSucceed(q.status))
		{
			q.status = m_navquery->finalizeSlicedFindPath(q.path, &q.npath, m_maxPathSize);
		}

		if (iterCount <= 0)
			break;

		m_queueHead++;
	}
}

dtPathQueueRef dtPathQueue::request(dtPolyRef startRef, dtPolyRef endRef,
									const dtReal* startPos, const dtReal* endPos, const dtReal costLimit, const bool requireNavigableEndLocation, //@UE
									const dtQueryFilter* filter, TSharedPtr<dtQuerySpecialLinkFilter> linkFilter)
{
	// Find empty slot
	int slot = -1;
	for (int i = 0; i < MAX_QUEUE; ++i)
	{
		if (m_queue[i].ref == DT_PATHQ_INVALID)
		{
			slot = i;
			break;
		}
	}
	// Could not find slot.
	if (slot == -1)
		return DT_PATHQ_INVALID;
	
	dtPathQueueRef ref = m_nextHandle++;
	if (m_nextHandle == DT_PATHQ_INVALID) m_nextHandle++;
	
	PathQuery& q = m_queue[slot];
	q.ref = ref;
	dtVcopy(q.startPos, startPos);
	q.startRef = startRef;
	dtVcopy(q.endPos, endPos);
	q.endRef = endRef;
	q.costLimit = costLimit; //@UE
	q.requireNavigableEndLocation = requireNavigableEndLocation; //@UE
	
	q.status = 0;
	q.npath = 0;
	q.filter = filter;
	q.linkFilter = linkFilter;
	q.keepAlive = 0;
	
	return ref;
}

dtStatus dtPathQueue::getRequestStatus(dtPathQueueRef ref) const
{
	for (int i = 0; i < MAX_QUEUE; ++i)
	{
		if (m_queue[i].ref == ref)
			return m_queue[i].status;
	}
	return DT_FAILURE;
}

dtStatus dtPathQueue::getPathResult(dtPathQueueRef ref, dtPolyRef* path, int* pathSize, const int maxPath)
{
	for (int i = 0; i < MAX_QUEUE; ++i)
	{
		if (m_queue[i].ref == ref)
		{
			PathQuery& q = m_queue[i];
			// Free request for reuse.
			q.ref = DT_PATHQ_INVALID;
			q.status = 0;
			// Copy path
			int n = dtMin(q.npath, maxPath);
			memcpy(path, q.path, sizeof(dtPolyRef)*n);
			*pathSize = n;
			return DT_SUCCESS;
		}
	}
	return DT_FAILURE;
}
