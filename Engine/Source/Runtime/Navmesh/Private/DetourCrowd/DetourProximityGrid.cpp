// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DetourCrowd/DetourProximityGrid.h"
#include "Detour/DetourCommon.h"
#include "Detour/DetourAlloc.h"
#include "Detour/DetourAssert.h"


dtProximityGrid* dtAllocProximityGrid()
{
	void* mem = dtAlloc(sizeof(dtProximityGrid), DT_ALLOC_PERM_PROXIMITY_GRID);
	if (!mem) return 0;
	return new(mem) dtProximityGrid;
}

void dtFreeProximityGrid(dtProximityGrid* ptr)
{
	if (!ptr) return;
	ptr->~dtProximityGrid();
	dtFree(ptr, DT_ALLOC_PERM_PROXIMITY_GRID);
}


inline int hashPos2(int x, int y, int n)
{
	return ((x*73856093) ^ (y*19349663)) & (n-1);
}


dtProximityGrid::dtProximityGrid() :
	m_cellSize(0),
	m_pool(0),
	m_poolHead(0),
	m_poolSize(0),
	m_buckets(0),
	m_bucketsSize(0)
{
}

dtProximityGrid::~dtProximityGrid()
{
	dtFree(m_buckets, DT_ALLOC_PERM_PROXIMITY_GRID);
	dtFree(m_pool, DT_ALLOC_PERM_PROXIMITY_GRID);
}

bool dtProximityGrid::init(const int poolSize, const dtReal cellSize)
{
	dtAssert(poolSize > 0);
	dtAssert(cellSize > 0.0f);
	
	m_cellSize = cellSize;
	m_invCellSize = 1.0f / m_cellSize;
	
	// Allocate hashs buckets
	m_bucketsSize = dtNextPow2(poolSize);
	m_buckets = (unsigned short*)dtAlloc(sizeof(unsigned short)*m_bucketsSize, DT_ALLOC_PERM_PROXIMITY_GRID);
	if (!m_buckets)
		return false;
	
	// Allocate pool of items.
	m_poolSize = poolSize;
	m_poolHead = 0;
	m_pool = (Item*)dtAlloc(sizeof(Item)*m_poolSize, DT_ALLOC_PERM_PROXIMITY_GRID);
	if (!m_pool)
		return false;
	
	clear();
	
	return true;
}

void dtProximityGrid::clear()
{
	memset(m_buckets, 0xff, sizeof(unsigned short)*m_bucketsSize);
	m_poolHead = 0;
	m_bounds[0] = 0xffff;
	m_bounds[1] = 0xffff;
	m_bounds[2] = -0xffff;
	m_bounds[3] = -0xffff;
}

void dtProximityGrid::addItem(const unsigned short id,
							  const dtReal minx, const dtReal miny,
							  const dtReal maxx, const dtReal maxy)
{
	const int iminx = (int)dtFloor(minx * m_invCellSize);
	const int iminy = (int)dtFloor(miny * m_invCellSize);
	const int imaxx = (int)dtFloor(maxx * m_invCellSize);
	const int imaxy = (int)dtFloor(maxy * m_invCellSize);
	
	m_bounds[0] = dtMin(m_bounds[0], iminx);
	m_bounds[1] = dtMin(m_bounds[1], iminy);
	m_bounds[2] = dtMax(m_bounds[2], imaxx);
	m_bounds[3] = dtMax(m_bounds[3], imaxy);
	
	for (int y = iminy; y <= imaxy; ++y)
	{
		for (int x = iminx; x <= imaxx; ++x)
		{
			if (m_poolHead < m_poolSize)
			{
				const int h = hashPos2(x, y, m_bucketsSize);
				const unsigned short idx = (unsigned short)m_poolHead;
				m_poolHead++;
				Item& item = m_pool[idx];
				item.x = (short)x;
				item.y = (short)y;
				item.id = id;
				item.next = m_buckets[h];
				m_buckets[h] = idx;
			}
		}
	}
}

int dtProximityGrid::queryItems(const dtReal minx, const dtReal miny,
								const dtReal maxx, const dtReal maxy,
								unsigned short* ids, const int maxIds) const
{
	const int iminx = (int)dtFloor(minx * m_invCellSize);
	const int iminy = (int)dtFloor(miny * m_invCellSize);
	const int imaxx = (int)dtFloor(maxx * m_invCellSize);
	const int imaxy = (int)dtFloor(maxy * m_invCellSize);
	
	int n = 0;
	
	for (int y = iminy; y <= imaxy; ++y)
	{
		for (int x = iminx; x <= imaxx; ++x)
		{
			const int h = hashPos2(x, y, m_bucketsSize);
			unsigned short idx = m_buckets[h];
			while (idx != 0xffff)
			{
				Item& item = m_pool[idx];
				if ((int)item.x == x && (int)item.y == y)
				{
					// Check if the id exists already.
					const unsigned short* end = ids + n;
					unsigned short* i = ids;
					while (i != end && *i != item.id)
						++i;
					// Item not found, add it.
					if (i == end)
					{
						if (n >= maxIds)
							return n;
						ids[n++] = item.id;
					}
				}
				idx = item.next;
			}
		}
	}
	
	return n;
}

int dtProximityGrid::getItemCountAt(const int x, const int y) const
{
	int n = 0;
	
	const int h = hashPos2(x, y, m_bucketsSize);
	unsigned short idx = m_buckets[h];
	while (idx != 0xffff)
	{
		Item& item = m_pool[idx];
		if ((int)item.x == x && (int)item.y == y)
			n++;
		idx = item.next;
	}
	
	return n;
}
