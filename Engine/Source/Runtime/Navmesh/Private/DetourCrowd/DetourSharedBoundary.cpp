// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetourCrowd/DetourSharedBoundary.h"

void dtSharedBoundary::Initialize()
{
	CurrentTime = 0.0f;
	NextClearTime = 0.0f;

	for (int32 Idx = 0; Idx < DT_MAX_AREAS; Idx++)
	{
		SingleAreaFilter.setAreaCost(Idx, DT_UNWALKABLE_POLY_COST);
	}
}

void dtSharedBoundary::Tick(dtReal DeltaTime)
{
	CurrentTime += DeltaTime;
	
	// clear unused entries
	if (CurrentTime > NextClearTime)
	{
		const dtReal MaxLifeTime = 2.0f;
		NextClearTime = CurrentTime + MaxLifeTime;

		for (TSparseArray<dtSharedBoundaryData>::TIterator It(Data); It; ++It)
		{
			const dtReal LastAccess = CurrentTime - It->AccessTime;
			if (LastAccess >= MaxLifeTime)
			{
				It.RemoveCurrent();
			}
		}
	}
}

int32 dtSharedBoundary::CacheData(dtReal* Center, dtReal Radius, dtPolyRef CenterPoly, dtNavMeshQuery* NavQuery, dtQueryFilter* NavFilter)
{
	// bail if requested poly is not valid (e.g. rebuild in progress)
	if (NavQuery && !NavQuery->isValidPolyRef(CenterPoly, NavFilter))
	{
		return -1;
	}

	Radius *= 1.5f;

	int32 DataIdx = FindData(Center, Radius, CenterPoly, NavFilter);
	const bool bHasValidData = IsValid(DataIdx, NavQuery, NavFilter);
	if (!bHasValidData)
	{
		if (DataIdx >= 0)
		{
			// remove in next cleanup
			Data[DataIdx].AccessTime = 0.0f;
		}

		dtSharedBoundaryData NewData;
		dtVcopy(NewData.Center, Center);
		NewData.Radius = Radius;
		NewData.Filter = NavFilter;
		NewData.SingleAreaId = 0;

		FindEdges(NewData, CenterPoly, NavQuery, NavFilter);
		DataIdx = Data.Add(NewData);
	}

	Data[DataIdx].AccessTime = CurrentTime;
	return DataIdx;
}

int32 dtSharedBoundary::CacheData(dtReal* Center, dtReal Radius, dtPolyRef CenterPoly, dtNavMeshQuery* NavQuery, uint8 SingleAreaId)
{
	SingleAreaFilter.setAreaCost(SingleAreaId, 1.0f);

	// bail if requested poly is not valid (e.g. rebuild in progress)
	if (NavQuery && !NavQuery->isValidPolyRef(CenterPoly, &SingleAreaFilter))
	{
		return -1;
	}

	Radius *= 1.5f;

	int32 DataIdx = FindData(Center, Radius, CenterPoly, SingleAreaId);
	const bool bHasValidData = IsValid(DataIdx, NavQuery, &SingleAreaFilter);
	if (!bHasValidData)
	{
		if (DataIdx >= 0)
		{
			// remove in next cleanup
			Data[DataIdx].AccessTime = 0.0f;
		}

		dtSharedBoundaryData NewData;
		dtVcopy(NewData.Center, Center);
		NewData.Radius = Radius;
		NewData.SingleAreaId = SingleAreaId;

		FindEdges(NewData, CenterPoly, NavQuery, &SingleAreaFilter);
		DataIdx = Data.Add(NewData);
	}

	SingleAreaFilter.setAreaCost(SingleAreaId, DT_UNWALKABLE_POLY_COST);
	Data[DataIdx].AccessTime = CurrentTime;
	return DataIdx;
}

void dtSharedBoundary::FindEdges(dtSharedBoundaryData& SharedData, dtPolyRef CenterPoly, dtNavMeshQuery* NavQuery, dtQueryFilter* NavFilter)
{
	const int32 MaxWalls = 64;
	int32 NumWalls = 0;
	dtReal WallSegments[MaxWalls * 3 * 2] = { 0 };
	dtPolyRef WallPolys[MaxWalls * 2] = { 0 };

	const int32 MaxNeis = 64;
	int32 NumNeis = 0;
	dtPolyRef NeiPolys[MaxNeis] = { 0 };

	NavQuery->findWallsInNeighbourhood(CenterPoly, SharedData.Center, SharedData.Radius, NavFilter,
		NeiPolys, &NumNeis, MaxNeis, WallSegments, WallPolys, &NumWalls, MaxWalls);
	
	dtSharedBoundaryEdge NewEdge;
	for (int32 Idx = 0; Idx < NumWalls; Idx++)
	{
		dtVcopy(NewEdge.v0, &WallSegments[Idx * 6]);
		dtVcopy(NewEdge.v1, &WallSegments[Idx * 6 + 3]);
		NewEdge.p0 = WallPolys[Idx * 2];
		NewEdge.p1 = WallPolys[Idx * 2 + 1];

		SharedData.Edges.Add(NewEdge);
	}

	SharedData.Polys.Reserve(NumNeis);
	for (int32 Idx = 0; Idx < NumNeis; Idx++)
	{
		SharedData.Polys.Add(NeiPolys[Idx]);
	}
}

int32 dtSharedBoundary::FindData(dtReal* Center, dtReal Radius, dtPolyRef ReqPoly, dtQueryFilter* NavFilter) const
{
	const dtReal RadiusThr = 50.0f;
	const dtReal DistThrSq = dtSqr(Radius * 0.5f);

	for (int32 Idx = 0; Idx < Data.Num(); Idx++)
	{
		if (Data[Idx].Filter == NavFilter)
		{
			const dtReal DistSq = dtVdistSqr(Center, Data[Idx].Center);
			if (DistSq <= DistThrSq && dtAbs(Data[Idx].Radius - Radius) < RadiusThr)
			{
				if (Data[Idx].Polys.Contains(ReqPoly))
				{
					return Idx;
				}
			}
		}
	}

	return -1;
}

int32 dtSharedBoundary::FindData(dtReal* Center, dtReal Radius, dtPolyRef ReqPoly, uint8 SingleAreaId) const
{
	const dtReal DistThrSq = dtSqr(Radius * 0.5f);
	const dtReal RadiusThr = 50.0f;

	for (int32 Idx = 0; Idx < Data.Num(); Idx++)
	{
		if (Data[Idx].SingleAreaId == SingleAreaId)
		{
			const dtReal DistSq = dtVdistSqr(Center, Data[Idx].Center);
			if (DistSq <= DistThrSq && dtAbs(Data[Idx].Radius - Radius) < RadiusThr)
			{
				if (Data[Idx].Polys.Contains(ReqPoly))
				{
					return Idx;
				}
			}
		}
	}

	return -1;
}

bool dtSharedBoundary::HasSample(int32 Idx) const
{
	return (Idx >= 0) && (Idx < Data.GetMaxIndex()) && Data.IsAllocated(Idx);
}

bool dtSharedBoundary::IsValid(int32 Idx, dtNavMeshQuery* NavQuery, dtQueryFilter* NavFilter) const
{
	bool bValid = HasSample(Idx);
	if (bValid)
	{
		for (auto It = Data[Idx].Polys.CreateConstIterator(); It; ++It)
		{
			const dtPolyRef TestRef = *It;
			const bool bValidRef = NavQuery->isValidPolyRef(TestRef, NavFilter);
			if (!bValidRef)
			{
				bValid = false;
				break;
			}
		}
	}

	return bValid;
}
