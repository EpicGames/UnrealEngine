// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DetourTileCache/DetourTileCache.h"
#include "DetourTileCache/DetourTileCacheBuilder.h"
#include "Detour/DetourNavMeshBuilder.h"
#include "Detour/DetourNavMesh.h"
#include "Detour/DetourCommon.h"
#include "Detour/DetourAssert.h"

/// Region partitioning methods
/// @see rcConfig
enum dtRegionPartitioning
{
	DT_REGION_MONOTONE,		///< monotone partitioning
	DT_REGION_WATERSHED,	///< watershed partitioning
	DT_REGION_CHUNKY,		///< monotone partitioning on small chunks
};

dtTileCache* dtAllocTileCache()
{
	void* mem = dtAlloc(sizeof(dtTileCache), DT_ALLOC_PERM_TILE_DATA);
	if (!mem) return 0;
	return new(mem) dtTileCache;
}

void dtFreeTileCache(dtTileCache* tc)
{
	if (!tc) return;
	tc->~dtTileCache();
	dtFree(tc, DT_ALLOC_PERM_TILE_DATA);
}

static bool contains(const dtCompressedTileRef* a, const int n, const dtCompressedTileRef v)
{
	for (int i = 0; i < n; ++i)
		if (a[i] == v)
			return true;
	return false;
}

namespace TileCacheFunc
{
	inline int computeTileHash(int x, int y, const int mask)
	{
		const unsigned int h1 = 0x8da6b343; // Large multiplicative constants;
		const unsigned int h2 = 0xd8163841; // here arbitrarily chosen primes
		unsigned int n = h1 * x + h2 * y;
		return (int)(n & mask);
	}
}

struct BuildContext
{
	inline BuildContext(struct dtTileCacheAlloc* a)
		: layer(0)
		, dfield(0)
		, lcset(0)
		//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
		, lclusters(0)
#endif //WITH_NAVMESH_CLUSTER_LINKS
		//@UE END
		, lmesh(0)
		, alloc(a)
	{}
	inline ~BuildContext() { purge(); }
	void purge()
	{
		dtFreeTileCacheLayer(alloc, layer);
		layer = 0;
		dtFreeTileCacheDistanceField(alloc, dfield);
		dfield = 0;
		dtFreeTileCacheContourSet(alloc, lcset);
		lcset = 0;
		//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
		dtFreeTileCacheClusterSet(alloc, lclusters);
		lclusters = 0;
#endif //WITH_NAVMESH_CLUSTER_LINKS
		//@UE END
		dtFreeTileCachePolyMesh(alloc, lmesh);
		lmesh = 0;
	}
	struct dtTileCacheLayer* layer;
	struct dtTileCacheDistanceField* dfield;
	struct dtTileCacheContourSet* lcset;
	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	struct dtTileCacheClusterSet* lclusters;
#endif //WITH_NAVMESH_CLUSTER_LINKS
	//@UE END
	struct dtTileCachePolyMesh* lmesh;
	struct dtTileCacheAlloc* alloc;
};


dtTileCache::dtTileCache() :
	m_tileLutSize(0),
	m_tileLutMask(0),
	m_posLookup(0),
	m_nextFreeTile(0),	
	m_tiles(0),	
	m_saltBits(0),
	m_tileBits(0),
	m_talloc(0),
	m_tcomp(0),
	m_tmproc(0),
	m_obstacles(0),
	m_nextFreeObstacle(0),
	m_nreqs(0),
	m_nupdate(0)
{
	memset(&m_params, 0, sizeof(m_params));
}
	
dtTileCache::~dtTileCache()
{
	for (int i = 0; i < m_params.maxTiles; ++i)
	{
		if (m_tiles[i].flags & DT_COMPRESSEDTILE_FREE_DATA)
		{
			dtFree(m_tiles[i].data, DT_ALLOC_PERM_TILE_DATA);
			m_tiles[i].data = 0;
		}
	}
	dtFree(m_obstacles, DT_ALLOC_PERM_TILE_DATA);
	m_obstacles = 0;
	dtFree(m_posLookup, DT_ALLOC_PERM_TILE_DATA);
	m_posLookup = 0;
	dtFree(m_tiles, DT_ALLOC_PERM_TILE_DATA);
	m_tiles = 0;
	m_nreqs = 0;
	m_nupdate = 0;
}

const dtCompressedTile* dtTileCache::getTileByRef(dtCompressedTileRef ref) const
{
	if (!ref)
		return 0;
	unsigned int tileIndex = decodeTileIdTile(ref);
	unsigned int tileSalt = decodeTileIdSalt(ref);
	if ((int)tileIndex >= m_params.maxTiles)
		return 0;
	const dtCompressedTile* tile = &m_tiles[tileIndex];
	if (tile->salt != tileSalt)
		return 0;
	return tile;
}


dtStatus dtTileCache::init(const dtTileCacheParams* params,
						   dtTileCacheAlloc* talloc,
						   dtTileCacheCompressor* tcomp,
						   dtTileCacheMeshProcess* tmproc)
{
	m_talloc = talloc;
	m_tcomp = tcomp;
	m_tmproc = tmproc;
	m_nreqs = 0;
	memcpy(&m_params, params, sizeof(m_params));
	
	// Alloc space for obstacles.
	m_obstacles = (dtTileCacheObstacle*)dtAlloc(sizeof(dtTileCacheObstacle)*m_params.maxObstacles, DT_ALLOC_PERM_TILE_DATA);
	if (!m_obstacles)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(m_obstacles, 0, sizeof(dtTileCacheObstacle)*m_params.maxObstacles);
	m_nextFreeObstacle = 0;
	for (int i = m_params.maxObstacles-1; i >= 0; --i)
	{
		m_obstacles[i].salt = DT_SALT_BASE;
		m_obstacles[i].next = m_nextFreeObstacle;
		m_nextFreeObstacle = &m_obstacles[i];
	}
	
	// Init tiles
	m_tileLutSize = dtNextPow2(m_params.maxTiles/4);
	if (!m_tileLutSize) m_tileLutSize = 1;
	m_tileLutMask = m_tileLutSize-1;
	
	m_tiles = (dtCompressedTile*)dtAlloc(sizeof(dtCompressedTile)*m_params.maxTiles, DT_ALLOC_PERM_TILE_DATA);
	if (!m_tiles)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	m_posLookup = (dtCompressedTile**)dtAlloc(sizeof(dtCompressedTile*)*m_tileLutSize, DT_ALLOC_PERM_TILE_DATA);
	if (!m_posLookup)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(m_tiles, 0, sizeof(dtCompressedTile)*m_params.maxTiles);
	memset(m_posLookup, 0, sizeof(dtCompressedTile*)*m_tileLutSize);
	m_nextFreeTile = 0;
	for (int i = m_params.maxTiles-1; i >= 0; --i)
	{
		m_tiles[i].salt = DT_SALT_BASE;
		m_tiles[i].next = m_nextFreeTile;
		m_nextFreeTile = &m_tiles[i];
	}
	
	// Init ID generator values.
	m_tileBits = dtIlog2(dtNextPow2((unsigned int)m_params.maxTiles));
	// Only allow 31 salt bits, since the salt mask is calculated using 32bit uint and it will overflow.
	m_saltBits = dtMin((unsigned int)31, 32 - m_tileBits);
	if (m_saltBits < 10)
		return DT_FAILURE | DT_INVALID_PARAM;
	
	return DT_SUCCESS;
}

int dtTileCache::getTilesAt(const int tx, const int ty, dtCompressedTileRef* tiles, const int maxTiles) const 
{
	int n = 0;
	
	// Find tile based on hash.
	int h = TileCacheFunc::computeTileHash(tx, ty, m_tileLutMask);
	dtCompressedTile* tile = m_posLookup[h];
	while (tile)
	{
		if (tile->header &&
			tile->header->tx == tx &&
			tile->header->ty == ty)
		{
			if (n < maxTiles)
				tiles[n++] = getTileRef(tile);
		}
		tile = tile->next;
	}
	
	return n;
}

dtCompressedTile* dtTileCache::getTileAt(const int tx, const int ty, const int tlayer)
{
	// Find tile based on hash.
	int h = TileCacheFunc::computeTileHash(tx, ty, m_tileLutMask);
	dtCompressedTile* tile = m_posLookup[h];
	while (tile)
	{
		if (tile->header &&
			tile->header->tx == tx &&
			tile->header->ty == ty &&
			tile->header->tlayer == tlayer)
		{
			return tile;
		}
		tile = tile->next;
	}
	return 0;
}

dtCompressedTileRef dtTileCache::getTileRef(const dtCompressedTile* tile) const
{
	if (!tile) return 0;
	const unsigned int it = (unsigned int)(tile - m_tiles);
	return (dtCompressedTileRef)encodeTileId(tile->salt, it);
}

dtObstacleRef dtTileCache::getObstacleRef(const dtTileCacheObstacle* ob) const
{
	if (!ob) return 0;
	const unsigned int idx = (unsigned int)(ob - m_obstacles);
	return encodeObstacleId(ob->salt, idx);
}

const dtTileCacheObstacle* dtTileCache::getObstacleByRef(dtObstacleRef ref)
{
	if (!ref)
		return 0;
	unsigned int idx = decodeObstacleIdObstacle(ref);
	if ((int)idx >= m_params.maxObstacles)
		return 0;
	const dtTileCacheObstacle* ob = &m_obstacles[idx];
	unsigned int salt = decodeObstacleIdSalt(ref);
	if (ob->salt != salt)
		return 0;
	return ob;
}

dtStatus dtTileCache::addTile(unsigned char* data, const int dataSize, unsigned char flags, dtCompressedTileRef* result)
{
	// Make sure the data is in right format.
	dtTileCacheLayerHeader* header = (dtTileCacheLayerHeader*)data;
	if (header->version != DT_TILECACHE_VERSION)
		return DT_FAILURE | DT_WRONG_VERSION;
	
	// Make sure the location is free.
	if (getTileAt(header->tx, header->ty, header->tlayer))
		return DT_FAILURE;
	
	// Allocate a tile.
	dtCompressedTile* tile = 0;
	if (m_nextFreeTile)
	{
		tile = m_nextFreeTile;
		m_nextFreeTile = tile->next;
		tile->next = 0;
	}
	
	// Make sure we could allocate a tile.
	if (!tile)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	
	// Insert tile into the position lut.
	int h = TileCacheFunc::computeTileHash(header->tx, header->ty, m_tileLutMask);
	tile->next = m_posLookup[h];
	m_posLookup[h] = tile;
	
	// Init tile.
	const int headerSize = dtAlign(sizeof(dtTileCacheLayerHeader));
	tile->header = (dtTileCacheLayerHeader*)data;
	tile->data = data;
	tile->dataSize = dataSize;
	tile->compressed = tile->data + headerSize;
	tile->compressedSize = tile->dataSize - headerSize;
	tile->flags = flags;
	
	if (result)
		*result = getTileRef(tile);
	
	return DT_SUCCESS;
}

dtStatus dtTileCache::removeTile(dtCompressedTileRef ref, unsigned char** data, int* dataSize)
{
	if (!ref)
		return DT_FAILURE | DT_INVALID_PARAM;
	unsigned int tileIndex = decodeTileIdTile(ref);
	unsigned int tileSalt = decodeTileIdSalt(ref);
	if ((int)tileIndex >= m_params.maxTiles)
		return DT_FAILURE | DT_INVALID_PARAM;
	dtCompressedTile* tile = &m_tiles[tileIndex];
	if (tile->salt != tileSalt)
		return DT_FAILURE | DT_INVALID_PARAM;
	
	// Remove tile from hash lookup.
	const int h = TileCacheFunc::computeTileHash(tile->header->tx, tile->header->ty, m_tileLutMask);
	dtCompressedTile* prev = 0;
	dtCompressedTile* cur = m_posLookup[h];
	while (cur)
	{
		if (cur == tile)
		{
			if (prev)
				prev->next = cur->next;
			else
				m_posLookup[h] = cur->next;
			break;
		}
		prev = cur;
		cur = cur->next;
	}
	
	// Reset tile.
	if (tile->flags & DT_COMPRESSEDTILE_FREE_DATA)
	{
		// Owns data
		dtFree(tile->data, DT_ALLOC_PERM_TILE_DATA);
		tile->data = 0;
		tile->dataSize = 0;
		if (data) *data = 0;
		if (dataSize) *dataSize = 0;
	}
	else
	{
		if (data) *data = tile->data;
		if (dataSize) *dataSize = tile->dataSize;
	}
	
	tile->header = 0;
	tile->data = 0;
	tile->dataSize = 0;
	tile->compressed = 0;
	tile->compressedSize = 0;
	tile->flags = 0;
	
	// Update salt, salt should never be zero.
	tile->salt = (tile->salt+1) & ((1<<m_saltBits)-1);
	if (tile->salt == 0)
		tile->salt++;
	
	// Add to free list.
	tile->next = m_nextFreeTile;
	m_nextFreeTile = tile;
	
	return DT_SUCCESS;
}


dtObstacleRef dtTileCache::addObstacle(const dtReal* pos, const dtReal radius, const dtReal height, dtObstacleRef* result)
{
	if (m_nreqs >= MAX_REQUESTS)
		return DT_FAILURE | DT_BUFFER_TOO_SMALL;
	
	dtTileCacheObstacle* ob = 0;
	if (m_nextFreeObstacle)
	{
		ob = m_nextFreeObstacle;
		m_nextFreeObstacle = ob->next;
		ob->next = 0;
	}
	if (!ob)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	
	unsigned short salt = ob->salt;
	memset(ob, 0, sizeof(dtTileCacheObstacle));
	ob->salt = salt;
	ob->state = DT_OBSTACLE_PROCESSING;
	dtVcopy(ob->pos, pos);
	ob->radius = radius;
	ob->height = height;
	
	ObstacleRequest* req = &m_reqs[m_nreqs++];
	memset(req, 0, sizeof(ObstacleRequest));
	req->action = REQUEST_ADD;
	req->ref = getObstacleRef(ob);
	
	if (result)
		*result = req->ref;
	
	return DT_SUCCESS;
}

dtObstacleRef dtTileCache::removeObstacle(const dtObstacleRef ref)
{
	if (!ref)
		return DT_SUCCESS;
	if (m_nreqs >= MAX_REQUESTS)
		return DT_FAILURE | DT_BUFFER_TOO_SMALL;
	
	ObstacleRequest* req = &m_reqs[m_nreqs++];
	memset(req, 0, sizeof(ObstacleRequest));
	req->action = REQUEST_REMOVE;
	req->ref = ref;
	
	return DT_SUCCESS;
}

dtStatus dtTileCache::queryTiles(const dtReal* bmin, const dtReal* bmax,
								 dtCompressedTileRef* results, int* resultCount, const int maxResults) const 
{
	const int MAX_TILES = 32;
	dtCompressedTileRef tiles[MAX_TILES];
	
	int n = 0;
	
	const dtReal tw = m_params.width * m_params.cs;
	const dtReal th = m_params.height * m_params.cs;
	const int tx0 = (int)dtFloor((bmin[0]-m_params.orig[0]) / tw);
	const int tx1 = (int)dtFloor((bmax[0]-m_params.orig[0]) / tw);
	const int ty0 = (int)dtFloor((bmin[2]-m_params.orig[2]) / th);
	const int ty1 = (int)dtFloor((bmax[2]-m_params.orig[2]) / th);
	
	for (int ty = ty0; ty <= ty1; ++ty)
	{
		for (int tx = tx0; tx <= tx1; ++tx)
		{
			const int ntiles = getTilesAt(tx,ty,tiles,MAX_TILES);
			
			for (int i = 0; i < ntiles; ++i)
			{
				const dtCompressedTile* tile = &m_tiles[decodeTileIdTile(tiles[i])];
				dtReal tbmin[3], tbmax[3];
				calcTightTileBounds(tile->header, tbmin, tbmax);
				
				if (dtOverlapBounds(bmin,bmax, tbmin,tbmax))
				{
					if (n < maxResults)
						results[n++] = tiles[i];
				}
			}
		}
	}
	
	*resultCount = n;
	
	return DT_SUCCESS;
}

dtStatus dtTileCache::update(const dtReal /*dt*/, dtNavMesh* navmesh)
{
	if (m_nupdate == 0)
	{
		// Process requests.
		for (int i = 0; i < m_nreqs; ++i)
		{
			ObstacleRequest* req = &m_reqs[i];
			
			unsigned int idx = decodeObstacleIdObstacle(req->ref);
			if ((int)idx >= m_params.maxObstacles)
				continue;
			dtTileCacheObstacle* ob = &m_obstacles[idx];
			unsigned int salt = decodeObstacleIdSalt(req->ref);
			if (ob->salt != salt)
				continue;
			
			if (req->action == REQUEST_ADD)
			{
				// Find touched tiles.
				dtReal bmin[3], bmax[3];
				getObstacleBounds(ob, bmin, bmax);

				int ntouched = 0;
				queryTiles(bmin, bmax, ob->touched, &ntouched, DT_MAX_TOUCHED_TILES);
				ob->ntouched = (unsigned char)ntouched;
				// Add tiles to update list.
				ob->npending = 0;
				for (int j = 0; j < ob->ntouched; ++j)
				{
					if (m_nupdate < MAX_UPDATE)
					{
						if (!contains(m_update, m_nupdate, ob->touched[j]))
							m_update[m_nupdate++] = ob->touched[j];
						ob->pending[ob->npending++] = ob->touched[j];
					}
				}
			}
			else if (req->action == REQUEST_REMOVE)
			{
				// Prepare to remove obstacle.
				ob->state = DT_OBSTACLE_REMOVING;
				// Add tiles to update list.
				ob->npending = 0;
				for (int j = 0; j < ob->ntouched; ++j)
				{
					if (m_nupdate < MAX_UPDATE)
					{
						if (!contains(m_update, m_nupdate, ob->touched[j]))
							m_update[m_nupdate++] = ob->touched[j];
						ob->pending[ob->npending++] = ob->touched[j];
					}
				}
			}
		}
		
		m_nreqs = 0;
	}
	
	// Process updates
	if (m_nupdate)
	{
		// Build mesh
		const dtCompressedTileRef ref = m_update[0];
		dtStatus status = buildNavMeshTile(ref, navmesh);
		m_nupdate--;
		if (m_nupdate > 0)
			memmove(m_update, m_update+1, m_nupdate*sizeof(dtCompressedTileRef));

		// Update obstacle states.
		for (int i = 0; i < m_params.maxObstacles; ++i)
		{
			dtTileCacheObstacle* ob = &m_obstacles[i];
			if (ob->state == DT_OBSTACLE_PROCESSING || ob->state == DT_OBSTACLE_REMOVING)
			{
				// Remove handled tile from pending list.
				for (int j = 0; j < (int)ob->npending; j++)
				{
					if (ob->pending[j] == ref)
					{
						ob->pending[j] = ob->pending[(int)ob->npending-1];
						ob->npending--;
						break;
					}
				}
				
				// If all pending tiles processed, change state.
				if (ob->npending == 0)
				{
					if (ob->state == DT_OBSTACLE_PROCESSING)
					{
						ob->state = DT_OBSTACLE_PROCESSED;
					}
					else if (ob->state == DT_OBSTACLE_REMOVING)
					{
						ob->state = DT_OBSTACLE_EMPTY;
						// Update salt, salt should never be zero.
						ob->salt = (ob->salt+1) & ((1<<16)-1);
						if (ob->salt == 0)
							ob->salt++;
						// Return obstacle to free list.
						ob->next = m_nextFreeObstacle;
						m_nextFreeObstacle = ob;
					}
				}
			}
		}
			
		if (dtStatusFailed(status))
			return status;
	}
	
	return DT_SUCCESS;
}


dtStatus dtTileCache::buildNavMeshTilesAt(const int tx, const int ty, dtNavMesh* navmesh)
{
	const int MAX_TILES = 32;
	dtCompressedTileRef tiles[MAX_TILES];
	const int ntiles = getTilesAt(tx,ty,tiles,MAX_TILES);
	
	for (int i = 0; i < ntiles; ++i)
	{
		dtStatus status = buildNavMeshTile(tiles[i], navmesh);
		if (dtStatusFailed(status))
			return status;
	}
	
	return DT_SUCCESS;
}

dtStatus dtTileCache::buildNavMeshTile(const dtCompressedTileRef ref, dtNavMesh* navmesh)
{	
	dtAssert(m_talloc);
	dtAssert(m_tcomp);
	
	unsigned int idx = decodeTileIdTile(ref);
	if (idx > (unsigned int)m_params.maxTiles)
		return DT_FAILURE | DT_INVALID_PARAM;
	const dtCompressedTile* tile = &m_tiles[idx];
	unsigned int salt = decodeTileIdSalt(ref);
	if (tile->salt != salt)
		return DT_FAILURE | DT_INVALID_PARAM;
	
	m_talloc->reset();
	
	BuildContext bc(m_talloc);
	const int walkableClimbVx = (int)(m_params.walkableClimb / m_params.ch);
	dtStatus status;
	
	// Decompress tile layer data. 
	status = dtDecompressTileCacheLayer(m_talloc, m_tcomp, tile->data, tile->dataSize, &bc.layer);
	if (dtStatusFailed(status))
		return status;

#if 0
	if (tile->header->tx != 1 || tile->header->ty != 0 || tile->header->tlayer < 1)
		return status;
#endif
	
	// Rasterize obstacles.
	for (int i = 0; i < m_params.maxObstacles; ++i)
	{
		const dtTileCacheObstacle* ob = &m_obstacles[i];
		if (ob->state == DT_OBSTACLE_EMPTY || ob->state == DT_OBSTACLE_REMOVING)
			continue;
		if (contains(ob->touched, ob->ntouched, ref))
		{
			dtMarkCylinderArea(*bc.layer, tile->header->bmin, m_params.cs, m_params.ch,
							   ob->pos, ob->radius, ob->height, 0);
		}
	}
	
	if (m_tmproc)
	{
		m_tmproc->markAreas(bc.layer, tile->header->bmin, m_params.cs, m_params.ch);
	}

	// Build navmesh
	if (m_params.regionPartitioning == DT_REGION_MONOTONE)
	{
		status = dtBuildTileCacheRegionsMonotone(m_talloc, m_params.minRegionArea, m_params.mergeRegionArea, *bc.layer);
	}
	else if (m_params.regionPartitioning == DT_REGION_WATERSHED)
	{
		bc.dfield = dtAllocTileCacheDistanceField(m_talloc);
		if (!bc.dfield)
			return status;

		status = dtBuildTileCacheDistanceField(m_talloc, *bc.layer, *bc.dfield);
		if (dtStatusFailed(status))
			return status;

		status = dtBuildTileCacheRegions(m_talloc, m_params.minRegionArea, m_params.mergeRegionArea, *bc.layer, *bc.dfield);
	}
	else
	{
		status = dtBuildTileCacheRegionsChunky(m_talloc, m_params.minRegionArea, m_params.mergeRegionArea, *bc.layer, m_params.regionChunkSize);
	}
	if (dtStatusFailed(status))
		return status;
	
	bc.lcset = dtAllocTileCacheContourSet(m_talloc);
	if (!bc.lcset)
		return status;

	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	bc.lclusters = dtAllocTileCacheClusterSet(m_talloc);
	if (!bc.lclusters)
		return status;
	status = dtBuildTileCacheContours(m_talloc, *bc.layer, walkableClimbVx,
									  m_params.maxSimplificationError, m_params.simplificationElevationRatio, m_params.cs, m_params.ch,
									  *bc.lcset, *bc.lclusters);
#else
	status = dtBuildTileCacheContours(m_talloc, *bc.layer, walkableClimbVx,
									  m_params.maxSimplificationError, m_params.simplificationElevationRatio, m_params.cs, m_params.ch, *bc.lcset);
#endif //WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	if (dtStatusFailed(status))
		return status;
	
	bc.lmesh = dtAllocTileCachePolyMesh(m_talloc);
	if (!bc.lmesh)
		return status;
	status = dtBuildTileCachePolyMesh(m_talloc, 0, *bc.lcset, *bc.lmesh, m_params.walkableClimb);
	if (dtStatusFailed(status))
		return status;
	
	// Early out if the mesh tile is empty.
	if (!bc.lmesh->npolys)
		return DT_SUCCESS;

	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	status = dtBuildTileCacheClusters(m_talloc, *bc.lclusters, *bc.lmesh);
	if (dtStatusFailed(status))
		return status;
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END
	
	dtNavMeshCreateParams params;
	memset(&params, 0, sizeof(params));
	params.verts = bc.lmesh->verts;
	params.vertCount = bc.lmesh->nverts;
	params.polys = bc.lmesh->polys;
	params.polyAreas = bc.lmesh->areas;
	params.polyFlags = bc.lmesh->flags;
	params.polyCount = bc.lmesh->npolys;
	params.nvp = DT_VERTS_PER_POLYGON;
	params.walkableHeight = m_params.walkableHeight;
	params.walkableRadius = m_params.walkableRadius;
	params.walkableClimb = m_params.walkableClimb;
	params.tileX = tile->header->tx;
	params.tileY = tile->header->ty;
	params.tileLayer = tile->header->tlayer;
	params.cs = m_params.cs;
	params.ch = m_params.ch;
	params.buildBvTree = false;
	params.bMinimizeLinkPoolSize = m_params.bMinimizeLinkPoolSize;
	dtVcopy(params.bmin, tile->header->bmin);
	dtVcopy(params.bmax, tile->header->bmax);
	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	params.polyClusters = bc.lclusters->polyMap;
	params.clusterCount = (unsigned short)bc.lclusters->nclusters;
#endif //WITH_NAVMESH_CLUSTER_LINKS
	//@UE END
	
	if (m_tmproc)
	{
		m_tmproc->process(&params, bc.lmesh->areas, bc.lmesh->flags);
	}
	
	unsigned char* navData = 0;
	int navDataSize = 0;
	if (!dtCreateNavMeshData(&params, &navData, &navDataSize))
		return DT_FAILURE;

	// Remove existing tile.
	navmesh->removeTile(navmesh->getTileRefAt(tile->header->tx,tile->header->ty,tile->header->tlayer),0,0);

	// Add new tile, or leave the location empty.
	if (navData)
	{
		// Let the navmesh own the data.
		status = navmesh->addTile(navData,navDataSize,DT_TILE_FREE_DATA,0,0);
		if (dtStatusFailed(status))
		{
			dtFree(navData, DT_ALLOC_PERM_TILE_DATA);
			return status;
		}
	}
	
	return DT_SUCCESS;
}

void dtTileCache::calcTightTileBounds(const dtTileCacheLayerHeader* header, dtReal* bmin, dtReal* bmax) const
{
	const dtReal cs = m_params.cs;
	bmin[0] = header->bmin[0] + header->minx*cs;
	bmin[1] = header->bmin[1];
	bmin[2] = header->bmin[2] + header->miny*cs;
	bmax[0] = header->bmin[0] + (header->maxx+1)*cs;
	bmax[1] = header->bmax[1];
	bmax[2] = header->bmin[2] + (header->maxy+1)*cs;
}

void dtTileCache::getObstacleBounds(const struct dtTileCacheObstacle* ob, dtReal* bmin, dtReal* bmax) const
{
	bmin[0] = ob->pos[0] - ob->radius;
	bmin[1] = ob->pos[1];
	bmin[2] = ob->pos[2] - ob->radius;
	bmax[0] = ob->pos[0] + ob->radius;
	bmax[1] = ob->pos[1] + ob->height;
	bmax[2] = ob->pos[2] + ob->radius;	
}
