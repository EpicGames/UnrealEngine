// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "DetourTileCache/DetourTileCacheBuilder.h"
#include "Detour/DetourCommon.h"
#include "Detour/DetourAssert.h"
#include "DebugUtils/DetourDebugDraw.h"
#include "Stats/Stats.h"
#include <limits>

static const int MAX_VERTS_PER_POLY = 6;	// TODO: use the DT_VERTS_PER_POLYGON
static const int MAX_REM_EDGES = 48;		// TODO: make this an expression.


dtTileCacheContourSet* dtAllocTileCacheContourSet(dtTileCacheAlloc* alloc)
{
	dtAssert(alloc);

	dtTileCacheContourSet* cset = (dtTileCacheContourSet*)alloc->alloc(sizeof(dtTileCacheContourSet));
	memset(cset, 0, sizeof(dtTileCacheContourSet));
	return cset;
}

void dtFreeTileCacheContourSet(dtTileCacheAlloc* alloc, dtTileCacheContourSet* cset)
{
	QUICK_SCOPE_CYCLE_COUNTER(dtFreeTileCacheContourSet);
	
	dtAssert(alloc);

	if (!cset) return;
	for (int i = 0; i < cset->nconts; ++i)
		alloc->free(cset->conts[i].verts);
	alloc->free(cset->conts);
	alloc->free(cset);
}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
dtTileCacheClusterSet* dtAllocTileCacheClusterSet(dtTileCacheAlloc* alloc)
{
	dtAssert(alloc);

	dtTileCacheClusterSet* clusters = (dtTileCacheClusterSet*)alloc->alloc(sizeof(dtTileCacheClusterSet));
	memset(clusters, 0, sizeof(dtTileCacheClusterSet));
	return clusters;
}

void dtFreeTileCacheClusterSet(dtTileCacheAlloc* alloc, dtTileCacheClusterSet* clusters)
{
	QUICK_SCOPE_CYCLE_COUNTER(dtFreeTileCacheClusterSet);
	
	dtAssert(alloc);

	if (!clusters) return;
	alloc->free(clusters->polyMap);
	alloc->free(clusters->regMap);
	alloc->free(clusters);
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

dtTileCachePolyMesh* dtAllocTileCachePolyMesh(dtTileCacheAlloc* alloc)
{
	dtAssert(alloc);

	dtTileCachePolyMesh* lmesh = (dtTileCachePolyMesh*)alloc->alloc(sizeof(dtTileCachePolyMesh));
	memset(lmesh, 0, sizeof(dtTileCachePolyMesh));
	return lmesh;
}

void dtFreeTileCachePolyMesh(dtTileCacheAlloc* alloc, dtTileCachePolyMesh* lmesh)
{
	QUICK_SCOPE_CYCLE_COUNTER(dtFreeTileCachePolyMesh);
	
	dtAssert(alloc);
	
	if (!lmesh) return;
	alloc->free(lmesh->verts);
	alloc->free(lmesh->polys);
	alloc->free(lmesh->flags);
	alloc->free(lmesh->areas);
	alloc->free(lmesh->regs);
	alloc->free(lmesh);
}

dtTileCachePolyMeshDetail* dtAllocTileCachePolyMeshDetail(dtTileCacheAlloc* alloc)
{
	dtAssert(alloc);

	dtTileCachePolyMeshDetail* dmesh = (dtTileCachePolyMeshDetail*)alloc->alloc(sizeof(dtTileCachePolyMeshDetail));
	memset(dmesh, 0, sizeof(dtTileCachePolyMeshDetail));
	return dmesh;
}

void dtFreeTileCachePolyMeshDetail(dtTileCacheAlloc* alloc, dtTileCachePolyMeshDetail* dmesh)
{
	QUICK_SCOPE_CYCLE_COUNTER(dtFreeTileCachePolyMeshDetail);
	
	dtAssert(alloc);

	if (!dmesh) return;
	alloc->free(dmesh->meshes);
	alloc->free(dmesh->verts);
	alloc->free(dmesh->tris);
	alloc->free(dmesh);
} 

dtTileCacheDistanceField* dtAllocTileCacheDistanceField(dtTileCacheAlloc* alloc)
{
	dtAssert(alloc);

	dtTileCacheDistanceField* dfield = (dtTileCacheDistanceField*)alloc->alloc(sizeof(dtTileCacheDistanceField));
	memset(dfield, 0, sizeof(dtTileCacheDistanceField));
	return dfield;
}

void dtFreeTileCacheDistanceField(dtTileCacheAlloc* alloc, dtTileCacheDistanceField* dfield)
{
	QUICK_SCOPE_CYCLE_COUNTER(dtFreeTileCacheDistanceField);

	dtAssert(alloc);

	if (!dfield) return;
	alloc->free(dfield->data);
	alloc->free(dfield);
}


struct dtTempContour
{
	inline dtTempContour(unsigned short* vbuf, const int nvbuf,
						 unsigned short* pbuf, const int npbuf) :
		verts(vbuf), nverts(0), cverts(nvbuf),
		poly(pbuf), npoly(0), cpoly(npbuf) 
	{
	}
	unsigned short* verts; // v[0], v[1], v[2] coordinates, v[3] region, v[4] area (high bit indicates vertex pinning, removed in countour simplification).
	int nverts;
	int cverts;
	unsigned short* poly; // index in verts for the contour of the poly
	int npoly;
	int cpoly;
};




inline bool overlapRangeExl(const unsigned short amin, const unsigned short amax,
							const unsigned short bmin, const unsigned short bmax,
							const unsigned short ya, const unsigned short eya,
							const unsigned short yb, const unsigned short eyb,
							const int walkableClimb)
{
	const bool longitudinalOverlapExl  = !(amin >= bmax || amax <= bmin);
	const bool elevationOverlap = (dtAbs(ya-eya) <= walkableClimb) || (dtAbs(yb-eyb) <= walkableClimb);
	return longitudinalOverlapExl && elevationOverlap;
}

// Returns true on success, false if there was an error adding a vertex.
static bool appendVertex(dtTempContour& cont, const int x, const int y, const int z, const int neiReg, const unsigned char areaId, const bool allowMerging) // UE
{
	// Try to merge with existing segments.
	if (cont.nverts > 1)
	{
		// pa---------pb---------new(x,y,z)
		const unsigned short* pa = &cont.verts[(cont.nverts-2)*5];
		unsigned short* pb = &cont.verts[(cont.nverts-1)*5];
		const unsigned short pr = pb[3];
		if (allowMerging && pr == neiReg)	// UE
		{
			if (pa[0] == pb[0] && (int)pb[0] == x)
			{
				// The verts are aligned aling x-axis, update z.
				pb[1] = (unsigned short)y;
				pb[2] = (unsigned short)z;
				return true;
			}
			else if (pa[2] == pb[2] && (int)pb[2] == z)
			{
				// The verts are aligned aling z-axis, update x.
				pb[0] = (unsigned short)x;
				pb[1] = (unsigned short)y;
				return true;
			}
		}
	}

	// Add new point.
	if (cont.nverts+1 > cont.cverts)
		return false;

	unsigned short* v = &cont.verts[cont.nverts*5];
	v[0] = (unsigned short)x;
	v[1] = (unsigned short)y;
	v[2] = (unsigned short)z;
	v[3] = (unsigned short)neiReg;
	v[4] = areaId;
	cont.nverts++;

	return true;
}

//@UE BEGIN
static void getNeighbourRegAndAreaAndVertexHeight(dtTileCacheLayer& layer,
	const int ax, const int ay, const int abDir,
	unsigned short& neiReg, unsigned char& neiArea, unsigned char& cornerNeiArea, unsigned short& vertexHeight)
{
	// [a] is the current cell, [b] is the direct neighbour in the direction 'dir'.
	//   ^
	//  [b][c]
	//  [a][d]
	
	const int w = (int)layer.header->width;
	const int ia = ax + ay*w;
	const unsigned char acon = layer.cons[ia] & 0xf;
	const unsigned char portal = layer.cons[ia] >> 4;
	const unsigned char abDirMask = (unsigned char)(1 << abDir);

	const int bcDir = (abDir + 1) & 0x3;
	const unsigned char bcDirMask = (unsigned char)(1 << bcDir);
	
	cornerNeiArea = 0;

	if ((acon & abDirMask) == 0)
	{
		// No connection, return portal or hard edge.
		if (portal & abDirMask)
		{
			neiReg = 0xf800 + (unsigned char)abDir;
			neiArea = 0;
		}
		else
		{
			neiReg = 0xffff;
			neiArea = 0;
		}

		// Find the vertex height. Try going A-D-C, get height of d and c.
		vertexHeight = layer.heights[ia];
		if ((acon & bcDirMask) != 0)
		{
			// a is connected to d
			const int dx = ax + getDirOffsetX(bcDir);
			const int dy = ay + getDirOffsetY(bcDir);
			const int id = dx + dy * w;
			vertexHeight = dtMax(vertexHeight, layer.heights[id]);

			const unsigned char dcon = layer.cons[id] & 0xf;
			if ((dcon & abDirMask) != 0)
			{
				// d is connected to c
				const int cx = dx + getDirOffsetX(abDir);
				const int cy = dy + getDirOffsetY(abDir);
				const int ic = cx + cy * w;
				vertexHeight = dtMax(vertexHeight, layer.heights[ic]);
			}
		}
	}
	else
	{
		// a is connected to b
		const int bx = ax + getDirOffsetX(abDir);
		const int by = ay + getDirOffsetY(abDir);
		const int ib = bx + by*w;

		neiReg = layer.regs[ib];
		neiArea = layer.areas[ib];
		vertexHeight = dtMax(layer.heights[ia], layer.heights[ib]);

		// Get area type of the cell diagonal [c] to current cell [a]. Where [b] is direct neighbour in the direction of 'dir'.
		//   ^
		//  [b][c]
		//  [a][d]
		const unsigned char bcon = layer.cons[ib] & 0xf;
		if ((bcon & bcDirMask) == 0)
		{
			cornerNeiArea = 0;
		}
		else
		{
			// b is connected to c
			const int cx = bx + getDirOffsetX(bcDir);
			const int cy = by + getDirOffsetY(bcDir);
			const int ic = cx + cy * w;
			cornerNeiArea = layer.areas[ic];
			vertexHeight = dtMax(vertexHeight, layer.heights[ic]);
		}

		// If connected, check the height of d to get the maximum for the vertexHeight.
		if ((acon & bcDirMask) != 0)
		{
			// a is connected to d
			const int dx = ax + getDirOffsetX(bcDir);
			const int dy = ay + getDirOffsetY(bcDir);
			const int id = dx + dy * w;
			vertexHeight = dtMax(vertexHeight, layer.heights[id]);
		}
	}
}
//@UE END

static bool walkContour(dtTileCacheLayer& layer, int x, int y, int idx, const bool allowMerging, unsigned char* flags, dtTempContour& cont, int& contourIndex) // UE
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	unsigned char dir = 0;
	while ((flags[idx] & (1 << dir)) == 0)
		dir++;

	int startDir = dir;
	int startIdx = idx;
	cont.nverts = 0;

	unsigned short neiReg = 0xffff;
	unsigned char neiArea = 0;
	unsigned char cornerNeiArea = 0;
	unsigned short vertexHeight = 0;	// UE
	unsigned short prevNeiArea = 0;
	unsigned short prevCornerNeiArea = 0;
	bool checkForPinning = false;

	const int maxIter = w * h * 2;
	int iter = 0;
	while (iter < maxIter)
	{
		int nx = x;
		int ny = y;
		unsigned char ndir = dir;

		getNeighbourRegAndAreaAndVertexHeight(layer, x, y, dir, neiReg, neiArea, cornerNeiArea, vertexHeight);	// UE

		// Check if the region in the provided direction is different than the region at x,y.
		if (neiReg != layer.regs[x+y*w])
		{
			// Solid edge.
			if (checkForPinning)
			{
				// Detect if there was 8-connected area type change during the turn.
				// If it did, we need to make sure the vertex added during turn does not get simplified.
				// AB
				// xC
				// x = current location, A = prevNeiArea, B = prevCornerNeiArea, C = neiArea.
				const bool shouldPinVertex = prevNeiArea == neiArea && prevCornerNeiArea != neiArea;
				if (shouldPinVertex && cont.nverts > 0)
				{
					unsigned short* v = &cont.verts[(cont.nverts - 1) * 5];
					v[4] |= 0x8000; // Use high bits in the area type to flag pinning of the vertex, will be removed in the contour simplification.
				}
			}

			// Get corner vertex
			int px = x;
			int pz = y;
			switch(dir)
			{
			case 0: pz++; break;
			case 1: px++; pz++; break;
			case 2: px++; break;
			}

			// Try to merge with previous vertex.
			if (!appendVertex(cont, px, vertexHeight, pz, neiReg, neiArea, allowMerging)) // UE
				return false;

			flags[idx] &= ~(1 << dir); // Remove visited edges
			ndir = (dir+1) & 0x3;  // Rotate CW
			checkForPinning = true;
		}
		else
		{
			// Move to next.
			nx = x + getDirOffsetX(dir);
			ny = y + getDirOffsetY(dir);
			ndir = (dir+3) & 0x3;	// Rotate CCW
			checkForPinning = false;
		}

		if (iter > 0 && idx == startIdx && dir == startDir)
			break;

		prevNeiArea = neiArea;
		prevCornerNeiArea = cornerNeiArea;

		x = nx;
		y = ny;
		dir = ndir;
		idx = x+y*w;

		iter++;
	}

	// Remove last vertex if it is duplicate of the first one.
	unsigned short* pa = &cont.verts[(cont.nverts-1)*5];
	unsigned short* pb = &cont.verts[0];
	if (pa[0] == pb[0] && pa[2] == pb[2])
		cont.nverts--;

//@UE BEGIN
	// Check if first vertex should be merged.
	if (cont.nverts > 1)
	{
		unsigned short* last = &cont.verts[(cont.nverts-1)*5];
		unsigned short* first = &cont.verts[0*5];
		unsigned short* next = &cont.verts[1*5];

		// Check if we can remove first vertex. First vertex will become last vertex.
		if (allowMerging && first[3] == next[3])
		{
			if (last[0] == first[0] && first[0] == next[0])
			{
				// The verts are aligned aling x-axis, update z.
				first[1] = last[1];
				first[2] = last[2];
				first[3] = last[3];
				first[4] = last[4]; 
				cont.nverts--;	// remove last
			}
			else if (last[2] == first[2] && first[2] == next[2])
			{
				// The verts are aligned aling z-axis, update x.
				first[0] = last[0];
				first[1] = last[1];
				first[3] = last[3];
				first[4] = last[4];
				cont.nverts--; // remove last
			}
		}
	}	
//@UE END
	
	contourIndex++;

	return true;
}	

namespace TileCacheFunc
{
	static dtReal distancePtSegSqr2D(const int x, const int z, const int px, const int pz, const int qx, const int qz) // UE
	{
		dtReal pqx = (dtReal)(qx - px);
		dtReal pqz = (dtReal)(qz - pz);
		dtReal dx = (dtReal)(x - px);
		dtReal dz = (dtReal)(z - pz);
		dtReal d = pqx*pqx + pqz*pqz;
		dtReal t = pqx*dx + pqz*dz;
		if (d > 0)
			t /= d;
		if (t < 0)
			t = 0;
		else if (t > 1)
			t = 1;

		dx = px + t*pqx - x;
		dz = pz + t*pqz - z;

		return dx*dx + dz*dz;
	}
}

static void simplifyContour(unsigned char area, unsigned short region, dtTempContour& cont, const dtReal maxError, const dtReal elevationRatio, const dtReal cs, const dtReal ch) // UE
{
	cont.npoly = 0;

	if (cont.nverts < 2)
	{
		// corrupted, remove it
		cont.nverts = 0;
		return;
	}

	for (int i = 0; i < cont.nverts; ++i)
	{
		int j = (i+1) % cont.nverts;
		// Check for start of a wall segment or pinned vertices.
		const unsigned short ra = cont.verts[j*5+3];
		const unsigned short rb = cont.verts[i*5+3];
		const bool pinnedVertex = (cont.verts[i*5+4] & 0x8000) != 0;
		if (ra != rb || pinnedVertex)
			cont.poly[cont.npoly++] = (unsigned short)i;
	}

	if (cont.npoly < 2)
	{
		// If there is no transitions at all,
		// create some initial points for the simplification process. 
		// Find lower-left and upper-right vertices of the contour.
		int llx = cont.verts[0];
		int llz = cont.verts[2];
		int lli = 0;
		int urx = cont.verts[0];
		int urz = cont.verts[2];
		int uri = 0;
		for (int i = 1; i < cont.nverts; ++i)
		{
			int x = cont.verts[i*5+0];
			int z = cont.verts[i*5+2];
			if (x < llx || (x == llx && z < llz))
			{
				llx = x;
				llz = z;
				lli = i;
			}
			if (x > urx || (x == urx && z > urz))
			{
				urx = x;
				urz = z;
				uri = i;
			}
		}
		cont.npoly = 0;
		cont.poly[cont.npoly++] = (unsigned short)lli;
		cont.poly[cont.npoly++] = (unsigned short)uri;
	}

	const dtReal heightRatio = elevationRatio * ch / cs; // UE
	const bool checkElevation = elevationRatio > 0;		 // UE

	// Add points until all raw points are within
	// error tolerance to the simplified shape.
	for (int i = 0; i < cont.npoly; )
	{
		int ii = (i+1) % cont.npoly;

		const int ai = (int)cont.poly[i];
		int ax = (int)cont.verts[ai*5+0];
		int ay = (int)cont.verts[ai*5+1]; // UE
		int az = (int)cont.verts[ai*5+2];

		const int bi = (int)cont.poly[ii];
		int bx = (int)cont.verts[bi*5+0];
		int by = (int)cont.verts[bi*5+1]; // UE
		int bz = (int)cont.verts[bi*5+2];

		// Find maximum deviation from the segment.
		dtReal maxd = 0;
		int maxi = -1;
		int ci, cinc, endi;

//@UE BEGIN
		dtReal a[3];
		dtReal b[3];
		if (checkElevation)
		{
			a[0] = (dtReal)ax;
			a[1] = heightRatio * ay;
			a[2] = (dtReal)az;
			b[0] = (dtReal)bx;
			b[1] = heightRatio * by;
			b[2] = (dtReal)bz;
		}
//@UE END
		
		// Traverse the segment in lexilogical order so that the
		// max deviation is calculated similarly when traversing
		// opposite segments.
		if (bx > ax || (bx == ax && bz > az))
		{
			cinc = 1;
			ci = (ai+cinc) % cont.nverts;
			endi = bi;
		}
		else
		{
			cinc = cont.nverts-1;
			ci = (bi+cinc) % cont.nverts;
			endi = ai;

//@UE BEGIN
			// Because of floating point imprecision, dtDistancePtSegSqr to a-b might be slightly differ from dtDistancePtSegSqr to b-a.
			// Swap points because we need the maximum deviation to be computed the same way for opposite segments to match.
			if (checkElevation)
			{
				dtSwap(a[0], b[0]);
				dtSwap(a[1], b[1]);
				dtSwap(a[2], b[2]);
			}
			else
			{
				dtSwap(ax, bx);
				dtSwap(az, bz);
			}
//@UE END
		}

		// Tessellate only between regions and areas.
		const unsigned short* ciSrc = &cont.verts[ci*5];
		const int ciReg = ciSrc[3];
		const unsigned char ciArea = (unsigned char)ciSrc[4];
		if (area != ciArea || ciReg == 0xffff || (checkElevation && region != ciReg)) // UE
		{
			while (ci != endi)
			{
//@UE BEGIN
				dtReal d;
				if (checkElevation)
				{
					// Instead of multiplying all components by ch or cs to go from voxels to world units, 
					// we just use the heightRatio (avoiding extra cs multiplication on x and z).
					const dtReal pt[3] = { (dtReal)cont.verts[ci*5+0], heightRatio*cont.verts[ci*5+1], (dtReal)cont.verts[ci*5+2] };
					d = dtDistancePtSegSqr(pt, a, b);
				}
				else
				{
					d = TileCacheFunc::distancePtSegSqr2D(cont.verts[ci*5+0], cont.verts[ci*5+2], ax, az, bx, bz);
				}
//@UE END
				if (d > maxd)
				{
					maxd = d;
					maxi = ci;
				}
				ci = (ci+cinc) % cont.nverts;
			}
		}

		// If the max deviation is larger than accepted error,
		// add new point, else continue to next segment.
		if (maxi != -1 && maxd > (maxError*maxError))
		{
			cont.npoly++;
			for (int j = cont.npoly-1; j > i; --j)
				cont.poly[j] = cont.poly[j-1];
			cont.poly[i+1] = (unsigned short)maxi;
		}
		else
		{
			++i;
		}
	}

	// Remap vertices
	int start = 0;
	for (int i = 1; i < cont.npoly; ++i)
		if (cont.poly[i] < cont.poly[start])
			start = i;

	cont.nverts = 0;
	for (int i = 0; i < cont.npoly; ++i)
	{
		const int j = (start+i) % cont.npoly;
		unsigned short* src = &cont.verts[cont.poly[j]*5];
		unsigned short* dst = &cont.verts[cont.nverts*5];

		// check for degenerated segments (RecastContour.cpp : removeDegenerateSegments)
		const int nj = (start+i+1) % cont.npoly;
		unsigned short* nextSeg = &cont.verts[cont.poly[nj]*5];
		if (src[0] == nextSeg[0] && src[2] == nextSeg[2])
		{
			// skip degenerated ones
			continue;
		}

		dst[0] = src[0];
		dst[1] = src[1];
		dst[2] = src[2];
		dst[3] = src[3];
		dst[4] = src[4] & 0xff; // Mask out pinned vertex flag.
		cont.nverts++;
	}
}

static unsigned short getCornerHeight(dtTileCacheLayer& layer, const int x, const int y, const int z, const int walkableClimb, bool& shouldRemove)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	int n = 0;

	unsigned char portal = 0xf;
	unsigned short height = 0;
	unsigned short preg = 0xffff;	// portal region
	bool allSameReg = true;

	for (int dz = -1; dz <= 0; ++dz)
	{
		for (int dx = -1; dx <= 0; ++dx)
		{
			const int px = x+dx;
			const int pz = z+dz;
			if (px >= 0 && pz >= 0 && px < w && pz < h)
			{
				const int idx  = px + pz*w;
				const int lh = (int)layer.heights[idx];
				if (dtAbs(lh-y) <= walkableClimb && layer.areas[idx] != DT_TILECACHE_NULL_AREA)
				{
					height = dtMax(height, (unsigned short)lh);
					portal &= (layer.cons[idx] >> 4);
					if (preg != 0xffff && preg != layer.regs[idx])
						allSameReg = false;
					preg = layer.regs[idx]; 
					n++;
				}
			}
		}
	}

	int portalCount = 0;
	for (int dir = 0; dir < 4; ++dir)
	{
		if (portal & (1<<dir))
			portalCount++;
	}

	shouldRemove = false;
	if (n > 1 && portalCount == 1 && allSameReg)
	{
		shouldRemove = true;
	}

	return height;
}

static int calcAreaOfPolygon2D(const unsigned short* verts, const int nverts)
{
	int area = 0;
	for (int i = 0, j = nverts-1; i < nverts; j=i++)
	{
		const unsigned short* vi = &verts[i*4];
		const unsigned short* vj = &verts[j*4];
		area += vi[0] * vj[2] - vj[0] * vi[2];
	}
	return (area+1) / 2;
}

inline bool ileft(const unsigned short* a, const unsigned short* b, const unsigned short* c)
{
	return (b[0] - a[0]) * (c[2] - a[2]) - (c[0] - a[0]) * (b[2] - a[2]) <= 0;
}

static void getClosestIndices(const unsigned short* vertsa, const int nvertsa,
	const unsigned short* vertsb, const int nvertsb,
	int& ia, int& ib, int& closestDist)
{
	closestDist = 0xfffffff;
	ia = -1, ib = -1;
	for (int i = 0; i < nvertsa; ++i)
	{
		const int in = (i+1) % nvertsa;
		const int ip = (i+nvertsa-1) % nvertsa;
		const unsigned short* va = &vertsa[i*4];
		const unsigned short* van = &vertsa[in*4];
		const unsigned short* vap = &vertsa[ip*4];

		for (int j = 0; j < nvertsb; ++j)
		{
			const unsigned short* vb = &vertsb[j*4];
			// vb must be "infront" of va.
			if (ileft(vap,va,vb) && ileft(va,van,vb))
			{
				const int dx = vb[0] - va[0];
				const int dz = vb[2] - va[2];
				const int d = dx*dx + dz*dz;
				if (d < closestDist)
				{
					ia = i;
					ib = j;
					closestDist = d;
				}
			}
		}
	}
}

static bool mergeContours(dtTileCacheAlloc* alloc, dtTileCacheContour& ca, dtTileCacheContour& cb, int ia, int ib)
{
	check(ca.nverts > 0 && cb.nverts > 0);
	const int maxVerts = ca.nverts + cb.nverts + 2;
	unsigned short* verts = (unsigned short*)alloc->alloc(sizeof(unsigned short)*maxVerts*4);
	if (!verts)
		return false;

	int nv = 0;

	// Copy contour A.
	for (int i = 0; i <= ca.nverts; ++i)
	{
		unsigned short* dst = &verts[nv*4];
		const unsigned short* src = &ca.verts[((ia+i)%ca.nverts)*4];
		dst[0] = src[0];
		dst[1] = src[1];
		dst[2] = src[2];
		dst[3] = src[3];
		nv++;
	}

	// Copy contour B
	for (int i = 0; i <= cb.nverts; ++i)
	{
		unsigned short* dst = &verts[nv*4];
		const unsigned short* src = &cb.verts[((ib+i)%cb.nverts)*4];
		dst[0] = src[0];
		dst[1] = src[1];
		dst[2] = src[2];
		dst[3] = src[3];
		nv++;
	}

	alloc->free(ca.verts);
	ca.verts = verts;
	ca.nverts = nv;

	alloc->free(cb.verts);
	cb.verts = 0;
	cb.nverts = 0;

	return true;
}

static void getContourCenter(const dtTileCacheContour* cont, const dtReal* orig, dtReal cs, dtReal ch, dtReal* center)
{
	center[0] = 0;
	center[1] = 0;
	center[2] = 0;
	if (!cont->nverts)
		return;
	for (int i = 0; i < cont->nverts; ++i)
	{
		const unsigned short* v = &cont->verts[i*4];
		center[0] += (dtReal)v[0];
		center[1] += (dtReal)v[1];
		center[2] += (dtReal)v[2];
	}
	const dtReal s = dtReal(1.) / cont->nverts;
	center[0] *= s * cs;
	center[1] *= s * ch;
	center[2] *= s * cs;
	center[0] += orig[0];
	center[1] += orig[1] + 4 * ch;
	center[2] += orig[2];
}

static void addUniqueRegion(unsigned short* arr, unsigned short v, int& n)
{
	for (int i = 0; i < n; i++)
	{
		if (arr[i] == v)
			return;
	}

	arr[n] = v;
	n++;
}

// TODO: move this somewhere else, once the layer meshing is done.
dtStatus dtBuildTileCacheContours(dtTileCacheAlloc* alloc, dtTileCacheLayer& layer,
	const int walkableClimb, const dtReal maxError, const dtReal simplificationElevationRatio, // UE
	const dtReal cs, const dtReal ch,
	dtTileCacheContourSet& lcset
	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	, dtTileCacheClusterSet& clusters
#endif //WITH_NAVMESH_CLUSTER_LINKS
	, const bool skipContourSimplification /*=false*/
	//@UE END
	)
{
	dtAssert(alloc);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	int maxConts = layer.regCount;
	lcset.nconts = 0;
	lcset.conts = (dtTileCacheContour*)alloc->alloc(sizeof(dtTileCacheContour)*maxConts);
	if (!lcset.conts)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(lcset.conts, 0, sizeof(dtTileCacheContour)*maxConts);

	// Allocate temp buffer for contour tracing.
	const int maxTempVerts = w*h;

	dtFixedArray<unsigned short> tempVerts(alloc, maxTempVerts*6);
	if (!tempVerts)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	dtFixedArray<unsigned short> tempPoly(alloc, maxTempVerts);
	if (!tempPoly)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	dtFixedArray<unsigned char> flags(alloc, w*h);
	if (!flags)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	// Mark area boundaries
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int idx = x+y*w;
			const unsigned short ri = layer.regs[idx];
			if (ri == 0xffff)
			{
				flags[idx] = 0;
				continue;
			}

			unsigned char res = 0;
			for (int dir = 0; dir < 4; ++dir)
			{
				const unsigned char con = layer.cons[idx] & 0xf;
				const unsigned char mask = (unsigned char)(1<<dir);
				unsigned short r = 0xffff;

				if (con & mask)
				{
					const int ax = x + getDirOffsetX(dir);
					const int ay = y + getDirOffsetY(dir);
					if (ax >= 0 && ay >= 0 && ax < w && ay < h)
					{
						const int aidx = ax + ay*w;
						r = layer.regs[aidx];
					}
				}

				if (r == ri)
					res |= (1 << dir);
			}

			flags[idx] = res ^ 0xf; // Inverse, mark non connected edges.
		}
	}

	dtTempContour temp(tempVerts, maxTempVerts, tempPoly, maxTempVerts);
	dtIntArray links;
	dtIntArray nlinks(maxConts);
	dtIntArray linksBase(maxConts);

	// Only allow merging when not using height in the contour simplification process.
	const bool allowMerging = (simplificationElevationRatio == 0);
	
	// Find contours.
	int contourIndex = 0;	// UE
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int idx = x+y*w;
			if (flags[idx] == 0)
			{
				// All cell edges are connected, ignore it.
				continue;
			}

			const unsigned short ri = layer.regs[idx];
			if (ri == 0xffff || ri == 0)
				continue;

			if (!walkContour(layer, x, y, idx, allowMerging, flags, temp, contourIndex)) // UE
			{
				// Too complex contour.
				// Note: If you hit here often, try increasing 'maxTempVerts'.
				return DT_FAILURE | DT_BUFFER_TOO_SMALL;
			}

			if (!skipContourSimplification)
			{
				simplifyContour(layer.areas[idx], ri, temp, maxError, simplificationElevationRatio, cs, ch); // UE
			}

			// Store contour.
			if (lcset.nconts >= maxConts)
			{
				// Allocate more contours.
				maxConts *= 2;

				dtTileCacheContour* newConts = (dtTileCacheContour*)alloc->alloc(sizeof(dtTileCacheContour)*maxConts);
				if (!newConts)
					return DT_FAILURE | DT_OUT_OF_MEMORY;
				
				memset(newConts, 0, sizeof(dtTileCacheContour)*maxConts);
				for (int j = 0; j < lcset.nconts; ++j)
				{
					newConts[j] = lcset.conts[j];
					// Reset source pointers to prevent data deletion.
					lcset.conts[j].verts = 0;
				}
				alloc->free(lcset.conts);
				lcset.conts = newConts;

				linksBase.resize(maxConts);
				nlinks.resize(maxConts);
			}

			const int contIdx = lcset.nconts;
			lcset.nconts++;

			dtTileCacheContour& cont = lcset.conts[contIdx];
			cont.reg = ri;
			cont.area = layer.areas[idx];
			linksBase[contIdx] = links.size();

			int nnei = 0;
			cont.nverts = temp.nverts;
			if (cont.nverts > 0)
			{
				cont.verts = (unsigned short*)alloc->alloc(sizeof(unsigned short)*4*temp.nverts);
				if (!cont.verts)
					return DT_FAILURE | DT_OUT_OF_MEMORY;

				for (int i = 0, j = temp.nverts-1; i < temp.nverts; j=i++)
				{
					unsigned short* dst = &cont.verts[j*4];
					unsigned short* v = &temp.verts[j*5];
					unsigned short* vn = &temp.verts[i*5];
					unsigned short nei = vn[3]; // The neighbour reg is stored at segment vertex of a segment. 
					bool shouldRemove = false;
					unsigned short lh = getCornerHeight(layer, (int)v[0], (int)v[1], (int)v[2],
						walkableClimb, shouldRemove);

					if ((nei != 0xffff) && ((nei & 0xf800) == 0))
					{
						links.push(nei);
						nnei++;
					}

					dst[0] = v[0];
					dst[1] = lh;
					dst[2] = v[2];

					// Store portal direction and store remove status to the fourth component.
					dst[3] = 0x0f;	// Set direction to 0xf
					if (nei != 0xffff && nei >= 0xf800)
						dst[3] = (unsigned char)(nei - 0xf800);
					if (shouldRemove)
						dst[3] |= 0x80;
				}
			}

			nlinks[contIdx] = nnei;
		}
	}

	// Check and merge droppings.
	// Sometimes the previous algorithms can fail and create several contours
	// per area. This pass will try to merge the holes into the main region.
	for (int i = 0; i < lcset.nconts; ++i)
	{
		dtTileCacheContour& cont = lcset.conts[i];
		// Check if the contour is would backwards.
		if (calcAreaOfPolygon2D(cont.verts, cont.nverts) < 0)
		{
			// Find another contour which has the same region ID.
			int mergeIdx = -1;
			int mergePA = 0, mergePB = 0;
			int bestDist = 0xfffffff;

			for (int j = 0; j < lcset.nconts; ++j)
			{
				dtTileCacheContour& mcont = lcset.conts[j];
				if (i == j) continue;

				if (mcont.nverts && mcont.reg == cont.reg)
				{
					int ia = 0, ib = 0;
					int testDist = 0xfffffff;
					getClosestIndices(mcont.verts, mcont.nverts, cont.verts, cont.nverts, ia, ib, testDist);

					// there could be more than one (isolated islands), merge with closest contour
					if (ia != -1 && ib != -1)
					{
						if (mergeIdx < 0 || testDist < bestDist)
						{
							mergeIdx = j;
							mergePA = ia;
							mergePB = ib;
							bestDist = testDist;
						}
					}
				}
			}

			if (mergeIdx != -1)
			{
				dtTileCacheContour& mcont = lcset.conts[mergeIdx];
				mergeContours(alloc, mcont, cont, mergePA, mergePB);
			}
		}
	}

	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	// Build clusters
	clusters.nregs = layer.regCount ? (layer.regCount + 1) : 0;
	clusters.npolys = 0;
	clusters.nclusters = 0;
	clusters.regMap = (unsigned short*)alloc->alloc(sizeof(unsigned short)*clusters.nregs);
	if (!clusters.regMap)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	memset(clusters.regMap, 0xff, sizeof(unsigned short)*clusters.nregs);
	if (clusters.nregs <= 0)
	{
		return DT_SUCCESS;
	}

	// outer loop: find first unassigned region
	// loop: find all contours matching this region
	// - create new cluster (once)
	// - gather all neighbor regions
	// - repeat inner loop for every region from list
	
	dtScopedDelete<unsigned short> neiRegs(layer.regCount + 1);
	dtScopedDelete<unsigned short> newNeiRegs(layer.regCount + 1);
	int nneiRegs = 0;
	int nnewNeiRegs = 0;
	
	for (int i = 0; i < clusters.nregs; i++)
	{
		if (clusters.regMap[i] != 0xffff)
		{
			continue;
		}

		bool bCanAddCluster = true;
		int newClusterId = clusters.nclusters;
		nneiRegs = 0;

		for (int ic = 0; ic < lcset.nconts; ic++)
		{
			// there could be more than one contour per region...
			dtTileCacheContour& cont = lcset.conts[ic];
			if (cont.reg != (unsigned short)(i) || cont.area == DT_TILECACHE_NULL_AREA)
			{
				continue;
			}

			if (bCanAddCluster)
			{
				dtAssert(newClusterId < std::numeric_limits<unsigned short>::max());
				clusters.regMap[i] = (unsigned short)newClusterId;
				clusters.nclusters++;
				bCanAddCluster = false;
			}

			for (int j = 0; j < nlinks[ic]; j++)
			{
				unsigned short neiReg = (unsigned short)(links[linksBase[ic] + j]);
				addUniqueRegion(neiRegs, neiReg, nneiRegs);
			}
		}

		while (nneiRegs > 0)
		{
			nnewNeiRegs = 0;
			for (int ir = 0; ir < nneiRegs; ir++)
			{
				if ((neiRegs[ir] >= clusters.nregs) || (clusters.regMap[neiRegs[ir]] != 0xffff))
				{
					continue;
				}

				for (int ic = 0; ic < lcset.nconts; ic++)
				{
					// there could be more than one contour per region...
					dtTileCacheContour& cont = lcset.conts[ic];
					if (cont.reg != (unsigned short)neiRegs[ir] || cont.area == DT_TILECACHE_NULL_AREA)
					{
						continue;
					}

					dtAssert(newClusterId < std::numeric_limits<unsigned short>::max());
					clusters.regMap[cont.reg] = (unsigned short)newClusterId;
					for (int j = 0; j < nlinks[ic]; j++)
					{
						unsigned short neiReg = (unsigned short)(links[linksBase[ic] + j]);
						addUniqueRegion(newNeiRegs, neiReg, nnewNeiRegs);
					}
				}
			}

			nneiRegs = nnewNeiRegs;
			memcpy(neiRegs, newNeiRegs, sizeof(unsigned short)* nnewNeiRegs);
		}
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	return DT_SUCCESS;
}	

static const int VERTEX_BUCKET_COUNT2 = (1<<8);

inline int computeVertexHash2(int x, int y, int z)
{
	const unsigned int h1 = 0x8da6b343; // Large multiplicative constants;
	const unsigned int h2 = 0xd8163841; // here arbitrarily chosen primes
	const unsigned int h3 = 0xcb1ab31f;
	unsigned int n = h1 * x + h2 * y + h3 * z;
	return (int)(n & (VERTEX_BUCKET_COUNT2-1));
}

static unsigned short addVertex(unsigned short x, unsigned short y, unsigned short z,
								unsigned short* verts, unsigned short* firstVert, unsigned short* nextVert, int& nv)
{
	int bucket = computeVertexHash2(x, 0, z);
	unsigned short i = firstVert[bucket];
	
	while (i != DT_TILECACHE_NULL_IDX)
	{
		const unsigned short* v = &verts[i*3];
		if (v[0] == x && v[2] == z && (dtAbs(v[1] - y) <= 2))
			return i;
		i = nextVert[i]; // next
	}
	
	// Could not find, create new.
	i = (unsigned short)nv; nv++;
	unsigned short* v = &verts[i*3];
	v[0] = x;
	v[1] = y;
	v[2] = z;
	nextVert[i] = firstVert[bucket];
	firstVert[bucket] = i;
	
	return (unsigned short)i;
}

namespace TileCacheData
{
	struct dtEdge
	{
		unsigned short vert[2];			// index in verts (a,b)
		unsigned short polyEdge[2];		// index in polys (a,b)
		unsigned short poly[2];
	};
}

static bool buildMeshAdjacency(dtTileCacheAlloc* alloc,
							   unsigned short* polys, const int npolys,
							   const unsigned short* verts, const int nverts,
							   const dtTileCacheContourSet& lcset,
							   const int walkableClimb)
{
	// Based on code by Eric Lengyel from:
	// http://www.terathon.com/code/edges.php
	
	const int maxEdgeCount = npolys*MAX_VERTS_PER_POLY;
	dtFixedArray<unsigned short> firstEdge(alloc, nverts + maxEdgeCount);
	if (!firstEdge)
		return false;
	unsigned short* nextEdge = firstEdge + nverts;
	int edgeCount = 0;
	
	dtFixedArray<TileCacheData::dtEdge> edges(alloc, maxEdgeCount);
	if (!edges)
		return false;
	
	for (int i = 0; i < nverts; i++)
		firstEdge[i] = DT_TILECACHE_NULL_IDX;

	// Add edges
	for (int i = 0; i < npolys; ++i)
	{
		const unsigned short* t = &polys[i*MAX_VERTS_PER_POLY*2];
		for (int j = 0; j < MAX_VERTS_PER_POLY; ++j)
		{
			if (t[j] == DT_TILECACHE_NULL_IDX)
				break;

			const unsigned short v0 = t[j];
			const unsigned short v1 = (j+1 >= MAX_VERTS_PER_POLY || t[j+1] == DT_TILECACHE_NULL_IDX) ? t[0] : t[j+1];
			if (v0 < v1)
			{
				TileCacheData::dtEdge& edge = edges[edgeCount];
				edge.vert[0] = v0;
				edge.vert[1] = v1;
				edge.poly[0] = (unsigned short)i;
				edge.polyEdge[0] = (unsigned short)j;
				edge.poly[1] = (unsigned short)i;
				edge.polyEdge[1] = 0xff;
				// Insert edge
				nextEdge[edgeCount] = firstEdge[v0];
				firstEdge[v0] = (unsigned short)edgeCount;
				edgeCount++;
			}
		}
	}

	// Find matching edges
	for (int i = 0; i < npolys; ++i)
	{
		const unsigned short* t = &polys[i*MAX_VERTS_PER_POLY*2];
		for (int j = 0; j < MAX_VERTS_PER_POLY; ++j)
		{
			if (t[j] == DT_TILECACHE_NULL_IDX)
				break;

			unsigned short v0 = t[j];
			unsigned short v1 = (j+1 >= MAX_VERTS_PER_POLY || t[j+1] == DT_TILECACHE_NULL_IDX) ? t[0] : t[j+1];
			if (v0 > v1)
			{
				bool found = false;
				for (unsigned short e = firstEdge[v1]; e != DT_TILECACHE_NULL_IDX; e = nextEdge[e])
				{
					TileCacheData::dtEdge& edge = edges[e];
					if (edge.vert[1] == v0 && edge.poly[0] == edge.poly[1])
					{
						// Edges matches
						edge.poly[1] = (unsigned short)i;
						edge.polyEdge[1] = (unsigned short)j;
						found = true;
						break;
					}
				}
				
				if (!found)
				{
					// Matching edge not found, it is an open edge, add it.
					TileCacheData::dtEdge& edge = edges[edgeCount];
					edge.vert[0] = v1;
					edge.vert[1] = v0;
					edge.poly[0] = (unsigned short)i;
					edge.polyEdge[0] = (unsigned short)j;
					edge.poly[1] = (unsigned short)i;
					edge.polyEdge[1] = 0xff;

					// Insert edge
					nextEdge[edgeCount] = firstEdge[v1];
					firstEdge[v1] = (unsigned short)edgeCount;
					edgeCount++;
				}
			}
		}
	}
	
	// Mark portal edges.
	for (int i = 0; i < lcset.nconts; ++i)
	{
		const dtTileCacheContour& cont = lcset.conts[i];
		if (cont.nverts < 3)
			continue;
		
		for (int j = 0, k = cont.nverts-1; j < cont.nverts; k=j++)
		{
			const unsigned short* va = &cont.verts[k*4];
			const unsigned short* vb = &cont.verts[j*4];
			
			const unsigned char dir = va[3] & 0xf;
			if (dir == 0xf)
				continue;
			
			if (dir == 0 || dir == 2)
			{
				// Find matching edge on z axis
				const unsigned short x = va[0];
				unsigned short zmin = va[2];
				unsigned short zmax = vb[2];
				unsigned short ya = va[1];
				unsigned short yb = vb[1];
				if (zmin > zmax)
				{
					dtSwap(zmin, zmax);
					dtSwap(ya, yb);
				}
				
				for (int m = 0; m < edgeCount; ++m)
				{
					TileCacheData::dtEdge& e = edges[m];
					// Skip connected edges.
					if (e.poly[0] != e.poly[1])
						continue;

					const unsigned short* eva = &verts[e.vert[0]*3];
					const unsigned short* evb = &verts[e.vert[1]*3];
					if (eva[0] == x && evb[0] == x)
					{
						unsigned short ezmin = eva[2];
						unsigned short ezmax = evb[2];
						unsigned short eya = eva[1];
						unsigned short eyb = evb[1];
						if (ezmin > ezmax)
						{
							dtSwap(ezmin, ezmax);
							dtSwap(eya, eyb);
						}

						if (overlapRangeExl(zmin,zmax, ezmin, ezmax, ya, eya, yb, eyb, walkableClimb))
						{
							// Reuse the other polyedge to store dir.
							e.polyEdge[1] = dir;
						}
					}
				}
			}
			else
			{
				// Find matching edge on x axis
				const unsigned short z = va[2];
				unsigned short xmin = va[0];
				unsigned short xmax = vb[0];
				unsigned short ya = va[1];
				unsigned short yb = vb[1];
				if (xmin > xmax)
				{
					dtSwap(xmin, xmax);
					dtSwap(ya, yb);
				}
				
				for (int m = 0; m < edgeCount; ++m)
				{
					TileCacheData::dtEdge& e = edges[m];
					// Skip connected edges.
					if (e.poly[0] != e.poly[1])
						continue;
					const unsigned short* eva = &verts[e.vert[0]*3];
					const unsigned short* evb = &verts[e.vert[1]*3];
					if (eva[2] == z && evb[2] == z)
					{
						unsigned short exmin = eva[0];
						unsigned short exmax = evb[0];
						unsigned short eya = eva[1];
						unsigned short eyb = evb[1];
						if (exmin > exmax)
						{
							dtSwap(exmin, exmax);
							dtSwap(eya, eyb);
						}

						if (overlapRangeExl(xmin,xmax, exmin, exmax, ya, eya, yb, eyb, walkableClimb))
						{
							// Reuse the other polyedge to store dir.
							e.polyEdge[1] = dir;
						}
					}
				}
			}
		}
	}
	
	
	// Store adjacency
	// Adjacency between poly is store in the second arrays of polys.
	for (int i = 0; i < edgeCount; ++i)
	{
		const TileCacheData::dtEdge& e = edges[i];
		if (e.poly[0] != e.poly[1])
		{
			// If not the same poly, store the direction of the portal.
			unsigned short* p0 = &polys[e.poly[0]*MAX_VERTS_PER_POLY*2];
			unsigned short* p1 = &polys[e.poly[1]*MAX_VERTS_PER_POLY*2];
			p0[MAX_VERTS_PER_POLY + e.polyEdge[0]] = e.poly[1];
			p1[MAX_VERTS_PER_POLY + e.polyEdge[1]] = e.poly[0];
		}
		else if (e.polyEdge[1] != 0xff) // if we have a direction
		{
			// Same poly
			unsigned short* p0 = &polys[e.poly[0]*MAX_VERTS_PER_POLY*2];
			p0[MAX_VERTS_PER_POLY + e.polyEdge[0]] = 0x8000 | (unsigned short)e.polyEdge[1];
		}
		
	}
	
	return true;
}

namespace TileCacheFunc
{
	inline int prev(int i, int n) { return i - 1 >= 0 ? i - 1 : n - 1; }
	inline int next(int i, int n) { return i + 1 < n ? i + 1 : 0; }

	inline int area2(const unsigned short* a, const unsigned short* b, const unsigned short* c)
	{
		return ((int)b[0] - (int)a[0]) * ((int)c[2] - (int)a[2]) - ((int)c[0] - (int)a[0]) * ((int)b[2] - (int)a[2]);
	}

	//	Exclusive or: true iff exactly one argument is true.
	//	The arguments are negated to ensure that they are 0/1
	//	values.  Then the bitwise Xor operator may apply.
	//	(This idea is due to Michael Baldwin.)
	inline bool xorb(bool x, bool y)
	{
		return !x ^ !y;
	}

	// Returns true iff c is strictly to the left of the directed
	// line through a to b.
	inline bool left(const unsigned short* a, const unsigned short* b, const unsigned short* c)
	{
		return area2(a, b, c) < 0;
	}

	inline bool leftOn(const unsigned short* a, const unsigned short* b, const unsigned short* c)
	{
		return area2(a, b, c) <= 0;
	}

	inline bool collinear(const unsigned short* a, const unsigned short* b, const unsigned short* c)
	{
		return area2(a, b, c) == 0;
	}
}

//	Returns true iff ab properly intersects cd: they share
//	a point interior to both segments.  The properness of the
//	intersection is ensured by using strict leftness.
static bool intersectProp(const unsigned short* a, const unsigned short* b,
						  const unsigned short* c, const unsigned short* d)
{
	// Eliminate improper cases.
	if (TileCacheFunc::collinear(a, b, c) || TileCacheFunc::collinear(a, b, d) ||
		TileCacheFunc::collinear(c, d, a) || TileCacheFunc::collinear(c, d, b))
		return false;
	
	return TileCacheFunc::xorb(TileCacheFunc::left(a, b, c), TileCacheFunc::left(a, b, d)) &&
		TileCacheFunc::xorb(TileCacheFunc::left(c, d, a), TileCacheFunc::left(c, d, b));
}

// Returns T iff (a,b,c) are collinear and point c lies 
// on the closed segement ab.
static bool between(const unsigned short* a, const unsigned short* b, const unsigned short* c)
{
	if (!TileCacheFunc::collinear(a, b, c))
		return false;
	// If ab not vertical, check betweenness on x; else on y.
	if (a[0] != b[0])
		return ((a[0] <= c[0]) && (c[0] <= b[0])) || ((a[0] >= c[0]) && (c[0] >= b[0]));
	else
		return ((a[2] <= c[2]) && (c[2] <= b[2])) || ((a[2] >= c[2]) && (c[2] >= b[2]));
}

// Returns true iff segments ab and cd intersect, properly or improperly.
static bool intersect(const unsigned short* a, const unsigned short* b,
					  const unsigned short* c, const unsigned short* d)
{
	if (intersectProp(a, b, c, d))
		return true;
	else if (between(a, b, c) || between(a, b, d) ||
			 between(c, d, a) || between(c, d, b))
		return true;
	else
		return false;
}

static bool vequal(const unsigned short* a, const unsigned short* b)
{
	return a[0] == b[0] && a[2] == b[2];
}

// Returns T iff (v_i, v_j) is a proper internal *or* external
// diagonal of P, *ignoring edges incident to v_i and v_j*.
static bool diagonalie(int i, int j, int n, const unsigned short* verts, const unsigned short* indices)
{
	const unsigned short* d0 = &verts[(indices[i] & 0x7fff) * 4];
	const unsigned short* d1 = &verts[(indices[j] & 0x7fff) * 4];
	
	// For each edge (k,k+1) of P
	for (int k = 0; k < n; k++)
	{
		int k1 = TileCacheFunc::next(k, n);
		// Skip edges incident to i or j
		if (!((k == i) || (k1 == i) || (k == j) || (k1 == j)))
		{
			const unsigned short* p0 = &verts[(indices[k] & 0x7fff) * 4];
			const unsigned short* p1 = &verts[(indices[k1] & 0x7fff) * 4];
			
			if (vequal(d0, p0) || vequal(d1, p0) || vequal(d0, p1) || vequal(d1, p1))
				continue;
			
			if (intersect(d0, d1, p0, p1))
				return false;
		}
	}
	return true;
}

// Returns true iff the diagonal (i,j) is strictly internal to the 
// polygon P in the neighborhood of the i endpoint.
static bool	inCone(int i, int j, int n, const unsigned short* verts, const unsigned short* indices)
{
	const unsigned short* vi = &verts[(indices[i] & 0x7fff) * 4];
	const unsigned short* vj = &verts[(indices[j] & 0x7fff) * 4];
	const unsigned short* vi1 = &verts[(indices[TileCacheFunc::next(i, n)] & 0x7fff) * 4];
	const unsigned short* vin1 = &verts[(indices[TileCacheFunc::prev(i, n)] & 0x7fff) * 4];
	
	// If P[i] is a convex vertex [ i+1 left or on (i-1,i) ].
	if (TileCacheFunc::leftOn(vin1, vi, vi1))
		return TileCacheFunc::left(vi, vj, vin1) && TileCacheFunc::left(vj, vi, vi1);
	// Assume (i-1,i,i+1) not collinear.
	// else P[i] is reflex.
	return !(TileCacheFunc::leftOn(vi, vj, vi1) && TileCacheFunc::leftOn(vj, vi, vin1));
}

// Returns T iff (v_i, v_j) is a proper internal
// diagonal of P.
static bool diagonal(int i, int j, int n, const unsigned short* verts, const unsigned short* indices)
{
	return inCone(i, j, n, verts, indices) && diagonalie(i, j, n, verts, indices);
}

static int triangulate(int n, const unsigned short* verts, unsigned short* indices, unsigned short* tris)
{
	int ntris = 0;
	unsigned short* dst = tris;
	
	// The last bit of the index is used to indicate if the vertex can be removed.
	for (int i = 0; i < n; i++)
	{
		int i1 = TileCacheFunc::next(i, n);
		int i2 = TileCacheFunc::next(i1, n);
		if (diagonal(i, i2, n, verts, indices))
			indices[i1] |= 0x8000;
	}
	
	while (n > 3)
	{
		int minLen = -1;
		int mini = -1;
		for (int i = 0; i < n; i++)
		{
			int i1 = TileCacheFunc::next(i, n);
			if (indices[i1] & 0x8000)
			{
				const unsigned short* p0 = &verts[(indices[i] & 0x7fff) * 4];
				const unsigned short* p2 = &verts[(indices[TileCacheFunc::next(i1, n)] & 0x7fff) * 4];
				
				const int dx = (int)p2[0] - (int)p0[0];
				const int dz = (int)p2[2] - (int)p0[2];
				const int len = dx*dx + dz*dz;
				if (minLen < 0 || len < minLen)
				{
					minLen = len;
					mini = i;
				}
			}
		}
		
		if (mini == -1)
		{
			// Should not happen.
			/*			printf("mini == -1 ntris=%d n=%d\n", ntris, n);
			 for (int i = 0; i < n; i++)
			 {
			 printf("%d ", indices[i] & 0x0fffffff);
			 }
			 printf("\n");*/
			return -ntris;
		}
		
		int i = mini;
		int i1 = TileCacheFunc::next(i, n);
		int i2 = TileCacheFunc::next(i1, n);
		
		*dst++ = indices[i] & 0x7fff;
		*dst++ = indices[i1] & 0x7fff;
		*dst++ = indices[i2] & 0x7fff;
		ntris++;
		
		// Removes P[i1] by copying P[i+1]...P[n-1] left one index.
		n--;
		for (int k = i1; k < n; k++)
			indices[k] = indices[k+1];
		
		if (i1 >= n)
			i1 = 0;

		i = TileCacheFunc::prev(i1, n);
		
		// Update diagonal flags.
		if (diagonal(TileCacheFunc::prev(i, n), i1, n, verts, indices))
			indices[i] |= 0x8000;
		else
			indices[i] &= 0x7fff;
		
		if (diagonal(i, TileCacheFunc::next(i1, n), n, verts, indices))
			indices[i1] |= 0x8000;
		else
			indices[i1] &= 0x7fff;
	}
	
	// Append the remaining triangle.
	*dst++ = indices[0] & 0x7fff;
	*dst++ = indices[1] & 0x7fff;
	*dst++ = indices[2] & 0x7fff;
	ntris++;
	
	return ntris;
}


static int countPolyVerts(const unsigned short* p)
{
	for (int i = 0; i < MAX_VERTS_PER_POLY; ++i)
		if (p[i] == DT_TILECACHE_NULL_IDX)
			return i;
	return MAX_VERTS_PER_POLY;
}

namespace TileCacheFunc
{
	inline bool uleft(const unsigned short* a, const unsigned short* b, const unsigned short* c)
	{
		return ((int)b[0] - (int)a[0]) * ((int)c[2] - (int)a[2]) -
			((int)c[0] - (int)a[0]) * ((int)b[2] - (int)a[2]) < 0;
	}
}

static int getPolyMergeValue(const unsigned short* pa, const unsigned short* pb,
							 const unsigned short* verts, int& ea, int& eb)
{
	const int na = countPolyVerts(pa);
	const int nb = countPolyVerts(pb);
	
	// If the merged polygon would be too big, do not merge.
	if (na+nb-2 > MAX_VERTS_PER_POLY)
		return -1;
	
	// Check if the polygons share an edge.
	ea = -1;
	eb = -1;
	
	for (int i = 0; i < na; ++i)
	{
		unsigned short va0 = pa[i];
		unsigned short va1 = pa[(i+1) % na];
		if (va0 > va1)
			dtSwap(va0, va1);
		for (int j = 0; j < nb; ++j)
		{
			unsigned short vb0 = pb[j];
			unsigned short vb1 = pb[(j+1) % nb];
			if (vb0 > vb1)
				dtSwap(vb0, vb1);
			if (va0 == vb0 && va1 == vb1)
			{
				ea = i;
				eb = j;
				break;
			}
		}
	}
	
	// No common edge, cannot merge.
	if (ea == -1 || eb == -1)
		return -1;
	
	// Check to see if the merged polygon would be convex.
	unsigned short va, vb, vc;
	
	va = pa[(ea+na-1) % na];
	vb = pa[ea];
	vc = pb[(eb+2) % nb];
	if (!TileCacheFunc::uleft(&verts[va * 3], &verts[vb * 3], &verts[vc * 3]))
		return -1;
	
	va = pb[(eb+nb-1) % nb];
	vb = pb[eb];
	vc = pa[(ea+2) % na];
	if (!TileCacheFunc::uleft(&verts[va * 3], &verts[vb * 3], &verts[vc * 3]))
		return -1;
	
	va = pa[ea];
	vb = pa[(ea+1)%na];
	
	int dx = (int)verts[va*3+0] - (int)verts[vb*3+0];
	int dy = (int)verts[va*3+2] - (int)verts[vb*3+2];
	
	return dx*dx + dy*dy;
}

static void mergePolys(unsigned short* pa, unsigned short* pb, int ea, int eb)
{
	unsigned short tmp[MAX_VERTS_PER_POLY*2];
	
	const int na = countPolyVerts(pa);
	const int nb = countPolyVerts(pb);
	
	// Merge polygons.
	memset(tmp, 0xff, sizeof(unsigned short)*MAX_VERTS_PER_POLY*2);
	int n = 0;
	// Add pa
	for (int i = 0; i < na-1; ++i)
		tmp[n++] = pa[(ea+1+i) % na]; //-V609
	// Add pb
	for (int i = 0; i < nb-1; ++i)
		tmp[n++] = pb[(eb+1+i) % nb]; //-V609
	
	memcpy(pa, tmp, sizeof(unsigned short)*MAX_VERTS_PER_POLY);
}


static void pushFront(unsigned short v, unsigned short* arr, int& an)
{
	an++;
	for (int i = an-1; i > 0; --i)
		arr[i] = arr[i-1];
	arr[0] = v;
}

static void pushBack(unsigned short v, unsigned short* arr, int& an)
{
	arr[an] = v;
	an++;
}

static bool canRemoveVertex(dtTileCachePolyMesh& mesh, const unsigned short rem)
{
	// Count number of polygons to remove.
	int numRemovedVerts = 0;
	int numTouchedVerts = 0;
	int numRemainingEdges = 0;
	for (int i = 0; i < mesh.npolys; ++i)
	{
		const unsigned short* p = &mesh.polys[i*MAX_VERTS_PER_POLY*2];
		const int nv = countPolyVerts(p);
		int numRemoved = 0;
		int numVerts = 0;
		for (int j = 0; j < nv; ++j)
		{
			if (p[j] == rem)
			{
				numTouchedVerts++;
				numRemoved++;
			}
			numVerts++;
		}
		if (numRemoved)
		{
			numRemovedVerts += numRemoved;
			numRemainingEdges += numVerts-(numRemoved+1);
		}
	}
	
	// There would be too few edges remaining to create a polygon.
	// This can happen for example when a tip of a triangle is marked
	// as deletion, but there are no other polys that share the vertex.
	// In this case, the vertex should not be removed.
	if (numRemainingEdges <= 2)
		return false;
	
	// Check that there is enough memory for the test.
	const int maxEdges = numTouchedVerts*2;
	if (maxEdges > MAX_REM_EDGES)
		return false;
	
	// Find edges which share the removed vertex.
	unsigned short edges[MAX_REM_EDGES*3];
	int nedges = 0;
	
	for (int i = 0; i < mesh.npolys; ++i)
	{
		const unsigned short* p = &mesh.polys[i*MAX_VERTS_PER_POLY*2];
		const int nv = countPolyVerts(p);
		
		// Collect edges which touches the removed vertex.
		for (int j = 0, k = nv-1; j < nv; k = j++)
		{
			if (p[j] == rem || p[k] == rem)
			{
				// Arrange edge so that a=rem.
				int a = p[j], b = p[k];
				if (b == rem)
					dtSwap(a,b);
				
				// Check if the edge exists
				bool exists = false;
				for (int m = 0; m < nedges; ++m)
				{
					unsigned short* e = &edges[m*3];
					if (e[1] == b)
					{
						// Exists, increment vertex share count.
						e[2]++;
						exists = true;
					}
				}
				// Add new edge.
				if (!exists)
				{
					unsigned short* e = &edges[nedges*3];
					e[0] = (unsigned short)a;
					e[1] = (unsigned short)b;
					e[2] = 1;
					nedges++;
				}
			}
		}
	}
	
	// There should be no more than 2 open edges.
	// This catches the case that two non-adjacent polygons
	// share the removed vertex. In that case, do not remove the vertex.
	int numOpenEdges = 0;
	for (int i = 0; i < nedges; ++i)
	{
		if (edges[i*3+2] < 2)
			numOpenEdges++;
	}
	if (numOpenEdges > 2)
		return false;
	
	return true;
}

static dtStatus removeVertex(dtTileCacheLogContext* ctx, dtTileCachePolyMesh& mesh, const unsigned short rem, const int maxTris)
{
	// Count number of polygons to remove.
	int numRemovedVerts = 0;
	for (int i = 0; i < mesh.npolys; ++i)
	{
		const unsigned short* p = &mesh.polys[i*MAX_VERTS_PER_POLY*2];
		const int nv = countPolyVerts(p);
		for (int j = 0; j < nv; ++j)
		{
			if (p[j] == rem)
				numRemovedVerts++;
		}
	}

	int nedges = 0;
	int nhole = 0;
	int nharea = 0;
	unsigned short edgesStatic[MAX_REM_EDGES * 3];
	unsigned short holeStatic[MAX_REM_EDGES];
	unsigned short hareaStatic[MAX_REM_EDGES];

	const int MaxRemovedVertsStatic = MAX_REM_EDGES / mesh.nvp;
	const int DynamicAllocSize = (numRemovedVerts > MaxRemovedVertsStatic) ? (numRemovedVerts * mesh.nvp) : 0;
	dtScopedDelete<unsigned short> edgesDynamic(DynamicAllocSize * 4);
	dtScopedDelete<unsigned short> holeDynamic(DynamicAllocSize);
	dtScopedDelete<unsigned short> hareaDynamic(DynamicAllocSize);

	unsigned short* edges = (DynamicAllocSize > 0) ? edgesDynamic.get() : edgesStatic;
	unsigned short* hole = (DynamicAllocSize > 0) ? holeDynamic.get() : holeStatic;
	unsigned short* harea = (DynamicAllocSize > 0) ? hareaDynamic.get() : hareaStatic;
	
	for (int i = 0; i < mesh.npolys; ++i)
	{
		unsigned short* p = &mesh.polys[i*MAX_VERTS_PER_POLY*2];
		const int nv = countPolyVerts(p);
		bool hasRem = false;
		for (int j = 0; j < nv; ++j)
		{
			if (p[j] == rem)
			{
				hasRem = true;
				break;
			}
		}

		if (hasRem)
		{
			// Collect edges which does not touch the removed vertex.
			for (int j = 0, k = nv-1; j < nv; k = j++)
			{
				if (p[j] != rem && p[k] != rem)
				{
					unsigned short* e = &edges[nedges*3];
					e[0] = p[k];
					e[1] = p[j];
					e[2] = mesh.areas[i];
					nedges++;
				}
			}

			// Remove the polygon p.
			const unsigned short* lastp = &mesh.polys[(mesh.npolys-1)*MAX_VERTS_PER_POLY*2];
			memcpy(p, lastp, sizeof(unsigned short)*MAX_VERTS_PER_POLY*2);
			mesh.areas[i] = mesh.areas[mesh.npolys-1];
			mesh.npolys--;
			--i;
		}
	}
	
	// Remove vertex.
	for (int i = (int)rem; i < (mesh.nverts - 1); ++i)
	{
		mesh.verts[i*3+0] = mesh.verts[(i+1)*3+0];
		mesh.verts[i*3+1] = mesh.verts[(i+1)*3+1];
		mesh.verts[i*3+2] = mesh.verts[(i+1)*3+2];
	}
	mesh.nverts--;
	
	// Adjust indices to match the removed vertex layout.
	for (int i = 0; i < mesh.npolys; ++i)
	{
		unsigned short* p = &mesh.polys[i*MAX_VERTS_PER_POLY*2];
		const int nv = countPolyVerts(p);
		for (int j = 0; j < nv; ++j)
		{
			if (p[j] > rem)
				p[j]--;
		}
	}
	
	for (int i = 0; i < nedges; ++i)
	{
		if (edges[i*3+0] > rem) edges[i*3+0]--;
		if (edges[i*3+1] > rem) edges[i*3+1]--;
	}
	
	if (nedges == 0)
		return DT_SUCCESS;
	
	// Start with one vertex, keep appending connected
	// segments to the start and end of the hole.
	pushBack(edges[0], hole, nhole);
	pushBack(edges[2], harea, nharea);
	
	while (nedges)
	{
		bool match = false;
		
		for (int i = 0; i < nedges; ++i)
		{
			const unsigned short ea = edges[i*3+0];
			const unsigned short eb = edges[i*3+1];
			const unsigned short area = edges[i*3+2];
			bool add = false;
			if (hole[0] == eb)
			{
				// The segment matches the beginning of the hole boundary.
				pushFront(ea, hole, nhole);
				pushFront(area, harea, nharea);
				add = true;
			}
			else if (hole[nhole-1] == ea)
			{
				// The segment matches the end of the hole boundary.
				pushBack(eb, hole, nhole);
				pushBack(area, harea, nharea);
				add = true;
			}
			
			if (add)
			{
				// The edge segment was added, remove it.
				edges[i*3+0] = edges[(nedges-1)*3+0];
				edges[i*3+1] = edges[(nedges-1)*3+1];
				edges[i*3+2] = edges[(nedges-1)*3+2];
				--nedges;
				match = true;
				--i;
			}
		}
		
		if (!match)
			break;
	}

	// Remove duplicate vertex at end.
	if (nhole > 0)
	{
		if (hole[0] == hole[nhole-1])
			nhole--;
	}
	
	// Skip degenerated areas.
	if (nhole < 3)
		return DT_SUCCESS;
	
	unsigned short trisStatic[MAX_REM_EDGES * 3];
	unsigned short tvertsStatic[MAX_REM_EDGES * 3];
	unsigned short tpolyStatic[MAX_REM_EDGES * 3];

	const int DynamicAllocSize2 = (nhole * 4) > (MAX_REM_EDGES * 3) ? nhole : 0;
	dtScopedDelete<unsigned short> trisDynamic(DynamicAllocSize2 * 3);
	dtScopedDelete<unsigned short> tvertsDynamic(DynamicAllocSize2 * 4);
	dtScopedDelete<unsigned short> tpolyDynamic(DynamicAllocSize2);

	unsigned short* tris = (DynamicAllocSize2 > 0) ? trisDynamic.get() : trisStatic;
	unsigned short* tverts = (DynamicAllocSize2 > 0) ? tvertsDynamic.get() : tvertsStatic;
	unsigned short* tpoly = (DynamicAllocSize2 > 0) ? tpolyDynamic.get() : tpolyStatic;
	
	// Generate temp vertex array for triangulation.
	for (int i = 0; i < nhole; ++i)
	{
		const unsigned short hi = hole[i];
		tverts[i*4+0] = mesh.verts[hi*3+0];
		tverts[i*4+1] = mesh.verts[hi*3+1];
		tverts[i*4+2] = mesh.verts[hi*3+2];
		tverts[i*4+3] = 0;
		tpoly[i] = (unsigned short)i;
	}
	
	// Triangulate the hole.
	int ntris = triangulate(nhole, tverts, tpoly, tris);
	if (ntris < 0)
	{
		ntris = -ntris;
	}

	unsigned short polysStatic[MAX_REM_EDGES*MAX_VERTS_PER_POLY];
	unsigned char pareasStatic[MAX_REM_EDGES];

	const int DynamicAllocSize3 = ((ntris + 1) > MAX_REM_EDGES) ? (ntris + 1) : 0;
	dtScopedDelete<unsigned short> polysDynamic(DynamicAllocSize3 * MAX_VERTS_PER_POLY);
	dtScopedDelete<unsigned char> pareasDynamic(DynamicAllocSize3);

	unsigned short* polys = (DynamicAllocSize3 > 0) ? polysDynamic.get() : polysStatic;
	unsigned char* pareas = (DynamicAllocSize3 > 0) ? pareasDynamic.get() : pareasStatic;
	
	// Build initial polygons.
	int npolys = 0;
	memset(polys, 0xff, ntris*MAX_VERTS_PER_POLY*sizeof(unsigned short));
	for (int j = 0; j < ntris; ++j)
	{
		unsigned short* t = &tris[j*3];
		if (t[0] != t[1] && t[0] != t[2] && t[1] != t[2])
		{
			polys[npolys*MAX_VERTS_PER_POLY+0] = hole[t[0]];
			polys[npolys*MAX_VERTS_PER_POLY+1] = hole[t[1]];
			polys[npolys*MAX_VERTS_PER_POLY+2] = hole[t[2]];
			pareas[npolys] = (unsigned char)harea[t[0]];
			npolys++;
		}
	}
	if (!npolys)
		return DT_SUCCESS;
	
	// Merge polygons.
	int maxVertsPerPoly = MAX_VERTS_PER_POLY;
	if (maxVertsPerPoly > 3) //-V547
	{
		for (;;)
		{
			// Find best polygons to merge.
			int bestMergeVal = 0;
			int bestPa = 0, bestPb = 0, bestEa = 0, bestEb = 0;
			
			for (int j = 0; j < npolys-1; ++j)
			{
				unsigned short* pj = &polys[j*MAX_VERTS_PER_POLY];
				for (int k = j+1; k < npolys; ++k)
				{
					unsigned short* pk = &polys[k*MAX_VERTS_PER_POLY];
					int ea, eb;
					int v = getPolyMergeValue(pj, pk, mesh.verts, ea, eb);
					if (v > bestMergeVal)
					{
						bestMergeVal = v;
						bestPa = j;
						bestPb = k;
						bestEa = ea;
						bestEb = eb;
					}
				}
			}
			
			if (bestMergeVal > 0)
			{
				// Found best, merge.
				unsigned short* pa = &polys[bestPa*MAX_VERTS_PER_POLY];
				unsigned short* pb = &polys[bestPb*MAX_VERTS_PER_POLY];
				mergePolys(pa, pb, bestEa, bestEb);
				memcpy(pb, &polys[(npolys-1)*MAX_VERTS_PER_POLY], sizeof(unsigned short)*MAX_VERTS_PER_POLY);
				pareas[bestPb] = pareas[npolys-1];
				npolys--;
			}
			else
			{
				// Could not merge any polygons, stop.
				break;
			}
		}
	}
	
	// Store polygons.
	for (int i = 0; i < npolys; ++i)
	{
		if (mesh.npolys >= maxTris)
			break;

		unsigned short* newPoly = &mesh.polys[mesh.npolys*MAX_VERTS_PER_POLY*2];
		memset(newPoly,0xff,sizeof(unsigned short)*MAX_VERTS_PER_POLY*2);
		for (int j = 0; j < MAX_VERTS_PER_POLY; ++j)
		{
			newPoly[j] = polys[i*MAX_VERTS_PER_POLY+j];
		}
		mesh.areas[mesh.npolys] = pareas[i];
		mesh.npolys++;
		if (mesh.npolys > maxTris)
		{
			if (ctx)
				ctx->dtLog("removeVertex: Too many polygons %d (max:%d).", mesh.npolys, maxTris);
			return DT_FAILURE | DT_BUFFER_TOO_SMALL;
		}
	}
	
	return DT_SUCCESS;
}


dtStatus dtBuildTileCachePolyMesh(dtTileCacheAlloc* alloc, 
                                  dtTileCacheLogContext* ctx,
                                  dtTileCacheContourSet& lcset,
                                  dtTileCachePolyMesh& mesh,
                                  const int walkableClimb)
{
	dtAssert(alloc);
	
	int maxVertices = 0;
	int maxTris = 0;
	int maxVertsPerCont = 0;

	for (int i = 0; i < lcset.nconts; ++i)
	{
		// Skip null contours.
		if (lcset.conts[i].nverts < 3 || lcset.conts[i].area == DT_TILECACHE_NULL_AREA)
			continue;

		maxVertices += lcset.conts[i].nverts;
		maxTris += lcset.conts[i].nverts - 2;
		maxVertsPerCont = dtMax(maxVertsPerCont, lcset.conts[i].nverts);
	}

	// TODO: warn about too many vertices?
	
	mesh.nvp = MAX_VERTS_PER_POLY;
	
// @UE BEGIN: special handling of "no valid contours"
	if (maxVertices == 0)
	{
		// treating this as success because no issues arised
		// there's just nothing to do.
		// Note that 'mesh' properties are properly initialized to 0 at this point
		// by dtAllocTileCachePolyMesh
		return DT_SUCCESS;
	}
// @UE END

	dtFixedArray<unsigned char> vflags(alloc, maxVertices);
	if (!vflags)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(vflags, 0, maxVertices);
	
	mesh.verts = (unsigned short*)alloc->alloc(sizeof(unsigned short)*maxVertices*3);
	if (!mesh.verts)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	
	mesh.polys = (unsigned short*)alloc->alloc(sizeof(unsigned short)*maxTris*MAX_VERTS_PER_POLY*2);
	if (!mesh.polys)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	mesh.areas = (unsigned char*)alloc->alloc(sizeof(unsigned char)*maxTris);
	if (!mesh.areas)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	mesh.flags = (unsigned short*)alloc->alloc(sizeof(unsigned short)*maxTris);
	if (!mesh.flags)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	mesh.regs = (unsigned short*)alloc->alloc(sizeof(unsigned short)*maxTris);
	if (!mesh.regs)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	// Just allocate and clean the mesh flags array. The user is resposible for filling it.
	memset(mesh.flags, 0, sizeof(unsigned short) * maxTris);
		
	mesh.nverts = 0;
	mesh.npolys = 0;
	
	memset(mesh.verts, 0, sizeof(unsigned short)*maxVertices*3);
	memset(mesh.polys, 0xff, sizeof(unsigned short)*maxTris*MAX_VERTS_PER_POLY*2);
	memset(mesh.areas, 0, sizeof(unsigned char)*maxTris);
	memset(mesh.regs, 0xff, sizeof(unsigned short)*maxTris);
	
	unsigned short firstVert[VERTEX_BUCKET_COUNT2];
	for (int i = 0; i < VERTEX_BUCKET_COUNT2; ++i)
	{
		firstVert[i] = DT_TILECACHE_NULL_IDX;
	}
	
	dtFixedArray<unsigned short> nextVert(alloc, maxVertices);
	if (!nextVert)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(nextVert, 0, sizeof(unsigned short)*maxVertices);
	
	dtFixedArray<unsigned short> indices(alloc, maxVertsPerCont);
	if (!indices)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	
	dtFixedArray<unsigned short> tris(alloc, maxVertsPerCont*3);
	if (!tris)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	dtFixedArray<unsigned short> polys(alloc, maxVertsPerCont*MAX_VERTS_PER_POLY);
	if (!polys)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	
	for (int i = 0; i < lcset.nconts; ++i)
	{
		const dtTileCacheContour& cont = lcset.conts[i];
		
		// Skip null contours.
		if (cont.nverts < 3 || lcset.conts[i].area == DT_TILECACHE_NULL_AREA)
			continue;
		
		// Triangulate contour
		for (int j = 0; j < cont.nverts; ++j)
			indices[j] = (unsigned short)j;
		
		int ntris = triangulate(cont.nverts, cont.verts, &indices[0], &tris[0]);
		if (ntris <= 0)
		{
			// TODO: issue warning!
			ntris = -ntris;
		}
		
		// Add and merge vertices.
		for (int j = 0; j < cont.nverts; ++j)
		{
			const unsigned short* v = &cont.verts[j*4];
			indices[j] = addVertex(v[0], v[1], v[2], mesh.verts, firstVert, nextVert, mesh.nverts);
			if (v[3] & 0x80)
			{
				// This vertex should be removed.
				vflags[indices[j]] = 1;
			}
		}
		
		// Build initial polygons.
		int npolys = 0;
		memset(polys, 0xff, sizeof(unsigned short) * maxVertsPerCont * MAX_VERTS_PER_POLY);
		for (int j = 0; j < ntris; ++j)
		{
			const unsigned short* t = &tris[j*3];
			if (t[0] != t[1] && t[0] != t[2] && t[1] != t[2])
			{
				polys[npolys*MAX_VERTS_PER_POLY+0] = indices[t[0]];
				polys[npolys*MAX_VERTS_PER_POLY+1] = indices[t[1]];
				polys[npolys*MAX_VERTS_PER_POLY+2] = indices[t[2]];
				npolys++;
			}
		}
		
		if (!npolys)
			continue;
		
		// Merge polygons.
		int maxVertsPerPoly = MAX_VERTS_PER_POLY;
		if (maxVertsPerPoly > 3) //-V547
		{
			for(;;)
			{
				// Find best polygons to merge.
				int bestMergeVal = 0;
				int bestPa = 0, bestPb = 0, bestEa = 0, bestEb = 0;
				
				for (int j = 0; j < npolys-1; ++j)
				{
					unsigned short* pj = &polys[j*MAX_VERTS_PER_POLY];
					for (int k = j+1; k < npolys; ++k)
					{
						unsigned short* pk = &polys[k*MAX_VERTS_PER_POLY];
						int ea, eb;
						int v = getPolyMergeValue(pj, pk, mesh.verts, ea, eb);
						if (v > bestMergeVal)
						{
							bestMergeVal = v;
							bestPa = j;
							bestPb = k;
							bestEa = ea;
							bestEb = eb;
						}
					}
				}
				
				if (bestMergeVal > 0)
				{
					// Found best, merge.
					unsigned short* pa = &polys[bestPa*MAX_VERTS_PER_POLY];
					unsigned short* pb = &polys[bestPb*MAX_VERTS_PER_POLY];
					mergePolys(pa, pb, bestEa, bestEb);
					memcpy(pb, &polys[(npolys-1)*MAX_VERTS_PER_POLY], sizeof(unsigned short)*MAX_VERTS_PER_POLY);
					npolys--;
				}
				else
				{
					// Could not merge any polygons, stop.
					break;
				}
			}
		}
		
		// Store polygons.
		for (int j = 0; j < npolys; ++j)
		{
			unsigned short* newPoly = &mesh.polys[mesh.npolys*MAX_VERTS_PER_POLY*2];
			const unsigned short* q = &polys[j*MAX_VERTS_PER_POLY];
			for (int k = 0; k < MAX_VERTS_PER_POLY; ++k)
				newPoly[k] = q[k];
			mesh.areas[mesh.npolys] = cont.area;
			mesh.regs[mesh.npolys] = cont.reg;
			mesh.npolys++;
			if (mesh.npolys > maxTris)
			{
				if (ctx)
					ctx->dtLog("can't store polys! npolys:%d limit:%d", npolys, maxTris);
				return DT_FAILURE | DT_BUFFER_TOO_SMALL;
			}
		}
	}
	
	
	// Remove edge vertices.
	for (int i = 0; i < mesh.nverts; ++i)
	{
		if (vflags[i])
		{
			if (!canRemoveVertex(mesh, (unsigned short)i))
				continue;
			dtStatus status = removeVertex(ctx, mesh, (unsigned short)i, maxTris);
			if (dtStatusFailed(status))
				return status;
			
			// Remove vertex
			// Note: mesh.nverts is already decremented inside removeVertex()!
			for (int j = i; j < mesh.nverts; ++j)
			{
				vflags[j] = vflags[j+1];
			}
			--i;
		}
	}
	
	// Calculate adjacency.
	if (!buildMeshAdjacency(alloc, mesh.polys, mesh.npolys, mesh.verts, mesh.nverts, lcset, walkableClimb))
		return DT_FAILURE | DT_OUT_OF_MEMORY;
		
	return DT_SUCCESS;
}

dtStatus dtMarkCylinderArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
							const dtReal* pos, const dtReal radius, const dtReal height, const unsigned char areaId)
{
	dtReal bmin[3], bmax[3];
	bmin[0] = pos[0] - radius;
	bmin[1] = pos[1];
	bmin[2] = pos[2] - radius;
	bmax[0] = pos[0] + radius;
	bmax[1] = pos[1] + height;
	bmax[2] = pos[2] + radius;
	const dtReal r2 = dtSqr(radius/cs + 0.5f);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal ics = 1.0f/cs;
	const dtReal ich = 1.0f/ch;
	
	const dtReal px = (pos[0]-orig[0])*ics;
	const dtReal pz = (pos[2]-orig[2])*ics;
	
	int minx = (int)dtFloor((bmin[0]-orig[0])*ics);
	int miny = (int)dtFloor((bmin[1]-orig[1])*ich);
	int minz = (int)dtFloor((bmin[2]-orig[2])*ics);
	int maxx = (int)dtFloor((bmax[0]-orig[0])*ics);
	int maxy = (int)dtFloor((bmax[1]-orig[1])*ich);
	int maxz = (int)dtFloor((bmax[2]-orig[2])*ics);

	if (maxx < 0) return DT_SUCCESS;
	if (minx >= w) return DT_SUCCESS;
	if (maxz < 0) return DT_SUCCESS;
	if (minz >= h) return DT_SUCCESS;
	
	if (minx < 0) minx = 0;
	if (maxx >= w) maxx = w-1;
	if (minz < 0) minz = 0;
	if (maxz >= h) maxz = h-1;
	
	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			if (layer.areas[x+z*w] == DT_TILECACHE_NULL_AREA)
				continue;

			const dtReal dx = dtReal(x)+0.5f-px;
			const dtReal dz = dtReal(z)+0.5f-pz;
			if (dx*dx + dz*dz > r2)
				continue;
			const int y = layer.heights[x+z*w];
			if (y < miny || y > maxy)
				continue;
			layer.areas[x+z*w] = areaId;
		}
	}

	return DT_SUCCESS;
}

dtStatus dtMarkBoxArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* pos, const dtReal* extent, const unsigned char areaId)
{
	dtReal bmin[3], bmax[3];
	dtVsub(bmin, pos, extent);
	dtVadd(bmax, pos, extent);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal ics = 1.0f/cs;
	const dtReal ich = 1.0f/ch;

	int minx = (int)dtFloor((bmin[0]-orig[0])*ics);
	int miny = (int)dtFloor((bmin[1]-orig[1])*ich);
	int minz = (int)dtFloor((bmin[2]-orig[2])*ics);
	int maxx = (int)dtFloor((bmax[0]-orig[0])*ics);
	int maxy = (int)dtFloor((bmax[1]-orig[1])*ich);
	int maxz = (int)dtFloor((bmax[2]-orig[2])*ics);

	if (maxx < 0) return DT_SUCCESS;
	if (minx >= w) return DT_SUCCESS;
	if (maxz < 0) return DT_SUCCESS;
	if (minz >= h) return DT_SUCCESS;

	if (minx < 0) minx = 0;
	if (maxx >= w) maxx = w-1;
	if (minz < 0) minz = 0;
	if (maxz >= h) maxz = h-1;

	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			if (layer.areas[x+z*w] == DT_TILECACHE_NULL_AREA)
				continue;

			const int y = layer.heights[x+z*w];
			if (y < miny || y > maxy)
				continue;
			layer.areas[x+z*w] = areaId;
		}
	}

	return DT_SUCCESS;
}

namespace TileCacheFunc
{
	static int pointInPoly(int nvert, const dtReal* verts, const dtReal* p)
	{
		int i, j, c = 0;
		for (i = 0, j = nvert - 1; i < nvert; j = i++)
		{
			const dtReal* vi = &verts[i * 3];
			const dtReal* vj = &verts[j * 3];
			if (((vi[2] > p[2]) != (vj[2] > p[2])) &&
				(p[0] < (vj[0] - vi[0]) * (p[2] - vi[2]) / (vj[2] - vi[2]) + vi[0]))
				c = !c;
		}
		return c;
	}
}

dtStatus dtMarkConvexArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* verts, const int nverts, const dtReal hmin, const dtReal hmax,
	const unsigned char areaId)
{
	dtReal bmin[3], bmax[3];
	dtVcopy(bmin, verts);
	dtVcopy(bmax, verts);
	for (int i = 1; i < nverts; ++i)
	{
		dtVmin(bmin, &verts[i*3]);
		dtVmax(bmax, &verts[i*3]);
	}
	bmin[1] = hmin;
	bmax[1] = hmax;

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal ics = 1.0f/cs;
	const dtReal ich = 1.0f/ch;

	int minx = (int)dtFloor((bmin[0]-orig[0])*ics);
	int miny = (int)dtFloor((bmin[1]-orig[1])*ich);
	int minz = (int)dtFloor((bmin[2]-orig[2])*ics);
	int maxx = (int)dtFloor((bmax[0]-orig[0])*ics);
	int maxy = (int)dtFloor((bmax[1]-orig[1])*ich);
	int maxz = (int)dtFloor((bmax[2]-orig[2])*ics);

	if (maxx < 0) return DT_SUCCESS;
	if (minx >= w) return DT_SUCCESS;
	if (maxz < 0) return DT_SUCCESS;
	if (minz >= h) return DT_SUCCESS;

	if (minx < 0) minx = 0;
	if (maxx >= w) maxx = w-1;
	if (minz < 0) minz = 0;
	if (maxz >= h) maxz = h-1;

	// TODO: Optimize.
	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			if (layer.areas[x+z*w] == DT_TILECACHE_NULL_AREA)
				continue;

			const int y = layer.heights[x+z*w];
			if (y < miny || y > maxy)
				continue;

			dtReal p[3];
			p[0] = orig[0] + (dtReal(x)+0.5f)*cs;
			p[1] = 0.0f;
			p[2] = orig[2] + (dtReal(z)+0.5f)*cs;

			if (TileCacheFunc::pointInPoly(nverts, verts, p))
			{
				layer.areas[x+z*w] = areaId;
			}
		}
	}

	return DT_SUCCESS;
}

dtStatus dtReplaceCylinderArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* pos, const dtReal radius, const dtReal height, const unsigned char areaId, const unsigned char filterAreaId)
{
	dtReal bmin[3], bmax[3];
	bmin[0] = pos[0] - radius;
	bmin[1] = pos[1];
	bmin[2] = pos[2] - radius;
	bmax[0] = pos[0] + radius;
	bmax[1] = pos[1] + height;
	bmax[2] = pos[2] + radius;
	const dtReal r2 = dtSqr(radius / cs + 0.5f);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal ics = 1.0f / cs;
	const dtReal ich = 1.0f / ch;

	const dtReal px = (pos[0] - orig[0])*ics;
	const dtReal pz = (pos[2] - orig[2])*ics;

	int minx = (int)dtFloor((bmin[0] - orig[0])*ics);
	int miny = (int)dtFloor((bmin[1] - orig[1])*ich);
	int minz = (int)dtFloor((bmin[2] - orig[2])*ics);
	int maxx = (int)dtFloor((bmax[0] - orig[0])*ics);
	int maxy = (int)dtFloor((bmax[1] - orig[1])*ich);
	int maxz = (int)dtFloor((bmax[2] - orig[2])*ics);

	if (maxx < 0) return DT_SUCCESS;
	if (minx >= w) return DT_SUCCESS;
	if (maxz < 0) return DT_SUCCESS;
	if (minz >= h) return DT_SUCCESS;

	if (minx < 0) minx = 0;
	if (maxx >= w) maxx = w - 1;
	if (minz < 0) minz = 0;
	if (maxz >= h) maxz = h - 1;

	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			if (layer.areas[x + z*w] != filterAreaId)
				continue;

			const dtReal dx = dtReal(x)+0.5f-px;
			const dtReal dz = dtReal(z)+0.5f-pz;
			if (dx*dx + dz*dz > r2)
				continue;
			const int y = layer.heights[x + z*w];
			if (y < miny || y > maxy)
				continue;

			layer.areas[x + z*w] = areaId;
		}
	}

	return DT_SUCCESS;
}

dtStatus dtReplaceBoxArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* pos, const dtReal* extent, const unsigned char areaId, const unsigned char filterAreaId)
{
	dtReal bmin[3], bmax[3];
	dtVsub(bmin, pos, extent);
	dtVadd(bmax, pos, extent);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal ics = 1.0f / cs;
	const dtReal ich = 1.0f / ch;

	int minx = (int)dtFloor((bmin[0] - orig[0])*ics);
	int miny = (int)dtFloor((bmin[1] - orig[1])*ich);
	int minz = (int)dtFloor((bmin[2] - orig[2])*ics);
	int maxx = (int)dtFloor((bmax[0] - orig[0])*ics);
	int maxy = (int)dtFloor((bmax[1] - orig[1])*ich);
	int maxz = (int)dtFloor((bmax[2] - orig[2])*ics);

	if (maxx < 0) return DT_SUCCESS;
	if (minx >= w) return DT_SUCCESS;
	if (maxz < 0) return DT_SUCCESS;
	if (minz >= h) return DT_SUCCESS;

	if (minx < 0) minx = 0;
	if (maxx >= w) maxx = w - 1;
	if (minz < 0) minz = 0;
	if (maxz >= h) maxz = h - 1;

	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			if (layer.areas[x + z*w] != filterAreaId)
				continue;

			const int y = layer.heights[x + z*w];
			if (y < miny || y > maxy)
				continue;

			layer.areas[x + z*w] = areaId;
		}
	}

	return DT_SUCCESS;
}

dtStatus dtReplaceConvexArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* verts, const int nverts, const dtReal hmin, const dtReal hmax,
	const unsigned char areaId, const unsigned char filterAreaId)
{
	dtReal bmin[3], bmax[3];
	dtVcopy(bmin, verts);
	dtVcopy(bmax, verts);
	for (int i = 1; i < nverts; ++i)
	{
		dtVmin(bmin, &verts[i * 3]);
		dtVmax(bmax, &verts[i * 3]);
	}
	bmin[1] = hmin;
	bmax[1] = hmax;

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const dtReal ics = 1.0f / cs;
	const dtReal ich = 1.0f / ch;

	int minx = (int)dtFloor((bmin[0] - orig[0])*ics);
	int miny = (int)dtFloor((bmin[1] - orig[1])*ich);
	int minz = (int)dtFloor((bmin[2] - orig[2])*ics);
	int maxx = (int)dtFloor((bmax[0] - orig[0])*ics);
	int maxy = (int)dtFloor((bmax[1] - orig[1])*ich);
	int maxz = (int)dtFloor((bmax[2] - orig[2])*ics);

	if (maxx < 0) return DT_SUCCESS;
	if (minx >= w) return DT_SUCCESS;
	if (maxz < 0) return DT_SUCCESS;
	if (minz >= h) return DT_SUCCESS;

	if (minx < 0) minx = 0;
	if (maxx >= w) maxx = w - 1;
	if (minz < 0) minz = 0;
	if (maxz >= h) maxz = h - 1;

	// TODO: Optimize.
	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			if (layer.areas[x + z*w] != filterAreaId)
				continue;

			const int y = layer.heights[x + z*w];
			if (y < miny || y > maxy)
				continue;

			dtReal p[3];
			p[0] = orig[0] + (dtReal(x) + 0.5f)*cs;
			p[1] = 0.0f;
			p[2] = orig[2] + (dtReal(z) + 0.5f)*cs;

			if (TileCacheFunc::pointInPoly(nverts, verts, p))
			{
				layer.areas[x + z*w] = areaId;
			}
		}
	}

	return DT_SUCCESS;
}

dtStatus dtReplaceArea(dtTileCacheLayer& layer, const unsigned char areaId, const unsigned char filterAreaId)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const int maxIdx = w * h;

	for (int i = 0; i < maxIdx; i++)
	{
		if (layer.areas[i] == filterAreaId)
		{
			layer.areas[i] = areaId;
		}
	}

	return DT_SUCCESS;
}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
dtStatus dtBuildTileCacheClusters(dtTileCacheAlloc* alloc, dtTileCacheClusterSet& lclusters, dtTileCachePolyMesh& lmesh)
{
	lclusters.npolys = lmesh.npolys;
	// special handling of "no polys"
	if (lmesh.npolys == 0)
	{
		// treating this as success there's just nothing to do
		// Note that at this point lclusters is properly initialized to 0
		// by dtAllocTileCacheClusterSet
		return DT_SUCCESS;
	}

	lclusters.polyMap = (unsigned short*)alloc->alloc(sizeof(unsigned short)*lclusters.npolys);
	if (!lclusters.polyMap)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	memset(lclusters.polyMap, 0, sizeof(unsigned short)*lclusters.npolys);
	for (int i = 0; i < lclusters.npolys; i++)
	{
		unsigned short reg = lmesh.regs[i];
		if (reg < lclusters.nregs)
		{
			dtAssert(reg < lclusters.nregs);
			lclusters.polyMap[i] = lclusters.regMap[reg];
		}
	}
	
	return DT_SUCCESS;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

dtStatus dtBuildTileCacheLayer(dtTileCacheCompressor* comp,
							   dtTileCacheLayerHeader* header,
							   const unsigned short* heights,
							   const unsigned char* areas,
							   const unsigned char* cons,
							   unsigned char** outData, int* outDataSize)
{
	const int headerSize = dtAlign(sizeof(dtTileCacheLayerHeader));
	const int gridSize = (int)header->width * (int)header->height;
	const int bufferSize = gridSize * 4;
	const int maxDataSize = headerSize + comp->maxCompressedSize(bufferSize);
	unsigned char* data = (unsigned char*)dtAlloc(maxDataSize, DT_ALLOC_PERM_TILE_DATA);
	if (!data)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(data, 0, maxDataSize);
	
	// Store header
	memcpy(data, header, sizeof(dtTileCacheLayerHeader));
	
	// Concatenate grid data for compression.
	unsigned char* buffer = (unsigned char*)dtAlloc(bufferSize, DT_ALLOC_TEMP);
	if (!buffer)
	{
		dtFree(data, DT_ALLOC_PERM_TILE_DATA);
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}
	memcpy(buffer, heights, gridSize*2);
	memcpy(buffer+gridSize*2, areas, gridSize);
	memcpy(buffer+gridSize*3, cons, gridSize);
	
	// Compress
	unsigned char* compressed = data + headerSize;
	const int maxCompressedSize = maxDataSize - headerSize;
	int compressedSize = 0;
	dtStatus status = comp->compress(buffer, bufferSize, compressed, maxCompressedSize, &compressedSize);

	dtFree(buffer, DT_ALLOC_TEMP);
	*outData = data;
	*outDataSize = headerSize + compressedSize;
	
	return status;
}

void dtFreeTileCacheLayer(dtTileCacheAlloc* alloc, dtTileCacheLayer* layer)
{
	QUICK_SCOPE_CYCLE_COUNTER(dtFreeTileCacheLayer);
	
	dtAssert(alloc);
	// The layer is allocated as one contiguous blob of data.
	alloc->free(layer);
}

dtStatus dtDecompressTileCacheLayer(dtTileCacheAlloc* alloc, dtTileCacheCompressor* comp,
									const unsigned char* compressed, const int compressedSize,
									dtTileCacheLayer** layerOut)
{
	dtAssert(alloc);
	dtAssert(comp);

	if (!layerOut)
		return DT_FAILURE | DT_INVALID_PARAM;
	if (!compressed)
		return DT_FAILURE | DT_INVALID_PARAM;

	*layerOut = 0;

	dtTileCacheLayerHeader* compressedHeader = (dtTileCacheLayerHeader*)compressed;
	if (compressedHeader->version != DT_TILECACHE_VERSION)
		return DT_FAILURE | DT_WRONG_VERSION;
	
	const int layerSize = dtAlign(sizeof(dtTileCacheLayer));
	const int headerSize = dtAlign(sizeof(dtTileCacheLayerHeader));
	const int gridSize = (int)compressedHeader->width * (int)compressedHeader->height;
	const int bufferSize = layerSize + headerSize + gridSize*6;
	
	unsigned char* buffer = (unsigned char*)alloc->alloc(bufferSize);
	if (!buffer)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(buffer, 0, bufferSize);

	dtTileCacheLayer* layer = (dtTileCacheLayer*)buffer;
	dtTileCacheLayerHeader* header = (dtTileCacheLayerHeader*)(buffer + layerSize);
	unsigned char* grids = buffer + layerSize + headerSize;
	const int gridsSize = bufferSize - (layerSize + headerSize); 
	
	// Copy header
	memcpy(header, compressedHeader, headerSize);
	// Decompress grid.
	int size = 0;
	dtStatus status = comp->decompress(compressed+headerSize, compressedSize-headerSize,
									   grids, gridsSize, &size);
	if (dtStatusFailed(status))
	{
		dtFree(buffer, DT_ALLOC_TEMP);
		return status;
	}
	
	layer->header = header;
	layer->heights = (unsigned short*)grids;
	layer->areas = grids + gridSize*2;
	layer->cons = grids + gridSize*3;
	layer->regs = (unsigned short*)(grids + gridSize*4);

	*layerOut = layer;
	
	return DT_SUCCESS;
}



bool dtTileCacheHeaderSwapEndian(unsigned char* data, const int dataSize)
{
	dtTileCacheLayerHeader* header = (dtTileCacheLayerHeader*)data;
	
	int swappedMagic = DT_TILECACHE_MAGIC;
	int swappedVersion = DT_TILECACHE_VERSION;
	dtSwapEndian(&swappedMagic);
	dtSwapEndian(&swappedVersion);
	
	if ((header->version != DT_TILECACHE_VERSION) &&
		(header->version != swappedVersion))
	{
		return false;
	}
	
	dtSwapEndian(&header->version);
	dtSwapEndian(&header->tx);
	dtSwapEndian(&header->ty);
	dtSwapEndian(&header->tlayer);
	dtSwapEndian(&header->bmin[0]);
	dtSwapEndian(&header->bmin[1]);
	dtSwapEndian(&header->bmin[2]);
	dtSwapEndian(&header->bmax[0]);
	dtSwapEndian(&header->bmax[1]);
	dtSwapEndian(&header->bmax[2]);
	dtSwapEndian(&header->hmin);	// @todo: remove
	dtSwapEndian(&header->hmax);

	dtSwapEndian(&header->width);
	dtSwapEndian(&header->height);
	dtSwapEndian(&header->minx);
	dtSwapEndian(&header->maxx);
	dtSwapEndian(&header->miny);
	dtSwapEndian(&header->maxy);
	
	return true;
}

void dtTileCacheLogContext::dtLog(const char* format, ...)
{
	static const int MSG_SIZE = 512;
	char msg[MSG_SIZE];
	va_list ap;
	va_start(ap, format);
	int len = FCStringAnsi::GetVarArgs(msg, MSG_SIZE, format, ap);
	if (len >= MSG_SIZE)
	{
		len = MSG_SIZE - 1;
		msg[MSG_SIZE - 1] = '\0';
	}
	va_end(ap);
	doDtLog(msg, len);
}
