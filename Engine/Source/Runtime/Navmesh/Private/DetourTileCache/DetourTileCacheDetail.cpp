// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "CoreMinimal.h"
#include "Detour/DetourCommon.h"
#include "Detour/DetourAssert.h"
#include "Detour/DetourAlloc.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "DetourTileCache/DetourTileCacheBuilder.h"
#define _USE_MATH_DEFINES

static const unsigned DT_UNSET_PATCH_HEIGHT = 0xffff;
static const unsigned DT_UNSET_LAYER_HEIGHT = 0xffff;

struct dtHeightPatch
{
	inline dtHeightPatch() : data(0), xmin(0), ymin(0), width(0), height(0) {}
	inline ~dtHeightPatch() { dtFree(data, DT_ALLOC_TEMP); }
	unsigned short* data;
	int xmin, ymin, width, height;
};

inline bool isCellConnected(const dtTileCacheLayer& layer, const int idx, const int dir)
{
	return (layer.cons[idx] & (1 << dir)) != 0;
}

static void getLayerHeightData(dtTileCacheLayer& layer,
	const unsigned short* poly, const unsigned short* verts, const int nverts,
	dtHeightPatch& hp, dtIntArray& stack)
{
	// Floodfill the heightfield to get 2D height data,
	// starting at vertex locations as seeds.

	// Note: Reads to the compact heightfield are offset by border size (bs)
	// since border size offset is already removed from the polymesh vertices.

	memset(hp.data, 0, sizeof(unsigned short)*hp.width*hp.height);
	stack.resize(0);

	static const int offset[9*2] =
	{
		0,0, -1,-1, 0,-1, 1,-1, 1,0, 1,1, 0,1, -1,1, -1,0,
	};

	// Use poly vertices as seed points for the flood fill.
	for (int j = 0; j < nverts; ++j)
	{
		int dmin = DT_UNSET_LAYER_HEIGHT;
		int cx = 0, cy = 0;
		for (int k = 0; k < 9; ++k)
		{
			const int ax = (int)verts[poly[j]*3+0] + offset[k*2+0];
			const int ay = (int)verts[poly[j]*3+2] + offset[k*2+1];

			if (ax < hp.xmin || ax >= hp.xmin+hp.width ||
				ay < hp.ymin || ay >= hp.ymin+hp.height)
			{
				continue;
			}

			const int hidx = ax+ay*layer.header->width;
			const int d = layer.heights[hidx];
			if (d < dmin)
			{
				cx = ax;
				cy = ay;
				dmin = d;
			}
		}

		if (dmin != DT_UNSET_LAYER_HEIGHT)
		{
			stack.push(cx);
			stack.push(cy);
		}
	}

	// Find center of the polygon using flood fill.
	int pcx = 0, pcy = 0;
	for (int j = 0; j < nverts; ++j)
	{
		pcx += (int)verts[poly[j]*3+0];
		pcy += (int)verts[poly[j]*3+2];
	}
	pcx /= nverts;
	pcy /= nverts;

	for (int i = 0; i < stack.size(); i += 2)
	{
		const int cx = stack[i+0];
		const int cy = stack[i+1];
		const int idx = cx-hp.xmin+(cy-hp.ymin)*hp.width;
		hp.data[idx] = 1;
	}

	while (stack.size() > 0)
	{
		int cy = stack.pop();
		int cx = stack.pop();

		// Check if close to center of the polygon.
		if (dtAbs(cx-pcx) <= 1 && dtAbs(cy-pcy) <= 1)
		{
			stack.resize(0);
			stack.push(cx);
			stack.push(cy);
			break;
		}

		for (int dir = 0; dir < 4; ++dir)
		{
			const int ax = cx + getDirOffsetX(dir);
			const int ay = cy + getDirOffsetY(dir);
			const int idx = ax-hp.xmin+(ay-hp.ymin)*hp.width;
			const int hidx = ax+ay*layer.header->width;

			if (ax < hp.xmin || ax >= (hp.xmin+hp.width) ||
				ay < hp.ymin || ay >= (hp.ymin+hp.height) ||
				layer.heights[hidx] == DT_UNSET_LAYER_HEIGHT ||
				hp.data[idx] != 0 ||
				!isCellConnected(layer, cx+cy*layer.header->width, dir))
			{
				continue;
			}

			hp.data[idx] = 1;
			stack.push(ax);
			stack.push(ay);
		}
	}

	memset(hp.data, 0xff, sizeof(unsigned short)*hp.width*hp.height);

	// Mark start locations.
	for (int i = 0; i < stack.size(); i += 2)
	{
		const int cx = stack[i+0];
		const int cy = stack[i+1];
		const int idx = cx-hp.xmin+(cy-hp.ymin)*hp.width;
		const int hidx = cx+cy*layer.header->width;
		hp.data[idx] = layer.heights[hidx];
	}

	static const int RETRACT_SIZE = 256;
	int head = 0;

	while (head*2 < stack.size())
	{
		int cx = stack[head*2+0];
		int cy = stack[head*2+1];
		head++;
		if (head >= RETRACT_SIZE)
		{
			head = 0;
			if (stack.size() > RETRACT_SIZE*2)
				memmove(&stack[0], &stack[RETRACT_SIZE*2], sizeof(int)*(stack.size()-RETRACT_SIZE*2));
			stack.resize(stack.size()-RETRACT_SIZE*2);
		}

		for (int dir = 0; dir < 4; ++dir)
		{
			const int ax = cx + getDirOffsetX(dir);
			const int ay = cy + getDirOffsetY(dir);
			const int idx = ax-hp.xmin+(ay-hp.ymin)*hp.width;
			const int hidx = ax+ay*layer.header->width;

			if (ax < hp.xmin || ax >= (hp.xmin+hp.width) ||
				ay < hp.ymin || ay >= (hp.ymin+hp.height) ||
				hp.data[idx] != DT_UNSET_PATCH_HEIGHT ||
				layer.heights[hidx] == DT_UNSET_LAYER_HEIGHT ||
				!isCellConnected(layer, cx+cy*layer.header->width, dir))
			{
				continue;
			}

			hp.data[idx] = layer.heights[hidx];
			stack.push(ax);
			stack.push(ay);
		}
	}
}

static unsigned short getHeight(const dtReal fx, const dtReal fy, const dtReal fz,
	const dtReal /*cs*/, const dtReal ics, const dtReal ch,
	const dtHeightPatch& hp)
{
	int ix = (int)dtFloor(fx*ics + 0.01f);
	int iz = (int)dtFloor(fz*ics + 0.01f);
	ix = dtClamp(ix-hp.xmin, 0, hp.width - 1);
	iz = dtClamp(iz-hp.ymin, 0, hp.height - 1);
	unsigned short h = hp.data[ix+iz*hp.width];
	if (h == DT_UNSET_PATCH_HEIGHT)
	{
		//@UE BEGIN
		// setting fallback value in case proper height is not found
		h = (unsigned short)dtFloor(fy/ch);
		//@UE END

		// Special case when data might be bad.
		// Find nearest neighbour pixel which has valid height.
		const int off[8*2] = { -1,0, -1,-1, 0,-1, 1,-1, 1,0, 1,1, 0,1, -1,1};
		dtReal dmin = DT_REAL_MAX;
		for (int i = 0; i < 8; ++i)
		{
			const int nx = ix+off[i*2+0];
			const int nz = iz+off[i*2+1];
			if (nx < 0 || nz < 0 || nx >= hp.width || nz >= hp.height) continue;
			const unsigned short nh = hp.data[nx+nz*hp.width];
			if (nh == DT_UNSET_PATCH_HEIGHT) continue;

			const dtReal d = dtAbs(nh*ch - fy);
			if (d < dmin)
			{
				h = nh;
				dmin = d;
			}
		}
	}
	return h;
}

namespace TileCacheFunc
{
	inline dtReal vdot2(const dtReal* a, const dtReal* b)
	{
		return a[0] * b[0] + a[2] * b[2];
	}

	inline dtReal vdistSq2(const dtReal* p, const dtReal* q)
	{
		const dtReal dx = q[0] - p[0];
		const dtReal dy = q[2] - p[2];
		return dx*dx + dy*dy;
	}

	inline dtReal vdist2(const dtReal* p, const dtReal* q)
	{
		return dtSqrt(vdistSq2(p, q));
	}

	inline dtReal vcross2(const dtReal* p1, const dtReal* p2, const dtReal* p3)
	{
		const dtReal u1 = p2[0] - p1[0];
		const dtReal v1 = p2[2] - p1[2];
		const dtReal u2 = p3[0] - p1[0];
		const dtReal v2 = p3[2] - p1[2];
		return u1 * v2 - v1 * u2;
	}

	static dtReal distancePtSeg(const dtReal* pt, const dtReal* p, const dtReal* q)
	{
		dtReal pqx = q[0] - p[0];
		dtReal pqy = q[1] - p[1];
		dtReal pqz = q[2] - p[2];
		dtReal dx = pt[0] - p[0];
		dtReal dy = pt[1] - p[1];
		dtReal dz = pt[2] - p[2];
		dtReal d = pqx*pqx + pqy*pqy + pqz*pqz;
		dtReal t = pqx*dx + pqy*dy + pqz*dz;
		if (d > 0)
			t /= d;
		if (t < 0)
			t = 0;
		else if (t > 1)
			t = 1;

		dx = p[0] + t*pqx - pt[0];
		dy = p[1] + t*pqy - pt[1];
		dz = p[2] + t*pqz - pt[2];

		return dx*dx + dy*dy + dz*dz;
	}

	static dtReal distancePtSeg2d(const dtReal* pt, const dtReal* p, const dtReal* q)
	{
		dtReal pqx = q[0] - p[0];
		dtReal pqz = q[2] - p[2];
		dtReal dx = pt[0] - p[0];
		dtReal dz = pt[2] - p[2];
		dtReal d = pqx*pqx + pqz*pqz;
		dtReal t = pqx*dx + pqz*dz;
		if (d > 0)
			t /= d;
		if (t < 0)
			t = 0;
		else if (t > 1)
			t = 1;

		dx = p[0] + t*pqx - pt[0];
		dz = p[2] + t*pqz - pt[2];

		return dx*dx + dz*dz;
	}

	static dtReal distPtTri(const dtReal* p, const dtReal* a, const dtReal* b, const dtReal* c)
	{
		dtReal v0[3], v1[3], v2[3];
		dtVsub(v0, c, a);
		dtVsub(v1, b, a);
		dtVsub(v2, p, a);

		const dtReal dot00 = vdot2(v0, v0);
		const dtReal dot01 = vdot2(v0, v1);
		const dtReal dot02 = vdot2(v0, v2);
		const dtReal dot11 = vdot2(v1, v1);
		const dtReal dot12 = vdot2(v1, v2);

		// Compute barycentric coordinates
		const dtReal invDenom = 1.0f / (dot00 * dot11 - dot01 * dot01);
		const dtReal u = (dot11 * dot02 - dot01 * dot12) * invDenom;
		dtReal v = (dot00 * dot12 - dot01 * dot02) * invDenom;

		// If point lies inside the triangle, return interpolated y-coord.
		static const dtReal EPS = 1e-4f;
		if (u >= -EPS && v >= -EPS && (u + v) <= 1 + EPS)
		{
			const dtReal y = a[1] + v0[1] * u + v1[1] * v;
			return dtAbs(y - p[1]);
		}
		return DT_REAL_MAX;
	}

	static dtReal distToTriMesh(const dtReal* p, const dtReal* verts, const int /*nverts*/, const int* tris, const int ntris)
	{
		dtReal dmin = DT_REAL_MAX;
		for (int i = 0; i < ntris; ++i)
		{
			const dtReal* va = &verts[tris[i * 4 + 0] * 3];
			const dtReal* vb = &verts[tris[i * 4 + 1] * 3];
			const dtReal* vc = &verts[tris[i * 4 + 2] * 3];
			dtReal d = distPtTri(p, va, vb, vc);
			if (d < dmin)
				dmin = d;
		}
		if (dmin == DT_REAL_MAX) return -1;
		return dmin;
	}

	static dtReal distToPoly(int nvert, const dtReal* verts, const dtReal* p)
	{
		dtReal dmin = DT_REAL_MAX;
		int i, j, c = 0;
		for (i = 0, j = nvert - 1; i < nvert; j = i++)
		{
			const dtReal* vi = &verts[i * 3];
			const dtReal* vj = &verts[j * 3];
			if (((vi[2] > p[2]) != (vj[2] > p[2])) &&
				(p[0] < (vj[0] - vi[0]) * (p[2] - vi[2]) / (vj[2] - vi[2]) + vi[0]))
				c = !c;
			dmin = dtMin(dmin, distancePtSeg2d(p, vj, vi));
		}
		return c ? -dmin : dmin;
	}

	static bool circumCircle(const dtReal* p1, const dtReal* p2, const dtReal* p3, dtReal* c, dtReal& r)
	{
		static const dtReal EPS = 1e-6f;

		const dtReal cp = vcross2(p1, p2, p3);
		if (dtAbs(cp) > EPS)
		{
			const dtReal p1Sq = vdot2(p1, p1);
			const dtReal p2Sq = vdot2(p2, p2);
			const dtReal p3Sq = vdot2(p3, p3);
			c[0] = (p1Sq*(p2[2] - p3[2]) + p2Sq*(p3[2] - p1[2]) + p3Sq*(p1[2] - p2[2])) / (2 * cp);
			c[2] = (p1Sq*(p3[0] - p2[0]) + p2Sq*(p1[0] - p3[0]) + p3Sq*(p2[0] - p1[0])) / (2 * cp);
			r = vdist2(c, p1);
			return true;
		}

		c[0] = p1[0];
		c[2] = p1[2];
		r = 0;
		return false;
	}
}

namespace TileCacheData
{
	enum EdgeValues
	{
		UNDEF = -1,
		HULL = -2,
	};
}

namespace TileCacheFunc
{
	static int findEdge(const int* edges, int nedges, int s, int t)
	{
		for (int i = 0; i < nedges; i++)
		{
			const int* e = &edges[i * 4];
			if ((e[0] == s && e[1] == t) || (e[0] == t && e[1] == s))
				return i;
		}
		return TileCacheData::UNDEF;
	}

	static int addEdge(int* edges, int& nedges, const int maxEdges, int s, int t, int l, int r)
	{
		if (nedges >= maxEdges)
		{
			return TileCacheData::UNDEF;
		}

		// Add edge if not already in the triangulation. 
		int e = findEdge(edges, nedges, s, t);
		if (e == TileCacheData::UNDEF)
		{
			int* edge = &edges[nedges * 4];
			edge[0] = s;
			edge[1] = t;
			edge[2] = l;
			edge[3] = r;
			return nedges++;
		}
		else
		{
			return TileCacheData::UNDEF;
		}
	}

	static void updateLeftFace(int* e, int s, int t, int f)
	{
		if (e[0] == s && e[1] == t && e[2] == TileCacheData::UNDEF)
			e[2] = f;
		else if (e[1] == s && e[0] == t && e[3] == TileCacheData::UNDEF)
			e[3] = f;
	}

	static int overlapSegSeg2d(const dtReal* a, const dtReal* b, const dtReal* c, const dtReal* d)
	{
		const dtReal a1 = vcross2(a, b, d);
		const dtReal a2 = vcross2(a, b, c);
		if (a1*a2 < 0.0f)
		{
			dtReal a3 = vcross2(c, d, a);
			dtReal a4 = a3 + a2 - a1;
			if (a3 * a4 < 0.0f)
				return 1;
		}
		return 0;
	}

	static bool overlapEdges(const dtReal* pts, const int* edges, int nedges, int s1, int t1)
	{
		for (int i = 0; i < nedges; ++i)
		{
			const int s0 = edges[i * 4 + 0];
			const int t0 = edges[i * 4 + 1];
			// Same or connected edges do not overlap.
			if (s0 == s1 || s0 == t1 || t0 == s1 || t0 == t1)
				continue;
			if (overlapSegSeg2d(&pts[s0 * 3], &pts[t0 * 3], &pts[s1 * 3], &pts[t1 * 3]))
				return true;
		}
		return false;
	}

	static void completeFacet(const dtReal* pts, int npts, int* edges, int& nedges, const int maxEdges, int& nfaces, int e)
	{
		static const dtReal EPS = 1e-5f;

		int* edge = &edges[e * 4];

		// Cache s and t.
		int s, t;
		if (edge[2] == TileCacheData::UNDEF)
		{
			s = edge[0];
			t = edge[1];
		}
		else if (edge[3] == TileCacheData::UNDEF)
		{
			s = edge[1];
			t = edge[0];
		}
		else
		{
			// Edge already completed. 
			return;
		}

		// Find best point on left of edge. 
		int pt = npts;
		dtReal c[3] = { 0, 0, 0 };
		dtReal r = -1;
		for (int u = 0; u < npts; ++u)
		{
			if (u == s || u == t) continue;
			if (vcross2(&pts[s * 3], &pts[t * 3], &pts[u * 3]) > EPS)
			{
				if (r < 0)
				{
					// The circle is not updated yet, do it now.
					pt = u;
					circumCircle(&pts[s * 3], &pts[t * 3], &pts[u * 3], c, r);
					continue;
				}
				const dtReal d = vdist2(c, &pts[u * 3]);
				// UE: increased tolerance of safe checks from 0.001f
				// it was producing (rarely) overlapping edges
				const dtReal tol = 0.005f;
				if (d > r*(1 + tol))
				{
					// Outside current circumcircle, skip.
					continue;
				}
				else if (d < r*(1 - tol))
				{
					// Inside safe circumcircle, update circle.
					pt = u;
					circumCircle(&pts[s * 3], &pts[t * 3], &pts[u * 3], c, r);
				}
				else
				{
					// Inside epsilon circum circle, do extra tests to make sure the edge is valid.
					// s-u and t-u cannot overlap with s-pt nor t-pt if they exists.
					if (overlapEdges(pts, edges, nedges, s, u))
						continue;
					if (overlapEdges(pts, edges, nedges, t, u))
						continue;
					// Edge is valid.
					pt = u;
					circumCircle(&pts[s * 3], &pts[t * 3], &pts[u * 3], c, r);
				}
			}
		}

		// Add new triangle or update edge info if s-t is on hull. 
		if (pt < npts)
		{
			// Update face information of edge being completed. 
			updateLeftFace(&edges[e * 4], s, t, nfaces);

			// Add new edge or update face info of old edge. 
			e = findEdge(edges, nedges, pt, s);
			if (e == TileCacheData::UNDEF)
				addEdge(edges, nedges, maxEdges, pt, s, nfaces, TileCacheData::UNDEF);
			else
				updateLeftFace(&edges[e * 4], pt, s, nfaces);

			// Add new edge or update face info of old edge. 
			e = findEdge(edges, nedges, t, pt);
			if (e == TileCacheData::UNDEF)
				addEdge(edges, nedges, maxEdges, t, pt, nfaces, TileCacheData::UNDEF);
			else
				updateLeftFace(&edges[e * 4], t, pt, nfaces);

			nfaces++;
		}
		else
		{
			updateLeftFace(&edges[e * 4], s, t, TileCacheData::HULL);
		}
	}

	static void delaunayHull(const int npts, const dtReal* pts,
		const int nhull, const int* hull,
		dtIntArray& tris, dtIntArray& edges)
	{
		int nfaces = 0;
		int nedges = 0;
		const int maxEdges = npts * 10;
		edges.resize(maxEdges * 4);

		for (int i = 0, j = nhull - 1; i < nhull; j = i++)
			addEdge(&edges[0], nedges, maxEdges, hull[j], hull[i], TileCacheData::HULL, TileCacheData::UNDEF);

		int currentEdge = 0;
		while (currentEdge < nedges)
		{
			if (edges[currentEdge * 4 + 2] == TileCacheData::UNDEF)
				completeFacet(pts, npts, &edges[0], nedges, maxEdges, nfaces, currentEdge);
			if (edges[currentEdge * 4 + 3] == TileCacheData::UNDEF)
				completeFacet(pts, npts, &edges[0], nedges, maxEdges, nfaces, currentEdge);
			currentEdge++;
		}

		// Create tris
		tris.resize(nfaces * 4);
		for (int i = 0; i < nfaces * 4; ++i)
			tris[i] = -1;

		for (int i = 0; i < nedges; ++i)
		{
			const int* e = &edges[i * 4];
			if (e[3] >= 0)
			{
				// Left face
				int* t = &tris[e[3] * 4];
				if (t[0] == -1)
				{
					t[0] = e[0];
					t[1] = e[1];
				}
				else if (t[0] == e[1])
					t[2] = e[0];
				else if (t[1] == e[0])
					t[2] = e[1];
			}
			if (e[2] >= 0)
			{
				// Right
				int* t = &tris[e[2] * 4];
				if (t[0] == -1)
				{
					t[0] = e[1];
					t[1] = e[0];
				}
				else if (t[0] == e[0])
					t[2] = e[1];
				else if (t[1] == e[1])
					t[2] = e[0];
			}
		}

		for (int i = 0; i < tris.size() / 4; ++i)
		{
			int* t = &tris[i * 4];
			if (t[0] == -1 || t[1] == -1 || t[2] == -1)
			{
				t[0] = tris[tris.size() - 4];
				t[1] = tris[tris.size() - 3];
				t[2] = tris[tris.size() - 2];
				t[3] = tris[tris.size() - 1];
				tris.resize(tris.size() - 4);
				--i;
			}
		}
	}

	static unsigned char getEdgeFlags(const dtReal* va, const dtReal* vb,
		const dtReal* vpoly, const int npoly)
	{
		// Return true if edge (va,vb) is part of the polygon.
		static const dtReal thrSqr = dtSqr(0.001f);
		for (int i = 0, j = npoly - 1; i < npoly; j = i++)
		{
			if (distancePtSeg2d(va, &vpoly[j * 3], &vpoly[i * 3]) < thrSqr &&
				distancePtSeg2d(vb, &vpoly[j * 3], &vpoly[i * 3]) < thrSqr)
				return 1;
		}
		return 0;
	}

	static unsigned char getTriFlags(const dtReal* va, const dtReal* vb, const dtReal* vc,
		const dtReal* vpoly, const int npoly)
	{
		unsigned char flags = 0;
		flags |= getEdgeFlags(va, vb, vpoly, npoly) << 0;
		flags |= getEdgeFlags(vb, vc, vpoly, npoly) << 2;
		flags |= getEdgeFlags(vc, va, vpoly, npoly) << 4;
		return flags;
	}
}

inline dtReal getJitterValueX(const int i)
{
	return (((i * 0x8da6b343) & 0xffff) / dtReal(65535.) * 2.0f) - 1.0f;
}

inline dtReal getJitterValueY(const int i)
{
	return (((i * 0xd8163841) & 0xffff) / dtReal(65535.) * 2.0f) - 1.0f;
}

static bool buildLayerPolyDetail(const dtReal* in, const int nin, const dtReal cs, const dtReal ch,
	const dtReal sampleDist, const dtReal sampleMaxError,
	const dtHeightPatch& hp, dtReal* verts, int& nverts, dtIntArray& tris,
	dtIntArray& edges, dtIntArray& samples)
{
	static const int MAX_VERTS = 127;
	static const int MAX_TRIS = 255;	// Max tris for delaunay is 2n-2-k (n=num verts, k=num hull verts).
	static const int MAX_VERTS_PER_EDGE = 32;
	dtReal edge[(MAX_VERTS_PER_EDGE+1)*3];
	int hull[MAX_VERTS];
	int nhull = 0;

	nverts = 0;

	for (int i = 0; i < nin; ++i)
		dtVcopy(&verts[i*3], &in[i*3]);
	nverts = nin;

	const dtReal ics = 1.0f/cs;

	// Tessellate outlines.
	// This is done in separate pass in order to ensure
	// seamless height values across the ply boundaries.
	if (sampleDist > 0)
	{
		for (int i = 0, j = nin-1; i < nin; j=i++)
		{
			const dtReal* vj = &in[j*3];
			const dtReal* vi = &in[i*3];
			bool swapped = false;
			// Make sure the segments are always handled in same order
			// using lexological sort or else there will be seams.
			if (dtAbs(vj[0]-vi[0]) < 1e-6f)
			{
				if (vj[2] > vi[2])
				{
					dtSwap(vj,vi);
					swapped = true;
				}
			}
			else
			{
				if (vj[0] > vi[0])
				{
					dtSwap(vj,vi);
					swapped = true;
				}
			}
			// Create samples along the edge.
			dtReal dx = vi[0] - vj[0];
			dtReal dy = vi[1] - vj[1];
			dtReal dz = vi[2] - vj[2];
			dtReal d = dtSqrt(dx*dx + dz*dz);
			int nn = 1 + (int)dtFloor(d/sampleDist);
			if (nn >= MAX_VERTS_PER_EDGE) nn = MAX_VERTS_PER_EDGE-1;
			if (nverts+nn >= MAX_VERTS)
				nn = MAX_VERTS-1-nverts;

			for (int k = 0; k <= nn; ++k)
			{
				dtReal u = (dtReal)k/(dtReal)nn;
				dtReal* pos = &edge[k*3];
				pos[0] = vj[0] + dx*u;
				pos[1] = vj[1] + dy*u;
				pos[2] = vj[2] + dz*u;
				pos[1] = getHeight(pos[0],pos[1],pos[2], cs, ics, ch, hp)*ch;
			}
			// Simplify samples.
			int idx[MAX_VERTS_PER_EDGE] = {0,nn};
			int nidx = 2;
			for (int k = 0; k < nidx-1; )
			{
				const int a = idx[k];
				const int b = idx[k+1];
				const dtReal* va = &edge[a*3];
				const dtReal* vb = &edge[b*3];
				// Find maximum deviation along the segment.
				dtReal maxd = 0;
				int maxi = -1;
				for (int m = a+1; m < b; ++m)
				{
					dtReal dev = TileCacheFunc::distancePtSeg(&edge[m * 3], va, vb);
					if (dev > maxd)
					{
						maxd = dev;
						maxi = m;
					}
				}
				// If the max deviation is larger than accepted error,
				// add new point, else continue to next segment.
				if (maxi != -1 && maxd > dtSqr(sampleMaxError))
				{
					for (int m = nidx; m > k; --m)
						idx[m] = idx[m-1];
					idx[k+1] = maxi;
					nidx++;
				}
				else
				{
					++k;
				}
			}

			hull[nhull++] = j;
			// Add new vertices.
			if (swapped)
			{
				for (int k = nidx-2; k > 0; --k)
				{
					dtVcopy(&verts[nverts*3], &edge[idx[k]*3]);
					hull[nhull++] = nverts;
					nverts++;
				}
			}
			else
			{
				for (int k = 1; k < nidx-1; ++k)
				{
					dtVcopy(&verts[nverts*3], &edge[idx[k]*3]);
					hull[nhull++] = nverts;
					nverts++;
				}
			}
		}
	}


	// Tessellate the base mesh.
	edges.resize(0);
	tris.resize(0);

	TileCacheFunc::delaunayHull(nverts, verts, nhull, hull, tris, edges);

	if (tris.size() == 0)
	{
		// Could not triangulate the poly, make sure there is some valid data there.
		for (int i = 2; i < nverts; ++i)
		{
			tris.push(0);
			tris.push(i-1);
			tris.push(i);
			tris.push(0);
		}
		return true;
	}

	if (sampleDist > 0)
	{
		// Create sample locations in a grid.
		dtReal bmin[3], bmax[3];
		dtVcopy(bmin, in);
		dtVcopy(bmax, in);
		for (int i = 1; i < nin; ++i)
		{
			dtVmin(bmin, &in[i*3]);
			dtVmax(bmax, &in[i*3]);
		}
		int x0 = (int)dtFloor(bmin[0]/sampleDist);
		int x1 = (int)dtCeil(bmax[0]/sampleDist);
		int z0 = (int)dtFloor(bmin[2]/sampleDist);
		int z1 = (int)dtCeil(bmax[2]/sampleDist);
		samples.resize(0);
		for (int z = z0; z < z1; ++z)
		{
			for (int x = x0; x < x1; ++x)
			{
				dtReal pt[3];
				pt[0] = x*sampleDist;
				pt[1] = (bmax[1]+bmin[1])*0.5f;
				pt[2] = z*sampleDist;
				// Make sure the samples are not too close to the edges.
				if (TileCacheFunc::distToPoly(nin, in, pt) > -sampleDist / 2) continue;
				samples.push(x);
				samples.push(getHeight(pt[0], pt[1], pt[2], cs, ics, ch, hp));
				samples.push(z);
				samples.push(0); // Not added
			}
		}

		// Add the samples starting from the one that has the most
		// error. The procedure stops when all samples are added
		// or when the max error is within treshold.
		const int nsamples = samples.size()/4;
		for (int iter = 0; iter < nsamples; ++iter)
		{
			if (nverts >= MAX_VERTS)
				break;

			// Find sample with most error.
			dtReal bestpt[3] = {0,0,0};
			dtReal bestd = 0;
			int besti = -1;
			for (int i = 0; i < nsamples; ++i)
			{
				const int* s = &samples[i*4];
				if (s[3]) continue; // skip added.
				dtReal pt[3];
				// The sample location is jittered to get rid of some bad triangulations
				// which are cause by symmetrical data from the grid structure.
				pt[0] = s[0]*sampleDist + getJitterValueX(i)*cs*0.1f;
				pt[1] = s[1]*ch;
				pt[2] = s[2]*sampleDist + getJitterValueY(i)*cs*0.1f;
				dtReal d = TileCacheFunc::distToTriMesh(pt, verts, nverts, &tris[0], tris.size() / 4);
				if (d < 0) continue; // did not hit the mesh.
				if (d > bestd)
				{
					bestd = d;
					besti = i;
					dtVcopy(bestpt,pt);
				}
			}
			// If the max error is within accepted threshold, stop tesselating.
			if (bestd <= sampleMaxError || besti == -1)
				break;
			// Mark sample as added.
			samples[besti*4+3] = 1;
			// Add the new sample point.
			dtVcopy(&verts[nverts*3],bestpt);
			nverts++;

			// Create new triangulation.
			// TODO: Incremental add instead of full rebuild.
			edges.resize(0);
			tris.resize(0);
			TileCacheFunc::delaunayHull(nverts, verts, nhull, hull, tris, edges);
		}		
	}

	const int ntris = tris.size()/4;
	if (ntris > MAX_TRIS)
	{
		tris.resize(MAX_TRIS*4);
	}

	return true;
}

dtStatus dtBuildTileCachePolyMeshDetail(dtTileCacheAlloc* alloc,
	const dtReal cs, const dtReal ch,
	const dtReal sampleDist, const dtReal sampleMaxError,
	dtTileCacheLayer& layer,
	dtTileCachePolyMesh& lmesh,
	dtTileCachePolyMeshDetail& dmesh)
{
	dtAssert(alloc);

	if (lmesh.nverts == 0 || lmesh.npolys == 0)
		return DT_SUCCESS;

	const int nvp = lmesh.nvp;
	const dtReal* orig = layer.header->bmin;

	dtHeightPatch hp;
	dtIntArray edges(64);
	dtIntArray tris(512);
	dtIntArray stack(512);
	dtIntArray samples(512);
	dtReal verts[256*3];
	int nPolyVerts = 0;
	int maxhw = 0, maxhh = 0;

	dtFixedArray<int> bounds(alloc, lmesh.npolys*4);
	if (!bounds)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}
	dtFixedArray<dtReal> poly(alloc, nvp*3);
	if (!poly)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	// Find max size for a polygon area.
	for (int i = 0; i < lmesh.npolys; ++i)
	{
		const unsigned short* p = &lmesh.polys[i*nvp*2];
		int& xmin = bounds[i*4+0];
		int& xmax = bounds[i*4+1];
		int& ymin = bounds[i*4+2];
		int& ymax = bounds[i*4+3];
		xmin = layer.header->width;
		xmax = 0;
		ymin = layer.header->height;
		ymax = 0;
		for (int j = 0; j < nvp; ++j)
		{
			if(p[j] == DT_TILECACHE_NULL_IDX) break;
			const unsigned short* v = &lmesh.verts[p[j]*3];
			xmin = dtMin(xmin, (int)v[0]);
			xmax = dtMax(xmax, (int)v[0]);
			ymin = dtMin(ymin, (int)v[2]);
			ymax = dtMax(ymax, (int)v[2]);
			nPolyVerts++;
		}
		xmin = dtMax(0,xmin-1);
		xmax = dtMin((int)layer.header->width,xmax+1);
		ymin = dtMax(0,ymin-1);
		ymax = dtMin((int)layer.header->height,ymax+1);
		if (xmin >= xmax || ymin >= ymax) continue;
		maxhw = dtMax(maxhw, xmax-xmin);
		maxhh = dtMax(maxhh, ymax-ymin);
	}

	hp.data = (unsigned short*)dtAlloc(sizeof(unsigned short)*maxhw*maxhh, DT_ALLOC_TEMP);
	if (!hp.data)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	dmesh.nmeshes = lmesh.npolys;
	dmesh.nverts = 0;
	dmesh.ntris = 0;
	dmesh.meshes = (unsigned int*)alloc->alloc(sizeof(unsigned int)*dmesh.nmeshes*4);
	if (!dmesh.meshes)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	int vcap = nPolyVerts+nPolyVerts/2;
	int tcap = vcap*2;

	dmesh.nverts = 0;
	dmesh.verts = (dtReal*)alloc->alloc(sizeof(dtReal)*vcap*3);
	if (!dmesh.verts)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}
	dmesh.ntris = 0;
	dmesh.tris = (unsigned char*)alloc->alloc(sizeof(unsigned char*)*tcap*4);
	if (!dmesh.tris)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	for (int i = 0; i < lmesh.npolys; ++i)
	{
		const unsigned short* p = &lmesh.polys[i*nvp*2];

		// Store polygon vertices for processing.
		int npoly = 0;
		for (int j = 0; j < nvp; ++j)
		{
			if(p[j] == DT_TILECACHE_NULL_IDX) break;
			const unsigned short* v = &lmesh.verts[p[j]*3];
			poly[j*3+0] = v[0]*cs;
			poly[j*3+1] = v[1]*ch;
			poly[j*3+2] = v[2]*cs;
			npoly++;
		}

		// Get the height data from the area of the polygon.
		hp.xmin = bounds[i*4+0];
		hp.ymin = bounds[i*4+2];
		hp.width = bounds[i*4+1]-bounds[i*4+0];
		hp.height = bounds[i*4+3]-bounds[i*4+2];
		getLayerHeightData(layer, p, lmesh.verts, npoly, hp, stack);

		// Build detail mesh.
		int nverts = 0;
		if (!buildLayerPolyDetail(poly, npoly, cs, ch,
			sampleDist, sampleMaxError, hp,
			verts, nverts, tris, edges, samples))
		{
			return DT_FAILURE;
		}

		// Move detail verts to world space.
		for (int j = 0; j < nverts; ++j)
		{
			verts[j*3+0] += orig[0];
			verts[j*3+1] += orig[1] + ch; // Is this offset necessary?
			verts[j*3+2] += orig[2];
		}
		// Offset poly too, will be used to flag checking.
		for (int j = 0; j < npoly; ++j)
		{
			poly[j*3+0] += orig[0];
			poly[j*3+1] += orig[1];
			poly[j*3+2] += orig[2];
		}

		// Store detail submesh.
		const int ntris = tris.size()/4;

		dmesh.meshes[i*4+0] = (unsigned int)dmesh.nverts;
		dmesh.meshes[i*4+1] = (unsigned int)nverts;
		dmesh.meshes[i*4+2] = (unsigned int)dmesh.ntris;
		dmesh.meshes[i*4+3] = (unsigned int)ntris;		

		// Store vertices, allocate more memory if necessary.
		if (dmesh.nverts+nverts > vcap)
		{
			while (dmesh.nverts+nverts > vcap)
				vcap += 256;

			dtReal* newv = (dtReal*)alloc->alloc(sizeof(dtReal)*vcap*3);
			if (!newv)
			{
				return DT_FAILURE | DT_OUT_OF_MEMORY;
			}
			if (dmesh.nverts)
			{
				memcpy(newv, dmesh.verts, sizeof(dtReal)*3*dmesh.nverts);
			}
			alloc->free(dmesh.verts);
			dmesh.verts = newv;
		}
		for (int j = 0; j < nverts; ++j)
		{
			dmesh.verts[dmesh.nverts*3+0] = verts[j*3+0];
			dmesh.verts[dmesh.nverts*3+1] = verts[j*3+1];
			dmesh.verts[dmesh.nverts*3+2] = verts[j*3+2];
			dmesh.nverts++;
		}

		// Store triangles, allocate more memory if necessary.
		if (dmesh.ntris+ntris > tcap)
		{
			while (dmesh.ntris+ntris > tcap)
				tcap += 256;

			unsigned char* newt = (unsigned char*)alloc->alloc(sizeof(unsigned char)*tcap*4);
			if (!newt)
			{
				return DT_FAILURE | DT_OUT_OF_MEMORY;
			}
			if (dmesh.ntris)
			{
				memcpy(newt, dmesh.tris, sizeof(unsigned char)*4*dmesh.ntris);
			}
			alloc->free(dmesh.tris);
			dmesh.tris = newt;
		}
		for (int j = 0; j < ntris; ++j)
		{
			const int* t = &tris[j*4];
			dmesh.tris[dmesh.ntris*4+0] = (unsigned char)t[0];
			dmesh.tris[dmesh.ntris*4+1] = (unsigned char)t[1];
			dmesh.tris[dmesh.ntris*4+2] = (unsigned char)t[2];
			dmesh.tris[dmesh.ntris*4+3] = TileCacheFunc::getTriFlags(&verts[t[0]*3], &verts[t[1]*3], &verts[t[2]*3], poly, npoly);
			dmesh.ntris++;
		}
	}

	return DT_SUCCESS;
}
