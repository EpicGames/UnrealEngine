// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "CoreMinimal.h"
#include "Detour/DetourCommon.h"
#include "Detour/DetourAssert.h"
#include "Detour/DetourAlloc.h"
#include "DetourTileCache/DetourTileCacheBuilder.h"
#define _USE_MATH_DEFINES

inline bool isConnected(const dtTileCacheLayer& layer, const int idx, const int dir)
{
	return (layer.cons[idx] & (1 << dir)) != 0;
}

static void calculateDistanceField(const dtTileCacheLayer& layer, unsigned short* src, unsigned short& maxDist)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	
	// Init distance and points.
	memset(src, 0xff, w*h*sizeof(unsigned short));

	// Mark boundary cells.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int i = x+y*w;
			const unsigned char area = layer.areas[i];
			if (area == DT_TILECACHE_NULL_AREA)
			{
				src[i] = 0;
				continue;
			}

			int nc = 0;
			for (int dir = 0; dir < 4; ++dir)
			{
				const int ax = x + getDirOffsetX(dir);
				const int ay = y + getDirOffsetY(dir);
				const int ai = ax+ay*w;
				if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, dir))
				{
					if (area == layer.areas[ai])
					{
						nc++;
					}
				}
			}

			if (nc != 4)
			{
				src[i] = 0;
			}
		}
	}


	// Pass 1
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int i = x+y*w;
			if (layer.areas[i] == DT_TILECACHE_NULL_AREA)
				continue;

			const int ax = x + getDirOffsetX(0);
			const int ay = y + getDirOffsetY(0);
			const int ai = ax+ay*w;
			if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, 0))
			{
				// (-1,0)
				if (src[ai]+2 < src[i])
					src[i] = src[ai]+2;

				const int aax = ax + getDirOffsetX(3);
				const int aay = ay + getDirOffsetY(3);
				const int aai = aax+aay*w;
				if (aax >= 0 && aax < w && aay >= 0 && aay < h && isConnected(layer, ai, 3))
				{
					// (-1,-1)
					if (src[aai]+3 < src[i])
						src[i] = src[aai]+3;
				}
			}

			const int ax2 = x + getDirOffsetX(3);
			const int ay2 = y + getDirOffsetY(3);
			const int ai2 = ax2+ay2*w;
			if (ax2 >= 0 && ax2 < w && ay2 >= 0 && ay2 < h && isConnected(layer, i, 3))
			{
				// (0,-1)
				if (src[ai2]+2 < src[i])
					src[i] = src[ai2]+2;
			
				const int aax2 = ax2 + getDirOffsetX(2);
				const int aay2 = ay2 + getDirOffsetY(2);
				const int aai2 = aax2+aay2*w;
				if (aax2 >= 0 && aax2 < w && aay2 >= 0 && aay2 < h && isConnected(layer, ai2, 2))
				{
					// (1,-1)
					if (src[aai2]+3 < src[i])
						src[i] = src[aai2]+3;
				}
			}
		}
	}

	// Pass 2
	for (int y = h-1; y >= 0; --y)
	{
		for (int x = w-1; x >= 0; --x)
		{
			const int i = x+y*w;
			if (layer.areas[i] == DT_TILECACHE_NULL_AREA)
				continue;

			const int ax = x + getDirOffsetX(2);
			const int ay = y + getDirOffsetY(2);
			const int ai = ax+ay*w;
			if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, 2))
			{
				// (1,0)
				if (src[ai]+2 < src[i])
					src[i] = src[ai]+2;

				const int aax = ax + getDirOffsetX(1);
				const int aay = ay + getDirOffsetY(1);
				const int aai = aax+aay*w;
				if (aax >= 0 && aax < w && aay >= 0 && aay < h && isConnected(layer, ai, 1))
				{
					// (1,1)
					if (src[aai]+3 < src[i])
						src[i] = src[aai]+3;
				}
			}

			const int ax2 = x + getDirOffsetX(1);
			const int ay2 = y + getDirOffsetY(1);
			const int ai2 = ax2+ay2*w;
			if (ax2 >= 0 && ax2 < w && ay2 >= 0 && ay2 < h && isConnected(layer, i, 1))
			{
				// (0,1)
				if (src[ai2]+2 < src[i])
					src[i] = src[ai2]+2;

				const int aax2 = ax2 + getDirOffsetX(0);
				const int aay2 = ay2 + getDirOffsetY(0);
				const int aai2 = aax2+aay2*w;
				if (aax2 >= 0 && aax2 < w && aay2 >= 0 && aay2 < h && isConnected(layer, ai2, 0))
				{
					// (-1,1)
					if (src[aai2]+3 < src[i])
						src[i] = src[aai2]+3;
				}
			}
		}
	}

	// calc max distance
	maxDist = 0;
	for (int i = w*h -1; i >= 0; i--)
	{
		maxDist = dtMax(src[i], maxDist);
	}
}

static unsigned short* boxBlur(dtTileCacheLayer& layer, int thr, unsigned short* src, unsigned short* dst)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	thr *= 2;

	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int i = x+y*w;

			const unsigned short cd = src[i];
			if (cd <= thr)
			{
				dst[i] = cd;
				continue;
			}

			int d = (int)cd;
			for (int dir = 0; dir < 4; ++dir)
			{
				const int ax = x + getDirOffsetX(dir);
				const int ay = y + getDirOffsetY(dir);
				const int ni = ax+ay*w;

				if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, dir))
				{
					d += (int)src[ni];

					const int dir2 = (dir+1) & 0x3;
					const int ax2 = ax + getDirOffsetX(dir2);
					const int ay2 = ay + getDirOffsetY(dir2);
					const int ni2 = ax2+ay2*w;
					if (ax2 >= 0 && ax2 < w && ay2 >= 0 && ay2 < h && isConnected(layer, ni, dir2))
					{
						d += (int)src[ni2];
					}
					else
					{
						d += cd;
					}
				}
				else
				{
					d += cd*2;
				}
			}

			dst[i] = (unsigned short)((d+5)/9);
		}
	}

	return dst;
}

dtStatus dtBuildTileCacheDistanceField(dtTileCacheAlloc* alloc, dtTileCacheLayer& layer, dtTileCacheDistanceField& dfield)
{
	dtAssert(alloc);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	dfield.data = (unsigned short*)alloc->alloc(w*h*sizeof(unsigned short));
	if (!dfield.data)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	dtTileCacheDistanceField tmpField;
	tmpField.data = (unsigned short*)alloc->alloc(w*h*sizeof(unsigned short));
	if (!tmpField.data)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	calculateDistanceField(layer, dfield.data, dfield.maxDist);
	if (boxBlur(layer, 1, dfield.data, tmpField.data) != dfield.data)
	{
		dtSwap(dfield.data, tmpField.data);
	}

	alloc->free(tmpField.data);
	return DT_SUCCESS;
}

static unsigned short* expandRegions(int maxIter, unsigned short level,
	dtTileCacheLayer& layer, dtTileCacheDistanceField& dfield,
	unsigned short* srcReg, unsigned short* srcDist,
	unsigned short* dstReg, unsigned short* dstDist,
	dtIntArray& stack)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	// Find cells revealed by the raised level.
	stack.resize(0);
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const int i = x+y*w;
			if (dfield.data[i] >= level && srcReg[i] == 0 && layer.areas[i] != DT_TILECACHE_NULL_AREA)
			{
				stack.push(x);
				stack.push(y);
				stack.push(i);
			}
		}
	}

	int iter = 0;
	while (stack.size() > 0)
	{
		int failed = 0;

		memcpy(dstReg, srcReg, sizeof(unsigned short)*w*h);
		memcpy(dstDist, srcDist, sizeof(unsigned short)*w*h);

		for (int j = 0; j < stack.size(); j += 3)
		{
			int x = stack[j+0];
			int y = stack[j+1];
			int i = stack[j+2];
			if (i < 0)
			{
				failed++;
				continue;
			}

			unsigned short r = srcReg[i];
			unsigned short d2 = 0xffff;
			const unsigned char area = layer.areas[i];
			for (int dir = 0; dir < 4; ++dir)
			{
				const int ax = x + getDirOffsetX(dir);
				const int ay = y + getDirOffsetY(dir);
				const int ai = ax+ay*w;
				if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, dir))
				{
					if (layer.areas[ai] != area) continue;
					if (srcReg[ai] > 0)
					{
						if ((int)srcDist[ai]+2 < (int)d2)
						{
							r = srcReg[ai];
							d2 = srcDist[ai]+2;
						}
					}
				}
			}
			if (r)
			{
				stack[j+2] = -1; // mark as used
				dstReg[i] = r;
				dstDist[i] = d2;
			}
			else
			{
				failed++;
			}
		}

		// rcSwap source and dest.
		dtSwap(srcReg, dstReg);
		dtSwap(srcDist, dstDist);

		if (failed*3 == stack.size())
			break;

		if (level > 0)
		{
			++iter;
			if (iter >= maxIter)
				break;
		}
	}

	return srcReg;
}

static bool floodRegion(int x, int y, int i, unsigned short level, unsigned short r,
	dtTileCacheLayer& layer, dtTileCacheDistanceField& dfield, unsigned short* srcReg, unsigned short* srcDist, dtIntArray& stack)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	const unsigned char area = layer.areas[i];

	// Flood fill mark region.
	stack.resize(0);
	stack.push((int)x);
	stack.push((int)y);
	stack.push((int)i);
	srcReg[i] = r;
	srcDist[i] = 0;

	unsigned short lev = level >= 2 ? level-2 : 0;
	int count = 0;

	while (stack.size() > 0)
	{
		int ci = stack.pop();
		int cy = stack.pop();
		int cx = stack.pop();

		// Check if any of the neighbours already have a valid region set.
		unsigned short ar = 0;
		for (int dir = 0; dir < 4; ++dir)
		{
			const int ax = cx + getDirOffsetX(dir);
			const int ay = cy + getDirOffsetY(dir);
			const int ai = ax+ay*w;

			// 8 connected
			if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, dir))
			{
				if (layer.areas[ai] != area)
					continue;
				unsigned short nr = srcReg[ai];
				if (nr != 0 && nr != r)
					ar = nr;

				const int dir2 = (dir+1) & 0x3;
				const int ax2 = ax + getDirOffsetX(dir2);
				const int ay2 = ay + getDirOffsetY(dir2);
				const int ai2 = ax2+ay2*w;
				if (ax2 >= 0 && ax2 < w && ay2 >= 0 && ay2 < h && isConnected(layer, ai, dir2))
				{
					if (layer.areas[ai2] != area)
						continue;
					unsigned short nr2 = srcReg[ai2];
					if (nr2 != 0 && nr2 != r)
						ar = nr2;
				}
			}
		}
		if (ar != 0)
		{
			srcReg[ci] = 0;
			continue;
		}
		count++;

		// Expand neighbours.
		for (int dir = 0; dir < 4; ++dir)
		{
			const int ax = cx + getDirOffsetX(dir);
			const int ay = cy + getDirOffsetY(dir);
			const int ai = ax+ay*w;
			if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, dir))
			{
				if (layer.areas[ai] != area)
					continue;
				if (dfield.data[ai] >= lev && srcReg[ai] == 0)
				{
					srcReg[ai] = r;
					srcDist[ai] = 0;
					stack.push(ax);
					stack.push(ay);
					stack.push(ai);
				}
			}
		}
	}

	return count > 0;
}

struct dtLayerRegion
{
	inline dtLayerRegion(unsigned short i) :
		cellCount(0),
		id(i),
		areaType(0),
		remap(false),
		visited(false),
		border(false)
	{}

	dtIntArray connections;
	int cellCount;					// Number of spans belonging to this region
	unsigned short id;				// ID of the region
	unsigned char areaType;			// Are type.
	unsigned char remap : 1;
	unsigned char visited : 1;
	unsigned char border : 1;

	dtLayerRegion& operator=(const dtLayerRegion& src)
	{
		this->cellCount = src.cellCount;
		this->id = src.id;
		this->areaType = src.areaType;
		this->remap = src.remap;
		this->visited = src.visited;
		this->border = src.border;
		this->connections.copy(src.connections);
		return *this;
	}
};

static void removeAdjacentNeighbours(dtLayerRegion& reg)
{
	// Remove adjacent duplicates.
	for (int i = 0; i < reg.connections.size() && reg.connections.size() > 1; )
	{
		int ni = (i+1) % reg.connections.size();
		if (reg.connections[i] == reg.connections[ni])
		{
			// Remove duplicate
			for (int j = i; j < reg.connections.size()-1; ++j)
				reg.connections[j] = reg.connections[j+1];
			reg.connections.pop();
		}
		else
			++i;
	}
}

static void replaceNeighbour(dtLayerRegion& reg, unsigned short oldId, unsigned short newId)
{
	bool neiChanged = false;
	for (int i = 0; i < reg.connections.size(); ++i)
	{
		if (reg.connections[i] == oldId)
		{
			reg.connections[i] = newId;
			neiChanged = true;
		}
	}
	if (neiChanged)
		removeAdjacentNeighbours(reg);
}

static bool canMergeWithRegion(const dtLayerRegion& rega, const dtLayerRegion& regb)
{
	if (rega.areaType != regb.areaType)
		return false;
	int n = 0;
	for (int i = 0; i < rega.connections.size(); ++i)
	{
		if (rega.connections[i] == regb.id)
			n++;
	}
	if (n > 1)
		return false;
	return true;
}

static bool mergeRegions(dtLayerRegion& rega, dtLayerRegion& regb)
{
	unsigned short aid = rega.id;
	unsigned short bid = regb.id;

	// Duplicate current neighbourhood.
	dtIntArray acon;
	acon.resize(rega.connections.size());
	for (int i = 0; i < rega.connections.size(); ++i)
		acon[i] = rega.connections[i];
	dtIntArray& bcon = regb.connections;

	// Find insertion point on A.
	int insa = -1;
	for (int i = 0; i < acon.size(); ++i)
	{
		if (acon[i] == bid)
		{
			insa = i;
			break;
		}
	}
	if (insa == -1)
		return false;

	// Find insertion point on B.
	int insb = -1;
	for (int i = 0; i < bcon.size(); ++i)
	{
		if (bcon[i] == aid)
		{
			insb = i;
			break;
		}
	}
	if (insb == -1)
		return false;

	// Merge neighbours.
	rega.connections.resize(0);
	for (int i = 0, ni = acon.size(); i < ni-1; ++i)
		rega.connections.push(acon[(insa+1+i) % ni]);

	for (int i = 0, ni = bcon.size(); i < ni-1; ++i)
		rega.connections.push(bcon[(insb+1+i) % ni]);

	removeAdjacentNeighbours(rega);

	rega.cellCount += regb.cellCount;
	regb.cellCount = 0;

	rega.border |= regb.border;
	regb.border = 0;

	regb.connections.resize(0);

	return true;
}

static bool isSolidEdge(dtTileCacheLayer& layer, unsigned short* srcReg,
	int x, int y, int i, int dir)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const int ax = x + getDirOffsetX(dir);
	const int ay = y + getDirOffsetY(dir);
	const int ai = ax+ay*w;
	unsigned short r = 0;
	if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, dir))
	{
		r = srcReg[ai];
	}

	return (r != srcReg[i]);
}

static void walkContour(int x, int y, int i, int dir,
	dtTileCacheLayer& layer, unsigned short* srcReg, dtIntArray& cont)
{
	int startDir = dir;
	int starti = i;

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	unsigned short curReg = 0;

	{
		const int ax = x + getDirOffsetX(dir);
		const int ay = y + getDirOffsetY(dir);
		const int ai = ax+ay*w;
		if (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, dir))
		{
			curReg = srcReg[ai];
		}
	}

	cont.push(curReg);

	int iter = 0;
	while (++iter < 40000)
	{
		const int ax = x + getDirOffsetX(dir);
		const int ay = y + getDirOffsetY(dir);
		const int ai = ax+ay*w;
		const bool bConnected = (ax >= 0 && ax < w && ay >= 0 && ay < h && isConnected(layer, i, dir));
		unsigned short r = 0;
		if (bConnected)
		{
			r = srcReg[ai];
		}

		if (r != srcReg[i])
		{
			// Choose the edge corner
			if (r != curReg)
			{
				curReg = r;
				cont.push(curReg);
			}

			dir = (dir+1) & 0x3;  // Rotate CW
		}
		else
		{
			int ni = -1;
			if (bConnected)
			{
				ni = ai;
			}

			if (ni == -1)
			{
				// Should not happen.
				return;
			}
			x = ax;
			y = ay;
			i = ni;
			dir = (dir+3) & 0x3;	// Rotate CCW
		}

		if (starti == i && startDir == dir)
		{
			break;
		}
	}

	// Remove adjacent duplicates.
	if (cont.size() > 1)
	{
		for (int j = 0; j < cont.size(); )
		{
			int nj = (j+1) % cont.size();
			if (cont[j] == cont[nj])
			{
				for (int k = j; k < cont.size()-1; ++k)
					cont[k] = cont[k+1];
				cont.pop();
			}
			else
				++j;
		}
	}
}

static dtStatus filterSmallRegions(dtTileCacheAlloc* alloc, dtTileCacheLayer& layer, int minRegionArea, int mergeRegionSize,
	unsigned short& maxRegionId, unsigned short* srcReg)
{
	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	const int nreg = maxRegionId+1;
	dtFixedArray<dtLayerRegion> regions(alloc, nreg);
	if (!regions)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	// Construct regions
	regions.set(0);
	for (int i = 0; i < nreg; ++i)
		regions[i] = dtLayerRegion((unsigned short)i);

	// Find edge of a region and find connections around the contour.
	for (int y = 0; y < h; ++y)
	{
		const bool borderY = (y == 0) || (y == (h - 1));
		for (int x = 0; x < w; ++x)
		{
			const int i = x+y*w;
			unsigned short r = srcReg[i];

			if (r == DT_TILECACHE_NULL_AREA || r >= nreg)
				continue;

			dtLayerRegion& reg = regions[r];
			reg.cellCount++;
			reg.border |= borderY || (x == 0) || (x == (w - 1));

			// Have found contour
			if (reg.connections.size() > 0)
				continue;

			reg.areaType = layer.areas[i];

			// Check if this cell is next to a border.
			int ndir = -1;
			for (int dir = 0; dir < 4; ++dir)
			{
				if (isSolidEdge(layer, srcReg, x, y, i, dir))
				{
					ndir = dir;
					break;
				}
			}

			if (ndir != -1)
			{
				// The cell is at border.
				// Walk around the contour to find all the neighbours.
				walkContour(x, y, i, ndir, layer, srcReg, reg.connections);
			}
		}
	}

	// Remove too small regions.
	dtIntArray stack(32);
	dtIntArray trace(32);
	for (int i = 0; i < nreg; ++i)
	{
		dtLayerRegion& reg = regions[i];
		if (reg.id == 0)
			continue;                       
		if (reg.cellCount == 0)
			continue;
		if (reg.visited)
			continue;

		// Count the total size of all the connected regions.
		// Also keep track of the regions connects to a tile border.
		bool connectsToBorder = false;
		int cellCount = 0;
		stack.resize(0);
		trace.resize(0);

		reg.visited = true;
		stack.push(i);

		while (stack.size())
		{
			// Pop
			int ri = stack.pop();

			dtLayerRegion& creg = regions[ri];

			connectsToBorder |= creg.border;
			cellCount += creg.cellCount;
			trace.push(ri);

			for (int j = 0; j < creg.connections.size(); ++j)
			{
				dtLayerRegion& neireg = regions[creg.connections[j]];
				if (neireg.visited)
					continue;
				if (neireg.id == 0)
					continue;
				// Visit
				stack.push(neireg.id);
				neireg.visited = true;
			}
		}

		// If the accumulated regions size is too small, remove it.
		// Do not remove areas which connect to tile borders
		// as their size cannot be estimated correctly and removing them
		// can potentially remove necessary areas.
		if (cellCount < minRegionArea && !connectsToBorder)
		{
			// Kill all visited regions.
			for (int j = 0; j < trace.size(); ++j)
			{
				regions[trace[j]].cellCount = 0;
				regions[trace[j]].id = 0;
			}
		}
	}

	// Merge too small regions to neighbour regions.
	int mergeCount = 0 ;
	do
	{
		mergeCount = 0;
		for (int i = 0; i < nreg; ++i)
		{
			dtLayerRegion& reg = regions[i];
			if (reg.id == 0)
				continue;                       
			if (reg.cellCount == 0)
				continue;

			// Check to see if the region should be merged.
			if (reg.cellCount > mergeRegionSize && reg.border)
				continue;

			// Small region with more than 1 connection.
			// Or region which is not connected to a border at all.
			// Find smallest neighbour region that connects to this one.
			int smallest = 0xfffffff;
			unsigned short mergeId = reg.id;
			for (int j = 0; j < reg.connections.size(); ++j)
			{
				dtLayerRegion& mreg = regions[reg.connections[j]];
				if (mreg.id == 0) continue;
				if (mreg.cellCount < smallest &&
					canMergeWithRegion(reg, mreg) &&
					canMergeWithRegion(mreg, reg))
				{
					smallest = mreg.cellCount;
					mergeId = mreg.id;
				}
			}
			// Found new id.
			if (mergeId != reg.id)
			{
				unsigned short oldId = reg.id;
				dtLayerRegion& target = regions[mergeId];

				// Merge neighbours.
				if (mergeRegions(target, reg))
				{
					// Fixup regions pointing to current region.
					for (int j = 0; j < nreg; ++j)
					{
						if (regions[j].id == 0) continue;
						// If another region was already merged into current region
						// change the nid of the previous region too.
						if (regions[j].id == oldId)
							regions[j].id = mergeId;
						// Replace the current region with the new one if the
						// current regions is neighbour.
						replaceNeighbour(regions[j], oldId, mergeId);
					}
					mergeCount++;
				}
			}
		}
	}
	while (mergeCount > 0);

	// Compress region Ids.
	for (int i = 0; i < nreg; ++i)
	{
		regions[i].remap = false;
		if (regions[i].id == DT_TILECACHE_NULL_AREA) continue;       // Skip nil regions.
		regions[i].remap = true;
	}

	unsigned short regIdGen = 0;
	for (int i = 0; i < nreg; ++i)
	{
		if (!regions[i].remap)
			continue;
		unsigned short oldId = regions[i].id;
		unsigned short newId = ++regIdGen;
		for (int j = i; j < nreg; ++j)
		{
			if (regions[j].id == oldId)
			{
				regions[j].id = newId;
				regions[j].remap = false;
			}
		}
	}
	maxRegionId = regIdGen;

	// Remap regions.
	for (int i = w*h-1; i >= 0; i--)
	{
		srcReg[i] = regions[srcReg[i]].id;
	}

	for (int i = 0; i < nreg; ++i)
		regions[i].~dtLayerRegion();

	return DT_SUCCESS;
}

dtStatus dtBuildTileCacheRegions(dtTileCacheAlloc* alloc,
	const int minRegionArea, const int mergeRegionArea,
	dtTileCacheLayer& layer, dtTileCacheDistanceField dfield)
{
	dtAssert(alloc);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;
	const int size = w*h;

	dtFixedArray<unsigned short> buf(alloc, size*4);
	if (!buf)
	{
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	}

	dtIntArray stack(1024);
	dtIntArray visited(1024);

	unsigned short* srcReg = buf;
	unsigned short* srcDist = buf+size;
	unsigned short* dstReg = buf+size*2;
	unsigned short* dstDist = buf+size*3;

	memset(srcReg, 0, sizeof(unsigned short)*size);
	memset(srcDist, 0, sizeof(unsigned short)*size);

	unsigned short regionId = 1;
	unsigned short level = (dfield.maxDist+1) & ~1;

	// TODO: Figure better formula, expandIters defines how much the 
	// watershed "overflows" and simplifies the regions. Tying it to
	// agent radius was usually good indication how greedy it could be.
	//	const int expandIters = 4 + walkableRadius * 2;
	const int expandIters = 8;

	while (level > 0)
	{
		level = level >= 2 ? level-2 : 0;

		// Expand current regions until no empty connected cells found.
		if (expandRegions(expandIters, level, layer, dfield, srcReg, srcDist, dstReg, dstDist, stack) != srcReg)
		{
			dtSwap(srcReg, dstReg);
			dtSwap(srcDist, dstDist);
		}

		// Mark new regions with IDs.
		for (int y = 0; y < h; ++y)
		{
			for (int x = 0; x < w; ++x)
			{
				const int i=x+y*w;
				if (dfield.data[i] < level || srcReg[i] != 0 || layer.areas[i] == DT_TILECACHE_NULL_AREA)
					continue;
				if (floodRegion(x, y, i, level, regionId, layer, dfield, srcReg, srcDist, stack))
					regionId++;
			}
		}
	}

	// Expand current regions until no empty connected cells found.
	if (expandRegions(expandIters*8, 0, layer, dfield, srcReg, srcDist, dstReg, dstDist, stack) != srcReg)
	{
		dtSwap(srcReg, dstReg);
		dtSwap(srcDist, dstDist);
	}

	dtStatus status = filterSmallRegions(alloc, layer, minRegionArea, mergeRegionArea, regionId, srcReg);
	if (dtStatusFailed(status))
	{
		return status;
	}

	// Write the result out.
	memcpy(layer.regs, srcReg, sizeof(unsigned short)*size);
	layer.regCount = regionId;

	return DT_SUCCESS;
}

struct dtLayerSweepSpan
{
	unsigned short ns;	// number samples
	unsigned short id;	// region id
	unsigned short nei;	// neighbour id
};

struct dtLayerMonotoneRegion
{
	dtIntArray neis;
	int area;
	int chunkId;
	unsigned short regId;
	unsigned char areaId;
	unsigned char remap : 1;
	unsigned char border : 1;
	unsigned char visited : 1;
};

static void addUniqueLast(dtIntArray& a, unsigned short v)
{
	if (!a.contains(v))
	{
		a.push(v);
	}
}

static bool canMerge(unsigned short oldRegId, unsigned short newRegId, const dtLayerMonotoneRegion* regs, const int nregs)
{
	int count = 0;
	for (int i = 0; i < nregs; ++i)
	{
		const dtLayerMonotoneRegion& reg = regs[i];
		if (reg.regId != oldRegId) continue;
		const int nnei = reg.neis.size();
		for (int j = 0; j < nnei; ++j)
		{
			if (regs[reg.neis[j]].regId == newRegId)
				count++;
		}
	}
	return count == 1;
}

static dtStatus CollectRegionsMonotone(dtTileCacheAlloc* alloc, dtTileCacheLayer& layer, dtLayerMonotoneRegion*& regs, int& nregs)
{
	dtAssert(alloc);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	memset(layer.regs,0xff,sizeof(unsigned short)*w*h);

	const int nsweeps = w;
	dtFixedArray<dtLayerSweepSpan> sweeps(alloc, nsweeps);
	if (!sweeps)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(sweeps,0,sizeof(dtLayerSweepSpan)*nsweeps);

	// Partition walkable area into monotone regions.
	dtIntArray prevCount(256);
	unsigned short regId = 0;

	for (int y = 0; y < h; ++y)
	{
		prevCount.resize(regId+1);
		memset(&prevCount[0],0,sizeof(int)*regId);
		unsigned short sweepId = 0;

		for (int x = 0; x < w; ++x)
		{
			const int idx = x + y*w;
			if (layer.areas[idx] == DT_TILECACHE_NULL_AREA) continue;

			unsigned short sid = 0xffff;

			// -x
			if (x > 0 && isConnected(layer, idx, 0))
			{
				const int xidx = (x-1)+y*w;
				if (layer.regs[xidx] != 0xffff && layer.areas[xidx] == layer.areas[idx])
					sid = layer.regs[xidx];
			}

			if (sid == 0xffff)
			{
				sid = sweepId++;
				sweeps[sid].nei = 0xffff;
				sweeps[sid].ns = 0;
			}

			// -y
			if (y > 0 && isConnected(layer, idx, 3))
			{
				const int yidx = x+(y-1)*w;
				const unsigned short nr = layer.regs[yidx];
				if (nr != 0xffff && layer.areas[yidx] == layer.areas[idx])
				{
					// Set neighbour when first valid neighbour is encoutered.
					if (sweeps[sid].ns == 0)
						sweeps[sid].nei = nr;

					if (sweeps[sid].nei == nr)
					{
						// Update existing neighbour
						sweeps[sid].ns++;
						prevCount[nr]++;
					}
					else
					{
						// This is hit if there is nore than one neighbour.
						// Invalidate the neighbour.
						sweeps[sid].nei = 0xffff;
					}
				}
			}

			layer.regs[idx] = sid;
		}

		// Create unique ID.
		for (int i = 0; i < sweepId; ++i)
		{
			// If the neighbour is set and there is only one continuous connection to it,
			// the sweep will be merged with the previous one, else new region is created.
			if (sweeps[i].nei != 0xffff && prevCount[sweeps[i].nei] == sweeps[i].ns)
			{
				sweeps[i].id = sweeps[i].nei;
			}
			else
			{
				sweeps[i].id = regId++;
			}
		}

		// Remap local sweep ids to region ids.
		for (int x = 0; x < w; ++x)
		{
			const int idx = x+y*w;
			if (layer.regs[idx] != 0xffff)
			{
				layer.regs[idx] = sweeps[layer.regs[idx]].id;
			}
		}
	}

	// Allocate and init layer regions.
	nregs = (int)regId;

// @UE BEGIN: special handling of "no regions"
	if (nregs == 0)
	{
		regs = 0;
		// treating this as success because we successfully generated 0 regions,
		// no issues occurred, everything was good. Just no regions. 
		return DT_SUCCESS;
	}
// @UE END

	regs = (dtLayerMonotoneRegion*)alloc->alloc(sizeof(dtLayerMonotoneRegion) * nregs);
	if (!regs)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	memset((void*)regs, 0, sizeof(dtLayerMonotoneRegion)*nregs);
	for (int i = 0; i < nregs; ++i)
	{
		regs[i].regId = 0xffff;
		regs[i].neis.resize(16);
		regs[i].neis.resize(0);
	}

	// Find region neighbours.
	for (int y = 0; y < h; ++y)
	{
		const bool borderY = (y == 0) || (y == (h - 1));
		for (int x = 0; x < w; ++x)
		{
			const int idx = x+y*w;
			const unsigned short ri = layer.regs[idx];
			if (ri == 0xffff)
				continue;

			// Update area.
			regs[ri].area++;
			regs[ri].areaId = layer.areas[idx];
			regs[ri].border |= borderY || (x == 0) || (x == (w - 1));

			// Update neighbours
			if (y > 0 && isConnected(layer, idx, 3))
			{
				const int ymi = x+(y-1)*w;
				const unsigned short rai = layer.regs[ymi];
				if (rai != 0xffff && rai != ri)
				{
					addUniqueLast(regs[ri].neis, rai);
					addUniqueLast(regs[rai].neis, ri);
				}
			}
		}
	}

	return DT_SUCCESS;
}

static dtStatus CollectRegionsChunky(dtTileCacheAlloc* alloc, dtTileCacheLayer& layer, int chunkSize, dtLayerMonotoneRegion*& regs, int& nregs)
{
	dtAssert(alloc);

	const int w = (int)layer.header->width;
	const int h = (int)layer.header->height;

	memset(layer.regs,0xff,sizeof(unsigned short)*w*h);

	const int nsweeps = w;
	dtFixedArray<dtLayerSweepSpan> sweeps(alloc, nsweeps);
	if (!sweeps)
		return DT_FAILURE | DT_OUT_OF_MEMORY;
	memset(sweeps,0,sizeof(dtLayerSweepSpan)*nsweeps);

	// Partition walkable area into monotone regions.
	dtIntArray prevCount(256);
	unsigned short regId = 0;

	const int numXChunks = dtMax(1, w / chunkSize); // Division floors, remainder is put to the last chunk.
	const int numYChunks = dtMax(1, h / chunkSize);

	for (int chunkx = 0; chunkx < numXChunks; chunkx++)
	{
		const bool lastXChunk = chunkx == (numXChunks - 1);
		const int minx = chunkx * chunkSize;
		const int maxx = lastXChunk ? w : (minx + chunkSize);

		for (int chunky = 0; chunky < numYChunks; chunky++)
		{
			const bool lastYChunk = chunky == (numYChunks - 1);
			const int miny = chunky * chunkSize;
			const int maxy = lastYChunk ? h : (miny + chunkSize);

			for (int y = miny; y < maxy; ++y)
			{
				prevCount.resize(regId+1);
				memset(&prevCount[0],0,sizeof(int)*regId);
				unsigned short sweepId = 0;

				for (int x = minx; x < maxx; ++x)
				{
					const int idx = x + y*w;
					if (layer.areas[idx] == DT_TILECACHE_NULL_AREA)
					{
						continue;
					}

					unsigned short sid = 0xffff;

					// -x
					if (x > minx && isConnected(layer, idx, 0))
					{
						const int xidx = (x-1)+y*w;
						if (layer.regs[xidx] != 0xffff && layer.areas[xidx] == layer.areas[idx])
							sid = layer.regs[xidx];
					}

					if (sid == 0xffff)
					{
						sid = sweepId++;
						sweeps[sid].nei = 0xffff;
						sweeps[sid].ns = 0;
					}

					// -y
					if (y > miny && isConnected(layer, idx, 3))
					{
						const int yidx = x+(y-1)*w;
						const unsigned short nr = layer.regs[yidx];
						if (nr != 0xffff && layer.areas[yidx] == layer.areas[idx])
						{
							// Set neighbour when first valid neighbour is encoutered.
							if (sweeps[sid].ns == 0)
								sweeps[sid].nei = nr;

							if (sweeps[sid].nei == nr)
							{
								// Update existing neighbour
								sweeps[sid].ns++;
								prevCount[nr]++;
							}
							else
							{
								// This is hit if there is nore than one neighbour.
								// Invalidate the neighbour.
								sweeps[sid].nei = 0xffff;
							}
						}
					}

					layer.regs[idx] = sid;
				}

				// Create unique ID.
				for (int i = 0; i < sweepId; ++i)
				{
					// If the neighbour is set and there is only one continuous connection to it,
					// the sweep will be merged with the previous one, else new region is created.
					if (sweeps[i].nei != 0xffff && prevCount[sweeps[i].nei] == sweeps[i].ns)
					{
						sweeps[i].id = sweeps[i].nei;
					}
					else
					{
						sweeps[i].id = regId++;
					}
				}

				// Remap local sweep ids to region ids.
				for (int x = minx; x < maxx; ++x)
				{
					const int idx = x+y*w;
					if (layer.regs[idx] != 0xffff)
					{
						unsigned short id = sweeps[layer.regs[idx]].id;

						layer.regs[idx] = id;
					}
				}
			}
		}
	}

	// Allocate and init layer regions.
	nregs = (int)regId;

// @UE BEGIN: special handling of "no regions"
	if (nregs == 0)
	{
		regs = 0;
		// treating this as success because we successfully generated 0 regions,
		// no issues occurred, everything was good. Just no regions. 
		return DT_SUCCESS;
	}
// @UE END

	regs = (dtLayerMonotoneRegion*)alloc->alloc(sizeof(dtLayerMonotoneRegion) * nregs);
	if (!regs)
		return DT_FAILURE | DT_OUT_OF_MEMORY;

	memset((void*)regs, 0, sizeof(dtLayerMonotoneRegion)*nregs);
	for (int i = 0; i < nregs; ++i)
	{
		regs[i].regId = 0xffff;
		regs[i].neis.resize(16);
		regs[i].neis.resize(0);
	}

	// Find region neighbours.
	for (int y = 0; y < h; ++y)
	{
		const int ciy = dtMin(y / chunkSize , numYChunks-1);

		const bool borderY = (y == 0) || (y == (h - 1));
		for (int x = 0; x < w; ++x)
		{
			const int idx = x+y*w;
			const unsigned short ri = layer.regs[idx];
			if (ri == 0xffff)
				continue;

			// Update area.
			regs[ri].area++;
			regs[ri].areaId = layer.areas[idx];

			const int cix = dtMin(x / chunkSize , numXChunks-1);
			regs[ri].chunkId = cix + (ciy * numXChunks);
			regs[ri].border |= borderY || (x == 0) || (x == (w - 1));

			// Update neighbours
			if (y > 0 && isConnected(layer, idx, 3))
			{
				const int ymi = x+(y-1)*w;
				const unsigned short rai = layer.regs[ymi];
				if (rai != 0xffff && rai != ri)
				{
					addUniqueLast(regs[ri].neis, rai);
					addUniqueLast(regs[rai].neis, ri);
				}
			}
		}
	}

	return DT_SUCCESS;
}

static void MergeAndCompressRegions(dtTileCacheAlloc* alloc, dtTileCacheLayer& layer, dtLayerMonotoneRegion* regs, int nregs, const int minRegionArea, const int mergeRegionArea)
{
	for (int i = 0; i < nregs; ++i)
		regs[i].regId = (unsigned short)(i + 1);

	// Remove too small regions.
	if (minRegionArea > 0)
	{
		dtIntArray stack(32);
		dtIntArray trace(32);
		for (int i = 0; i < nregs; ++i)
		{
			dtLayerMonotoneRegion& reg = regs[i];
			if (reg.visited || reg.area == 0)
				continue;

			// Count the total size of all the connected regions.
			// Also keep track of the regions connects to a tile border.
			bool connectsToBorder = false;
			int cellCount = 0;
			stack.resize(0);
			trace.resize(0);

			reg.visited = true;
			stack.push(i);

			while (stack.size())
			{
				// Pop
				int ri = stack.pop();

				dtLayerMonotoneRegion& creg = regs[ri];

				connectsToBorder |= creg.border;
				cellCount += creg.area;
				trace.push(ri);

				for (int j = 0; j < creg.neis.size(); ++j)
				{
					dtLayerMonotoneRegion& neireg = regs[creg.neis[j]];
					if (neireg.visited)
						continue;
					if (neireg.regId == 0)
						continue;
					// Visit
					stack.push(neireg.regId - 1);
					neireg.visited = true;
				}
			}

			// If the accumulated regions size is too small, remove it.
			// Do not remove areas which connect to tile borders
			// as their size cannot be estimated correctly and removing them
			// can potentially remove necessary areas.
			if (cellCount < minRegionArea && !connectsToBorder)
			{
				// Kill all visited regions.
				for (int j = 0; j < trace.size(); ++j)
				{
					regs[trace[j]].area = 0;
					regs[trace[j]].regId = 0;
				}
			}
		}
	}

	for (int i = 0; i < nregs; ++i)
	{
		dtLayerMonotoneRegion& reg = regs[i];
		if (reg.regId == 0)
			continue;
		// don't use mergeRegionArea, it doesn't work well with monotone partitioning
		// (results in even more long thin polys)

		int merge = -1;
		int mergea = 0;
		for (int j = 0; j < reg.neis.size(); ++j)
		{
			const unsigned short nei = (unsigned short)reg.neis[j];
			dtLayerMonotoneRegion& regn = regs[nei];
			if (reg.regId == regn.regId)
				continue;
			if (reg.areaId != regn.areaId || reg.chunkId != regn.chunkId)
				continue;
			if (regn.area > mergea)
			{
				if (canMerge(reg.regId, regn.regId, regs, nregs))
				{
					mergea = regn.area;
					merge = (int)nei;
				}
			}
		}
		if (merge != -1)
		{
			const unsigned short oldId = reg.regId;
			const unsigned short newId = regs[merge].regId;
			for (int j = 0; j < nregs; ++j)
				if (regs[j].regId == oldId)
					regs[j].regId = newId;
		}
	}

	unsigned short regId = 0;
	if (nregs < 256)
	{
		// Compact ids.
		unsigned short remap[256];
		memset(remap, 0, sizeof(unsigned short)*256);
		// Find number of unique regions.
		for (int i = 0; i < nregs; ++i)
			remap[regs[i].regId] = 1;
		// skip region id 0, it's used for skipping minRegionArea
		remap[0] = 0;
		for (int i = 1; i < 256; ++i)
			if (remap[i])
				remap[i] = ++regId;
		// Remap ids.
		for (int i = 0; i < nregs; ++i)
			regs[i].regId = remap[regs[i].regId];
	}
	else
	{
		for (int i = 0; i < nregs; ++i)
			regs[i].remap = true;

		for (int i = 0; i < nregs; ++i)
		{
			// skip region id 0, it's used for skipping minRegionArea
			if (!regs[i].remap || regs[i].regId == 0)
				continue;
			unsigned short oldId = regs[i].regId;
			unsigned short newId = ++regId;
			for (int j = i; j < nregs; ++j)
			{
				if (regs[j].regId == oldId)
				{
					regs[j].regId = newId;
					regs[j].remap = false;
				}
			}
		}
	}

	layer.regCount = regId;

	const int maxi = (int)layer.header->width * (int)layer.header->height;
	for (int i = 0; i < maxi; ++i)
	{
		if (layer.regs[i] != 0xffff)
			layer.regs[i] = regs[layer.regs[i]].regId;
	}
}

static void FreeRegions(dtTileCacheAlloc* alloc, dtLayerMonotoneRegion* regs, int nregs)
{
	// destroy all elements to free internal rcIntArray allocations
	for (int i = 0; i < nregs; i++)
	{
		regs[i].~dtLayerMonotoneRegion();
	}

	alloc->free(regs);
}

dtStatus dtBuildTileCacheRegionsMonotone(dtTileCacheAlloc* alloc, const int minRegionArea, const int mergeRegionArea, dtTileCacheLayer& layer)
{
	dtLayerMonotoneRegion* regs = NULL;
	int nregs = 0;

	dtStatus status = CollectRegionsMonotone(alloc, layer, regs, nregs);
	// having no regions and status being successful is a valid state
	// we can avoid calling MergeAndCompressRegions if that happens
	if (dtStatusSucceed(status) && nregs > 0)
	{
		MergeAndCompressRegions(alloc, layer, regs, nregs, minRegionArea, mergeRegionArea);
	}

	FreeRegions(alloc, regs, nregs);
	return status;
}

dtStatus dtBuildTileCacheRegionsChunky(dtTileCacheAlloc* alloc, const int minRegionArea, const int mergeRegionArea, dtTileCacheLayer& layer, int regionChunkSize)
{
	dtLayerMonotoneRegion* regs = NULL;
	int nregs = 0;

	dtStatus status = CollectRegionsChunky(alloc, layer, regionChunkSize, regs, nregs);
	// having no regions and status being successful is a valid state
	// we can avoid calling MergeAndCompressRegions if that happens
	if (dtStatusSucceed(status) && nregs > 0)
	{
		MergeAndCompressRegions(alloc, layer, regs, nregs, minRegionArea, mergeRegionArea);
	}

	FreeRegions(alloc, regs, nregs);
	return status;
}
