// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "Recast/Recast.h"
#define _USE_MATH_DEFINES
#include "Recast/RecastAlloc.h"
#include "Recast/RecastAssert.h"

DEFINE_LOG_CATEGORY(LogRecast);

float rcSqrt(float x)
{
	return sqrtf(x);
}

//@UE BEGIN Adding support for LWCoords.
double rcSqrt(double x)
{
	return sqrt(x);
}
//@UE END

/// @class rcContext
/// @par
///
/// This class does not provide logging or timer functionality on its 
/// own.  Both must be provided by a concrete implementation 
/// by overriding the protected member functions.  Also, this class does not 
/// provide an interface for extracting log messages. (Only adding them.) 
/// So concrete implementations must provide one.
///
/// If no logging or timers are required, just pass an instance of this 
/// class through the Recast build process.
///

/// @par
///
/// Example:
/// @code
/// // Where ctx is an instance of rcContext and filepath is a char array.
/// ctx->log(RC_LOG_ERROR, "buildTiledNavigation: Could not load '%s'", filepath);
/// @endcode
void rcContext::log(const rcLogCategory category, const char* format, ...)
{
	if (!m_logEnabled)
		return;
	static const int MSG_SIZE = 512;
	char msg[MSG_SIZE];
	va_list ap;
	va_start(ap, format);
	int len = FCStringAnsi::GetVarArgs(msg, MSG_SIZE, format, ap);
	if (len >= MSG_SIZE)
	{
		len = MSG_SIZE-1;
		msg[MSG_SIZE-1] = '\0';
	}
	va_end(ap);
	doLog(category, msg, len);
}

rcHeightfield* rcAllocHeightfield()
{
	rcHeightfield* hf = (rcHeightfield*)rcAlloc(sizeof(rcHeightfield), RC_ALLOC_PERM);
	memset(hf, 0, sizeof(rcHeightfield));
	return hf;
}

void rcFreeHeightField(rcHeightfield* hf)
{
	if (!hf) return;
	// Delete span array.
	rcFree(hf->spans);
	// Delete span pools.
	while (hf->pools)
	{
		rcSpanPool* next = hf->pools->next;
		rcFree(hf->pools);
		hf->pools = next;
	}
#if EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
	rcFree(hf->EdgeHits);
	rcFree(hf->RowExt);
	rcFree(hf->tempspans);
	rcFree(hf->tempSpanColumns);
#endif // EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
	rcFree(hf);
}

rcCompactHeightfield* rcAllocCompactHeightfield()
{
	rcCompactHeightfield* chf = (rcCompactHeightfield*)rcAlloc(sizeof(rcCompactHeightfield), RC_ALLOC_PERM);
	memset(chf, 0, sizeof(rcCompactHeightfield));
	return chf;
}

void rcFreeCompactHeightfield(rcCompactHeightfield* chf)
{
	if (!chf) return;
	rcFree(chf->cells);
	rcFree(chf->spans);
	rcFree(chf->dist);
	rcFree(chf->areas);
	rcFree(chf);
}


rcHeightfieldLayerSet* rcAllocHeightfieldLayerSet()
{
	rcHeightfieldLayerSet* lset = (rcHeightfieldLayerSet*)rcAlloc(sizeof(rcHeightfieldLayerSet), RC_ALLOC_PERM);
	memset(lset, 0, sizeof(rcHeightfieldLayerSet));
	return lset;
}

void rcFreeHeightfieldLayerSet(rcHeightfieldLayerSet* lset)
{
	if (!lset) return;
	for (int i = 0; i < lset->nlayers; ++i)
	{
		rcFree(lset->layers[i].heights);
		rcFree(lset->layers[i].areas);
		rcFree(lset->layers[i].cons);
	}
	rcFree(lset->layers);
	rcFree(lset);
}


rcContourSet* rcAllocContourSet()
{
	rcContourSet* cset = (rcContourSet*)rcAlloc(sizeof(rcContourSet), RC_ALLOC_PERM);
	memset(cset, 0, sizeof(rcContourSet));
	return cset;
}

void rcFreeContourSet(rcContourSet* cset)
{
	if (!cset) return;
	for (int i = 0; i < cset->nconts; ++i)
	{
		rcFree(cset->conts[i].verts);
		rcFree(cset->conts[i].rverts);
	}
	rcFree(cset->conts);
	rcFree(cset);
}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
rcClusterSet* rcAllocClusterSet()
{
	rcClusterSet* clusters = (rcClusterSet*)rcAlloc(sizeof(rcClusterSet), RC_ALLOC_PERM);
	memset(clusters, 0, sizeof(rcClusterSet));
	return clusters;
}

void rcFreeClusterSet(rcClusterSet* clusters)
{
	if (!clusters) return;
	rcFree(clusters->center);
	rcFree(clusters->nlinks);
	rcFree(clusters->links);
	rcFree(clusters);
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

rcPolyMesh* rcAllocPolyMesh()
{
	rcPolyMesh* pmesh = (rcPolyMesh*)rcAlloc(sizeof(rcPolyMesh), RC_ALLOC_PERM);
	memset(pmesh, 0, sizeof(rcPolyMesh));
	return pmesh;
}

void rcFreePolyMesh(rcPolyMesh* pmesh)
{
	if (!pmesh) return;
	rcFree(pmesh->verts);
	rcFree(pmesh->polys);
	rcFree(pmesh->regs);
	rcFree(pmesh->flags);
	rcFree(pmesh->areas);
	rcFree(pmesh);
}

rcPolyMeshDetail* rcAllocPolyMeshDetail()
{
	rcPolyMeshDetail* dmesh = (rcPolyMeshDetail*)rcAlloc(sizeof(rcPolyMeshDetail), RC_ALLOC_PERM);
	memset(dmesh, 0, sizeof(rcPolyMeshDetail));
	return dmesh;
}

void rcFreePolyMeshDetail(rcPolyMeshDetail* dmesh)
{
	if (!dmesh) return;
	rcFree(dmesh->meshes);
	rcFree(dmesh->verts);
	rcFree(dmesh->tris);
	rcFree(dmesh);
}

void rcCalcBounds(const rcReal* verts, int nv, rcReal* bmin, rcReal* bmax)
{
	// Calculate bounding box.
	rcVcopy(bmin, verts);
	rcVcopy(bmax, verts);
	for (int i = 1; i < nv; ++i)
	{
		const rcReal* v = &verts[i*3];
		rcVmin(bmin, v);
		rcVmax(bmax, v);
	}
}

void rcCalcGridSize(const rcReal* bmin, const rcReal* bmax, rcReal cs, int* w, int* h)
{
	*w = (int)((bmax[0] - bmin[0])/cs+0.5f);
	*h = (int)((bmax[2] - bmin[2])/cs+0.5f);
}

/// @par
///
/// See the #rcConfig documentation for more information on the configuration parameters.
/// 
/// @see rcAllocHeightfield, rcHeightfield 
bool rcCreateHeightfield(rcContext* /*ctx*/, rcHeightfield& hf, int width, int height,
						 const rcReal* bmin, const rcReal* bmax,
						 rcReal cs, rcReal ch, bool bAllocateTempSpanColumns)
{
	// TODO: VC complains about unref formal variable, figure out a way to handle this better.
//	rcAssert(ctx);
	
	hf.width = width;
	hf.height = height;
	rcVcopy(hf.bmin, bmin);
	rcVcopy(hf.bmax, bmax);
	hf.cs = cs;
	hf.ch = ch;
	hf.spans = (rcSpan**)rcAlloc(sizeof(rcSpan*)*hf.width*hf.height, RC_ALLOC_PERM);
	if (!hf.spans)
		return false;
	memset(hf.spans, 0, sizeof(rcSpan*)*hf.width*hf.height);

#if EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
	hf.EdgeHits = (rcEdgeHit*)rcAlloc(sizeof(rcEdgeHit) * (hf.height + 1), RC_ALLOC_PERM); 
	if (!hf.EdgeHits)
		return false;
	memset(hf.EdgeHits, 0, sizeof(rcEdgeHit) * (hf.height + 1));

	hf.RowExt = (rcRowExt*)rcAlloc(sizeof(rcRowExt) * (hf.height + 2), RC_ALLOC_PERM); 

	for (int i = 0; i < hf.height + 2; i++)
	{
		hf.RowExt[i].MinCol = hf.width + 2;
		hf.RowExt[i].MaxCol = -2;
	}

	hf.tempspans = (rcTempSpan*)rcAlloc(sizeof(rcTempSpan)*(hf.width + 2) * (hf.height + 2), RC_ALLOC_PERM); 
	if (!hf.tempspans)
		return false;

	for (int i = 0; i < hf.height + 2; i++)
	{
		for (int j = 0; j < hf.width + 2; j++)
		{
			constexpr int RANGE = INT_MAX;
			hf.tempspans[i * (hf.width + 2) + j].sminmax[0] = RANGE;
			hf.tempspans[i * (hf.width + 2) + j].sminmax[1] = -RANGE;
		}
	}

	if (bAllocateTempSpanColumns)
	{
		hf.tempSpanColumns = (rcSpanData*)rcAlloc(sizeof(rcSpanData)*hf.width*hf.height, RC_ALLOC_PERM); 
		if (!hf.tempSpanColumns)
			return false;
		memset(hf.tempSpanColumns, 0, sizeof(rcSpanData)*hf.width*hf.height);
	}
#endif // EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER

	return true;
}

void rcResetHeightfield(rcHeightfield& hf)
{
	// reset all spans in allocated pools
	hf.freelist = 0;
	for (rcSpanPool* ipool = hf.pools; ipool; ipool = ipool->next)
	{
		rcSpan* freelist = hf.freelist;
		rcSpan* head = &ipool->items[0];
		rcSpan* it = &ipool->items[RC_SPANS_PER_POOL];
		do
		{
			--it;
			it->next = freelist;
			freelist = it;
		}
		while (it != head);
		hf.freelist = it;
	}

	// reset grid
	memset(hf.spans, 0, sizeof(rcSpan*)*hf.width*hf.height);
}

static void calcTriNormal(const rcReal* v0, const rcReal* v1, const rcReal* v2, rcReal* norm)
{
	rcReal e0[3], e1[3];
	rcVsub(e0, v1, v0);
	rcVsub(e1, v2, v0);
	rcVcross(norm, e0, e1);
	rcVnormalize(norm);
}

//@UE BEGIN
void rcCalcTriNormals(const rcReal* verts, const int nv, const int* tris, const int nt, rcReal* norms)
{
	for (int i = 0; i < nt; ++i)
	{
		const int* tri = &tris[i*3];
		const rcReal* v0 = &verts[tri[0]*3];
		const rcReal* v1 = &verts[tri[1]*3];
		const rcReal* v2 = &verts[tri[2]*3];

		calcTriNormal(v0, v1, v2, &norms[i*3]);
	}
}
//@UE END

/// @par
///
/// Only sets the aread id's for the walkable triangles.  Does not alter the
/// area id's for unwalkable triangles.
/// 
/// See the #rcConfig documentation for more information on the configuration parameters.
/// 
/// @see rcHeightfield, rcClearUnwalkableTriangles, rcRasterizeTriangles
void rcMarkWalkableTriangles(rcContext* /*ctx*/, const rcReal walkableSlopeAngle,
							 const rcReal* verts, int /*nv*/,
							 const int* tris, int nt,
							 unsigned char* areas)
{
	// TODO: VC complains about unref formal variable, figure out a way to handle this better.
//	rcAssert(ctx);
	
	const rcReal walkableThr = rcCos(walkableSlopeAngle/180.0f*RC_PI);
	rcMarkWalkableTrianglesCos(0, walkableThr, verts, 0, tris, nt, areas);
}

void rcMarkWalkableTrianglesCos(rcContext* /*ctx*/, const rcReal walkableSlopeCos,
								const rcReal* verts, int /*nv*/,
								const int* tris, int nt,
								unsigned char* areas)
{
	rcReal norm[3];
	for (int i = 0; i < nt; ++i)
	{
		const int* tri = &tris[i*3];
		calcTriNormal(&verts[tri[0]*3], &verts[tri[1]*3], &verts[tri[2]*3], norm);
		// Check if the face is walkable.
		if (norm[1] > walkableSlopeCos)
			areas[i] = RC_WALKABLE_AREA;
	}
}

//@UE BEGIN
void rcApplyAreaToTriangles(unsigned char* areas, const int nt, const unsigned char areaId)
{
	if (areaId != RC_WALKABLE_AREA)
	{
		for (int i = 0; i < nt; ++i)
		{
			if (areas[i] == RC_WALKABLE_AREA)
				areas[i] = areaId;
		}
	}
}
//@UE END

/// @par
///
/// Only sets the aread id's for the unwalkable triangles.  Does not alter the
/// area id's for walkable triangles.
/// 
/// See the #rcConfig documentation for more information on the configuration parameters.
/// 
/// @see rcHeightfield, rcClearUnwalkableTriangles, rcRasterizeTriangles
void rcClearUnwalkableTriangles(rcContext* /*ctx*/, const rcReal walkableSlopeAngle,
								const rcReal* verts, int /*nv*/,
								const int* tris, int nt,
								unsigned char* areas)
{
	// TODO: VC complains about unref formal variable, figure out a way to handle this better.
//	rcAssert(ctx);
	
	const rcReal walkableThr = rcCos(walkableSlopeAngle/180.0f*RC_PI);
	
	rcReal norm[3];
	
	for (int i = 0; i < nt; ++i)
	{
		const int* tri = &tris[i*3];
		calcTriNormal(&verts[tri[0]*3], &verts[tri[1]*3], &verts[tri[2]*3], norm);
		// Check if the face is walkable.
		if (norm[1] <= walkableThr)
			areas[i] = RC_NULL_AREA;
	}
}

int rcGetHeightFieldSpanCount(rcContext* /*ctx*/, rcHeightfield& hf)
{
	// TODO: VC complains about unref formal variable, figure out a way to handle this better.
//	rcAssert(ctx);
	
	const int w = hf.width;
	const int h = hf.height;
	int spanCount = 0;
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			for (rcSpan* s = hf.spans[x + y*w]; s; s = s->next)
			{
				if (s->data.area != RC_NULL_AREA)
					spanCount++;
			}
		}
	}
	return spanCount;
}

/// @par
///
/// This is just the beginning of the process of fully building a compact heightfield.
/// Various filters may be applied, then the distance field and regions built.
/// E.g: #rcBuildDistanceField and #rcBuildRegions
///
/// See the #rcConfig documentation for more information on the configuration parameters.
///
/// @see rcAllocCompactHeightfield, rcHeightfield, rcCompactHeightfield, rcConfig
bool rcBuildCompactHeightfield(rcContext* ctx, const int walkableHeight, const int walkableClimb,
							   rcHeightfield& hf, rcCompactHeightfield& chf)
{
	rcAssert(ctx);
	
// @UE BEGIN: early-out when no walkable spans 
	const int spanCount = rcGetHeightFieldSpanCount(ctx, hf);
	if (spanCount == 0)
	{
		// no spans to speak of, bail out.
		return false;
	}
// @UE END
	ctx->startTimer(RC_TIMER_BUILD_COMPACTHEIGHTFIELD);
	
	const int w = hf.width;
	const int h = hf.height;

	// Fill in header.
	chf.width = w;
	chf.height = h;
	chf.spanCount = spanCount;
	chf.walkableHeight = walkableHeight;
	chf.walkableClimb = walkableClimb;
	chf.maxRegions = 0;
	rcVcopy(chf.bmin, hf.bmin);
	rcVcopy(chf.bmax, hf.bmax);
	chf.bmax[1] += walkableHeight*hf.ch;
	chf.cs = hf.cs;
	chf.ch = hf.ch;
	chf.cells = (rcCompactCell*)rcAlloc(sizeof(rcCompactCell)*w*h, RC_ALLOC_PERM);
	if (!chf.cells)
	{
		UE_LOGF(LogRecast, VeryVerbose, "rcBuildCompactHeightfield: Out of memory 'chf.cells' (%d)", w*h);
		return false;
	}
	memset(chf.cells, 0, sizeof(rcCompactCell)*w*h);
	chf.spans = (rcCompactSpan*)rcAlloc(sizeof(rcCompactSpan)*spanCount, RC_ALLOC_PERM);
	if (!chf.spans)
	{
		//converted to UE_log to avoid false positives with Chaos
		UE_LOGF(LogRecast, VeryVerbose, "rcBuildCompactHeightfield: Out of memory 'chf.spans' (%d)", spanCount);
		return false;
	}
	memset(chf.spans, 0, sizeof(rcCompactSpan)*spanCount);
	chf.areas = (unsigned char*)rcAlloc(sizeof(unsigned char)*spanCount, RC_ALLOC_PERM);
	if (!chf.areas)
	{
		UE_LOGF(LogRecast, VeryVerbose, "rcBuildCompactHeightfield: Out of memory 'chf.areas' (%d)", spanCount);
		return false;
	}
	memset(chf.areas, RC_NULL_AREA, sizeof(unsigned char)*spanCount);
	
	// Fill in cells and spans.
	int idx = 0;
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcSpan* s = hf.spans[x + y*w];
			// If there are no spans at this cell, just leave the data to index=0, count=0.
			if (!s) continue;
			rcCompactCell& c = chf.cells[x+y*w];
			c.index = idx;
			c.count = 0;
			while (s)
			{
				if (s->data.area != RC_NULL_AREA)
				{
					const rcSpanUInt bot = (int)s->data.smax;
					const rcSpanUInt top = s->next ? (int)s->next->data.smin : RC_SPAN_MAX_HEIGHT;
					chf.spans[idx].y = rcClamp(bot, 0, RC_SPAN_MAX_HEIGHT);
					chf.spans[idx].h = (unsigned char)rcClamp(top - bot, 0, 0xff);
					chf.areas[idx] = s->data.area;
					idx++;
					c.count++;
				}
				s = s->next;
			}
		}
	}

	// Find neighbour connections.
	const int MAX_LAYERS = RC_NOT_CONNECTED-1;
	int tooHighNeighbour = 0;
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				rcCompactSpan& s = chf.spans[i];
				
				for (int dir = 0; dir < 4; ++dir)
				{
					rcSetCon(s, dir, RC_NOT_CONNECTED);
					const int nx = x + rcGetDirOffsetX(dir);
					const int ny = y + rcGetDirOffsetY(dir);
					// First check that the neighbour cell is in bounds.
					if (nx < 0 || ny < 0 || nx >= w || ny >= h)
						continue;
						
					// Iterate over all neighbour spans and check if any of the is
					// accessible from current cell.
					const rcCompactCell& nc = chf.cells[nx+ny*w];
					for (int k = (int)nc.index, nk = (int)(nc.index+nc.count); k < nk; ++k)
					{
						const rcCompactSpan& ns = chf.spans[k];
						const rcSpanUInt bot = rcMax(s.y, ns.y);
						const rcSpanUInt top = rcMin(s.y+s.h, ns.y+ns.h);

						// Check that the gap between the spans is walkable,
						// and that the climb height between the gaps is not too high.
						if (((int)top - (int)bot) >= walkableHeight && rcAbs((int)ns.y - (int)s.y) <= walkableClimb)
						{
							// Mark direction as walkable.
							const int lidx = k - (int)nc.index;
							if (lidx < 0 || lidx > MAX_LAYERS)
							{
								tooHighNeighbour = rcMax(tooHighNeighbour, lidx);
								continue;
							}
							rcSetCon(s, dir, lidx);
							break;
						}
					}
					
				}
			}
		}
	}
	
	if (tooHighNeighbour > MAX_LAYERS)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildCompactHeightfield: Heightfield has too many layers %d (max: %d)",
				 tooHighNeighbour, MAX_LAYERS);
	}
		
	ctx->stopTimer(RC_TIMER_BUILD_COMPACTHEIGHTFIELD);
	
	return true;
}

/*
static int getHeightfieldMemoryUsage(const rcHeightfield& hf)
{
	int size = 0;
	size += sizeof(hf);
	size += hf.width * hf.height * sizeof(rcSpan*);
	
	rcSpanPool* pool = hf.pools;
	while (pool)
	{
		size += (sizeof(rcSpanPool) - sizeof(rcSpan)) + sizeof(rcSpan)*RC_SPANS_PER_POOL;
		pool = pool->next;
	}
	return size;
}

static int getCompactHeightFieldMemoryusage(const rcCompactHeightfield& chf)
{
	int size = 0;
	size += sizeof(rcCompactHeightfield);
	size += sizeof(rcCompactSpan) * chf.spanCount;
	size += sizeof(rcCompactCell) * chf.width * chf.height;
	return size;
}
*/
