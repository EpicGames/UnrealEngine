// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "CoreMinimal.h"
#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAlloc.h"
#include "Recast/RecastAssert.h"

/// @par 
/// 
/// Basically, any spans that are closer to a boundary or obstruction than the specified radius 
/// are marked as unwalkable.
///
/// This method is usually called immediately after the heightfield has been built.
///
/// @see rcCompactHeightfield, rcBuildCompactHeightfield, rcConfig::walkableRadius
bool rcErodeWalkableArea(rcContext* ctx, int radius, rcCompactHeightfield& chf)
{
	rcAssert(ctx);
	
	const int w = chf.width;
	const int h = chf.height;
	
	ctx->startTimer(RC_TIMER_ERODE_AREA);
	
	unsigned char* dist = (unsigned char*)rcAlloc(sizeof(unsigned char)*chf.spanCount, RC_ALLOC_TEMP);
	if (!dist)
	{
		ctx->log(RC_LOG_ERROR, "erodeWalkableArea: Out of memory 'dist' (%d).", chf.spanCount);
		return false;
	}
	
	// Init distance.
	memset(dist, 0xff, sizeof(unsigned char)*chf.spanCount);

	// Mark boundary cells.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				if (chf.areas[i] == RC_NULL_AREA)
				{
					dist[i] = 0;
				}
				else
				{
					const rcCompactSpan& s = chf.spans[i];
					int nc = 0;
					for (int dir = 0; dir < 4; ++dir)
					{
						if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
						{
							const int nx = x + rcGetDirOffsetX(dir);
							const int ny = y + rcGetDirOffsetY(dir);
							const int nidx = (int)chf.cells[nx+ny*w].index + rcGetCon(s, dir);
							if (chf.areas[nidx] != RC_NULL_AREA)
							{
								nc++;
							}
						}
					}
					// At least one missing neighbour.
					if (nc != 4)
						dist[i] = 0;
				}
			}
		}
	}
	
	unsigned char nd;

	// Pass 1
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];

				if (rcGetCon(s, 0) != RC_NOT_CONNECTED)
				{
					// (-1,0)
					const int ax = x + rcGetDirOffsetX(0);
					const int ay = y + rcGetDirOffsetY(0);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 0);
					const rcCompactSpan& as = chf.spans[ai];
					nd = (unsigned char)rcMin((int)dist[ai] + 2, 255);
					if (nd < dist[i])
					{
						dist[i] = nd;
					}

					// (-1,-1)
					if (rcGetCon(as, 3) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(3);
						const int aay = ay + rcGetDirOffsetY(3);
						const int aai = (int)chf.cells[aax + aay*w].index + rcGetCon(as, 3);
						nd = (unsigned char)rcMin((int)dist[aai] + 3, 255);
						if (nd < dist[i])
							dist[i] = nd;
					}
				}
				if (rcGetCon(s, 3) != RC_NOT_CONNECTED)
				{
					// (0,-1)
					const int ax = x + rcGetDirOffsetX(3);
					const int ay = y + rcGetDirOffsetY(3);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 3);
					const rcCompactSpan& as = chf.spans[ai];
					nd = (unsigned char)rcMin((int)dist[ai] + 2, 255);
					if (nd < dist[i])
						dist[i] = nd;

					// (1,-1)
					if (rcGetCon(as, 2) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(2);
						const int aay = ay + rcGetDirOffsetY(2);
						const int aai = (int)chf.cells[aax + aay*w].index + rcGetCon(as, 2);
						nd = (unsigned char)rcMin((int)dist[aai] + 3, 255);
						if (nd < dist[i])
							dist[i] = nd;
					}
				}
			}
		}
	}

	// Pass 2
	for (int y = h - 1; y >= 0; --y)
	{
		for (int x = w - 1; x >= 0; --x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];

				if (rcGetCon(s, 2) != RC_NOT_CONNECTED)
				{
					// (1,0)
					const int ax = x + rcGetDirOffsetX(2);
					const int ay = y + rcGetDirOffsetY(2);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 2);
					const rcCompactSpan& as = chf.spans[ai];
					nd = (unsigned char)rcMin((int)dist[ai] + 2, 255);
					if (nd < dist[i])
						dist[i] = nd;

					// (1,1)
					if (rcGetCon(as, 1) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(1);
						const int aay = ay + rcGetDirOffsetY(1);
						const int aai = (int)chf.cells[aax + aay*w].index + rcGetCon(as, 1);
						nd = (unsigned char)rcMin((int)dist[aai] + 3, 255);
						if (nd < dist[i])
							dist[i] = nd;
					}
				}
				if (rcGetCon(s, 1) != RC_NOT_CONNECTED)
				{
					// (0,1)
					const int ax = x + rcGetDirOffsetX(1);
					const int ay = y + rcGetDirOffsetY(1);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 1);
					const rcCompactSpan& as = chf.spans[ai];
					nd = (unsigned char)rcMin((int)dist[ai] + 2, 255);
					if (nd < dist[i])
						dist[i] = nd;

					// (-1,1)
					if (rcGetCon(as, 0) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(0);
						const int aay = ay + rcGetDirOffsetY(0);
						const int aai = (int)chf.cells[aax + aay*w].index + rcGetCon(as, 0);
						nd = (unsigned char)rcMin((int)dist[aai] + 3, 255);
						if (nd < dist[i])
							dist[i] = nd;
					}
				}
			}
		}
	}

	const unsigned char thr = (unsigned char)(radius*2);
	for (int i = 0; i < chf.spanCount; ++i)
		if (dist[i] < thr)
			chf.areas[i] = RC_NULL_AREA;
	
	rcFree(dist);

	ctx->stopTimer(RC_TIMER_ERODE_AREA);
	
	return true;
}

void SeedWalkableAreasErode(int w, int h, int lowSpanHeight, int lowSpanSeed, const rcCompactHeightfield& chf, unsigned char* dist, unsigned char* seed)
{
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				if (chf.areas[i] == RC_NULL_AREA)
				{
					dist[i] = 0;
				}
				else if (chf.spans[i].h < lowSpanHeight)
				{
					dist[i] = 0;
					seed[i] = (unsigned char)lowSpanSeed;
				}
				else
				{
					const rcCompactSpan& s = chf.spans[i];
					int nc = 0;
					for (int dir = 0; dir < 4; ++dir)
					{
						if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
						{
							const int nx = x + rcGetDirOffsetX(dir);
							const int ny = y + rcGetDirOffsetY(dir);
							const int nidx = (int)chf.cells[nx + ny*w].index + rcGetCon(s, dir);
							if (chf.areas[nidx] != RC_NULL_AREA)
							{
								nc++;
							}
						}
					}
					// At least one missing neighbour.
					if (nc != 4)
						dist[i] = 0;
				}
			}
		}
	}
}

void SeedWalkableAreasErodeFiltered(int w, int h, int lowSpanHeight, int lowSpanSeed, const rcCompactHeightfield& chf, unsigned char* dist, unsigned char* seed)
{
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];

			// initialize to max height in case it needs to start with a low span
			rcSpanUInt NextAllowedLowBase = RC_SPAN_MAX_HEIGHT;

			for (int ni = (int)c.index, i = (int)(c.index + c.count) - 1; i >= ni; i--)
			{
				const rcCompactSpan& s = chf.spans[i];
				if (chf.areas[i] == RC_NULL_AREA)
				{
					// null area doesn't reset low span Z gate, they are often mixed forming series of small spans (1..10 voxels height)
					dist[i] = 0;
				}
				else if (s.h < lowSpanHeight)
				{
					dist[i] = 0;

					if (s.y < NextAllowedLowBase)
					{
						seed[i] = (unsigned char)lowSpanSeed;

						// next low span is allowed after full agent height (or valid area from condition below)
						NextAllowedLowBase = rcMax(0, (int)s.y - lowSpanHeight);
					}
					else
					{
						// mark as unable to receive seed propagation
						seed[i] = 0xff;
					}
				}
				else
				{
					// allow low span under this one
					NextAllowedLowBase = s.y;

					int nc = 0;
					for (int dir = 0; dir < 4; ++dir)
					{
						if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
						{
							const int nx = x + rcGetDirOffsetX(dir);
							const int ny = y + rcGetDirOffsetY(dir);
							const int nidx = (int)chf.cells[nx + ny*w].index + rcGetCon(s, dir);
							if (chf.areas[nidx] != RC_NULL_AREA)
							{
								nc++;
							}
						}
					}
					// At least one missing neighbour.
					if (nc != 4)
						dist[i] = 0;
				}
			}
		}
	}
}

void FilterLowHeightSpans(int w, int h, int lowSpanHeight, unsigned char areaId, const rcCompactHeightfield& chf)
{
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];

			// initialize to max height in case it needs to start with a low span
			rcSpanUInt NextAllowedLowBase = RC_SPAN_MAX_HEIGHT;

			for (int ni = (int)c.index, i = (int)(c.index + c.count) - 1; i >= ni; i--)
			{
				const rcCompactSpan& s = chf.spans[i];
				if (chf.areas[i] == areaId)
				{
					if (s.y > NextAllowedLowBase)
					{
						chf.areas[i] = RC_NULL_AREA;
					}
					else
					{
						NextAllowedLowBase = rcMax(0, (int)s.y - lowSpanHeight);
					}
				}
				else if (chf.areas[i] != RC_NULL_AREA)
				{
					NextAllowedLowBase = s.y;
				}
			}
		}
	}
}

/// @par 
/// 
/// Basically, any spans that are closer to a boundary or obstruction than the specified radius 
/// are marked as unwalkable.
///
/// This method is usually called immediately after the heightfield has been built.
///
/// @see rcCompactHeightfield, rcBuildCompactHeightfield, rcConfig::walkableRadius
bool rcErodeWalkableAndLowAreas(rcContext* ctx, int radius, unsigned int height,
	unsigned char areaId, unsigned char filterFlags, rcCompactHeightfield& chf)
{
	rcAssert(ctx);

	const int w = chf.width;
	const int h = chf.height;

	ctx->startTimer(RC_TIMER_ERODE_AREA);

	unsigned char* dist = (unsigned char*)rcAlloc(sizeof(unsigned char)*chf.spanCount, RC_ALLOC_TEMP);
	if (!dist)
	{
		ctx->log(RC_LOG_ERROR, "erodeWalkableArea: Out of memory 'dist' (%d).", chf.spanCount);
		return false;
	}
	unsigned char* seed = (unsigned char*)rcAlloc(sizeof(unsigned char)*chf.spanCount, RC_ALLOC_TEMP);
	if (!seed)
	{
		rcFree(dist);
		ctx->log(RC_LOG_ERROR, "erodeWalkableArea: Out of memory 'seed' (%d).", chf.spanCount);
		return false;
	}

	// Init distance.
	memset(dist, 0xff, sizeof(unsigned char)*chf.spanCount);
	memset(seed, 0x0, sizeof(unsigned char)*chf.spanCount);

	const unsigned char thr = (unsigned char)(radius * 2);
	if (filterFlags & RC_LOW_FILTER_SEED_SPANS)
	{
		SeedWalkableAreasErodeFiltered(w, h, height, thr, chf, dist, seed);
	}
	else
	{
		SeedWalkableAreasErode(w, h, height, thr, chf, dist, seed);
	}

	unsigned char nd;
	unsigned char ns;

	// Pass 1
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				unsigned char updatedSeed = seed[i];

				if (rcGetCon(s, 0) != RC_NOT_CONNECTED)
				{
					// (-1,0)
					const int ax = x + rcGetDirOffsetX(0);
					const int ay = y + rcGetDirOffsetY(0);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 0);
					const rcCompactSpan& as = chf.spans[ai];
					nd = (unsigned char)rcMin((int)dist[ai] + 2, 255);
					if (nd < dist[i])
					{
						dist[i] = nd;
					}

					if (seed[ai] != 0xff)
					{
						ns = (unsigned char)rcMax((int)seed[ai] - 2, 0);
						updatedSeed = rcMax(updatedSeed, ns);
					}

					// (-1,-1)
					if (rcGetCon(as, 3) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(3);
						const int aay = ay + rcGetDirOffsetY(3);
						const int aai = (int)chf.cells[aax + aay*w].index + rcGetCon(as, 3);
						nd = (unsigned char)rcMin((int)dist[aai] + 3, 255);
						if (nd < dist[i])
							dist[i] = nd;

						if (seed[aai] != 0xff)
						{
							ns = (unsigned char)rcMax((int)seed[aai] - 3, 0);
							updatedSeed = rcMax(updatedSeed, ns);
						}
					}
				}
				if (rcGetCon(s, 3) != RC_NOT_CONNECTED)
				{
					// (0,-1)
					const int ax = x + rcGetDirOffsetX(3);
					const int ay = y + rcGetDirOffsetY(3);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 3);
					const rcCompactSpan& as = chf.spans[ai];
					nd = (unsigned char)rcMin((int)dist[ai] + 2, 255);
					if (nd < dist[i])
						dist[i] = nd;

					if (seed[ai] != 0xff)
					{
						ns = (unsigned char)rcMax((int)seed[ai] - 2, 0);
						updatedSeed = rcMax(updatedSeed, ns);
					}

					// (1,-1)
					if (rcGetCon(as, 2) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(2);
						const int aay = ay + rcGetDirOffsetY(2);
						const int aai = (int)chf.cells[aax + aay*w].index + rcGetCon(as, 2);
						nd = (unsigned char)rcMin((int)dist[aai] + 3, 255);
						if (nd < dist[i])
							dist[i] = nd;

						if (seed[aai] != 0xff)
						{
							ns = (unsigned char)rcMax((int)seed[aai] - 3, 0);
							updatedSeed = rcMax(updatedSeed, ns);
						}
					}
				}

				const bool bCanReceiveSeedPropagation = (seed[i] != 0xff);
				if (bCanReceiveSeedPropagation)
				{
					seed[i] = updatedSeed;
				}
			}
		}
	}

	// Pass 2
	for (int y = h - 1; y >= 0; --y)
	{
		for (int x = w - 1; x >= 0; --x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				unsigned char updatedSeed = seed[i];

				if (rcGetCon(s, 2) != RC_NOT_CONNECTED)
				{
					// (1,0)
					const int ax = x + rcGetDirOffsetX(2);
					const int ay = y + rcGetDirOffsetY(2);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 2);
					const rcCompactSpan& as = chf.spans[ai];
					nd = (unsigned char)rcMin((int)dist[ai] + 2, 255);
					if (nd < dist[i])
						dist[i] = nd;

					if (seed[ai] != 0xff)
					{
						ns = (unsigned char)rcMax((int)seed[ai] - 2, 0);
						updatedSeed = rcMax(updatedSeed, ns);
					}

					// (1,1)
					if (rcGetCon(as, 1) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(1);
						const int aay = ay + rcGetDirOffsetY(1);
						const int aai = (int)chf.cells[aax + aay*w].index + rcGetCon(as, 1);
						nd = (unsigned char)rcMin((int)dist[aai] + 3, 255);
						if (nd < dist[i])
							dist[i] = nd;

						if (seed[aai] != 0xff)
						{
							ns = (unsigned char)rcMax((int)seed[aai] - 3, 0);
							updatedSeed = rcMax(updatedSeed, ns);
						}
					}
				}
				if (rcGetCon(s, 1) != RC_NOT_CONNECTED)
				{
					// (0,1)
					const int ax = x + rcGetDirOffsetX(1);
					const int ay = y + rcGetDirOffsetY(1);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 1);
					const rcCompactSpan& as = chf.spans[ai];
					nd = (unsigned char)rcMin((int)dist[ai] + 2, 255);
					if (nd < dist[i])
						dist[i] = nd;

					if (seed[ai] != 0xff)
					{
						ns = (unsigned char)rcMax((int)seed[ai] - 2, 0);
						updatedSeed = rcMax(updatedSeed, ns);
					}

					// (-1,1)
					if (rcGetCon(as, 0) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(0);
						const int aay = ay + rcGetDirOffsetY(0);
						const int aai = (int)chf.cells[aax + aay*w].index + rcGetCon(as, 0);
						nd = (unsigned char)rcMin((int)dist[aai] + 3, 255);
						if (nd < dist[i])
							dist[i] = nd;

						if (seed[aai] != 0xff)
						{
							ns = (unsigned char)rcMax((int)seed[aai] - 3, 0);
							updatedSeed = rcMax(updatedSeed, ns);
						}
					}
				}

				const bool bCanReceiveSeedPropagation = (seed[i] != 0xff);
				if (bCanReceiveSeedPropagation)
				{
					seed[i] = updatedSeed;
				}
			}
		}
	}

	for (int i = 0; i < chf.spanCount; ++i)
	{
		if (seed[i])
		{
			chf.areas[i] = areaId;
		}
		else if (dist[i] < thr)
		{
			chf.areas[i] = RC_NULL_AREA;
		}
	}

	if (filterFlags & RC_LOW_FILTER_POST_PROCESS)
	{
		// seed propagation will flood a bit and create low spans too close to each other, filter them again 
		FilterLowHeightSpans(w, h, height, areaId, chf);
	}

	rcFree(dist);
	rcFree(seed);

	ctx->stopTimer(RC_TIMER_ERODE_AREA);

	return true;
}

static void insertSort(unsigned char* a, const int n)
{
	int i, j;
	for (i = 1; i < n; i++)
	{
		const unsigned char value = a[i];
		for (j = i - 1; j >= 0 && a[j] > value; j--)
			a[j+1] = a[j];
		a[j+1] = value;
	}
}

/// @par
///
/// This filter is usually applied after applying area id's using functions
/// such as #rcMarkBoxArea, #rcMarkConvexPolyArea, and #rcMarkCylinderArea.
/// 
/// @see rcCompactHeightfield
bool rcMedianFilterWalkableArea(rcContext* ctx, rcCompactHeightfield& chf)
{
	rcAssert(ctx);
	
	const int w = chf.width;
	const int h = chf.height;
	
	ctx->startTimer(RC_TIMER_MEDIAN_AREA);
	
	unsigned char* areas = (unsigned char*)rcAlloc(sizeof(unsigned char)*chf.spanCount, RC_ALLOC_TEMP);
	if (!areas)
	{
		ctx->log(RC_LOG_ERROR, "medianFilterWalkableArea: Out of memory 'areas' (%d).", chf.spanCount);
		return false;
	}
	
	// Init distance.
	memset(areas, 0xff, sizeof(unsigned char)*chf.spanCount);
	
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				if (chf.areas[i] == RC_NULL_AREA)
				{
					areas[i] = chf.areas[i];
					continue;
				}
				
				unsigned char nei[9];
				for (int j = 0; j < 9; ++j)
					nei[j] = chf.areas[i];
				
				for (int dir = 0; dir < 4; ++dir)
				{
					if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
					{
						const int ax = x + rcGetDirOffsetX(dir);
						const int ay = y + rcGetDirOffsetY(dir);
						const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, dir);
						if (chf.areas[ai] != RC_NULL_AREA)
							nei[dir*2+0] = chf.areas[ai];
						
						const rcCompactSpan& as = chf.spans[ai];
						const int dir2 = (dir+1) & 0x3;
						if (rcGetCon(as, dir2) != RC_NOT_CONNECTED)
						{
							const int ax2 = ax + rcGetDirOffsetX(dir2);
							const int ay2 = ay + rcGetDirOffsetY(dir2);
							const int ai2 = (int)chf.cells[ax2+ay2*w].index + rcGetCon(as, dir2);
							if (chf.areas[ai2] != RC_NULL_AREA)
								nei[dir*2+1] = chf.areas[ai2];
						}
					}
				}
				insertSort(nei, 9);
				areas[i] = nei[4];
			}
		}
	}
	
	memcpy(chf.areas, areas, sizeof(unsigned char)*chf.spanCount);
	
	rcFree(areas);

	ctx->stopTimer(RC_TIMER_MEDIAN_AREA);
	
	return true;
}

bool rcMarkLowAreas(rcContext* ctx, unsigned int height, unsigned char areaId, rcCompactHeightfield& chf)
{
	const int w = chf.width;
	const int h = chf.height;

	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				if (chf.areas[i] == RC_WALKABLE_AREA && chf.spans[i].h < height)
				{
					chf.areas[i] = areaId;
				}
			}
		}
	}

	return true;
}

/// @par
///
/// The value of spacial parameters are in world units.
/// 
/// @see rcCompactHeightfield, rcMedianFilterWalkableArea
void rcMarkBoxArea(rcContext* ctx, const rcReal* bmin, const rcReal* bmax, unsigned char areaId,
				   rcCompactHeightfield& chf)
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_MARK_BOX_AREA);

	int minx = (int)((bmin[0]-chf.bmin[0])/chf.cs);
	int miny = (int)((bmin[1]-chf.bmin[1])/chf.ch);
	int minz = (int)((bmin[2]-chf.bmin[2])/chf.cs);
	int maxx = (int)((bmax[0]-chf.bmin[0])/chf.cs);
	int maxy = (int)((bmax[1]-chf.bmin[1])/chf.ch);
	int maxz = (int)((bmax[2]-chf.bmin[2])/chf.cs);
	
	if (maxx < 0) return;
	if (minx >= chf.width) return;
	if (maxz < 0) return;
	if (minz >= chf.height) return;

	if (minx < 0) minx = 0;
	if (maxx >= chf.width) maxx = chf.width-1;
	if (minz < 0) minz = 0;
	if (maxz >= chf.height) maxz = chf.height-1;	
	
	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			const rcCompactCell& c = chf.cells[x+z*chf.width];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				rcCompactSpan& s = chf.spans[i];
				if ((int)s.y >= miny && (int)s.y <= maxy)
				{
					if (chf.areas[i] != RC_NULL_AREA)
						chf.areas[i] = areaId;
				}
			}
		}
	}

	ctx->stopTimer(RC_TIMER_MARK_BOX_AREA);

}


static int pointInPoly(int nvert, const rcReal* verts, const rcReal* p)
{
	int i, j, c = 0;
	for (i = 0, j = nvert-1; i < nvert; j = i++)
	{
		const rcReal* vi = &verts[i*3];
		const rcReal* vj = &verts[j*3];
		if (((vi[2] > p[2]) != (vj[2] > p[2])) &&
			(p[0] < (vj[0]-vi[0]) * (p[2]-vi[2]) / (vj[2]-vi[2]) + vi[0]) )
			c = !c;
	}
	return c;
}

/// @par
///
/// The value of spacial parameters are in world units.
/// 
/// The y-values of the polygon vertices are ignored. So the polygon is effectively 
/// projected onto the xz-plane at @p hmin, then extruded to @p hmax.
/// 
/// @see rcCompactHeightfield, rcMedianFilterWalkableArea
void rcMarkConvexPolyArea(rcContext* ctx, const rcReal* verts, const int nverts,
						  const rcReal hmin, const rcReal hmax, unsigned char areaId,
						  rcCompactHeightfield& chf)
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_MARK_CONVEXPOLY_AREA);

	rcReal bmin[3], bmax[3];
	rcVcopy(bmin, verts);
	rcVcopy(bmax, verts);
	for (int i = 1; i < nverts; ++i)
	{
		rcVmin(bmin, &verts[i*3]);
		rcVmax(bmax, &verts[i*3]);
	}
	bmin[1] = hmin;
	bmax[1] = hmax;

	int minx = (int)((bmin[0]-chf.bmin[0])/chf.cs);
	int miny = (int)((bmin[1]-chf.bmin[1])/chf.ch);
	int minz = (int)((bmin[2]-chf.bmin[2])/chf.cs);
	int maxx = (int)((bmax[0]-chf.bmin[0])/chf.cs);
	int maxy = (int)((bmax[1]-chf.bmin[1])/chf.ch);
	int maxz = (int)((bmax[2]-chf.bmin[2])/chf.cs);
	
	if (maxx < 0) return;
	if (minx >= chf.width) return;
	if (maxz < 0) return;
	if (minz >= chf.height) return;
	
	if (minx < 0) minx = 0;
	if (maxx >= chf.width) maxx = chf.width-1;
	if (minz < 0) minz = 0;
	if (maxz >= chf.height) maxz = chf.height-1;	
	
	
	// TODO: Optimize.
	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			const rcCompactCell& c = chf.cells[x+z*chf.width];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				rcCompactSpan& s = chf.spans[i];
				if (chf.areas[i] == RC_NULL_AREA)
					continue;
				if ((int)s.y >= miny && (int)s.y <= maxy)
				{
					rcReal p[3];
					p[0] = chf.bmin[0] + (rcReal(x)+0.5f)*chf.cs; 
					p[1] = 0;
					p[2] = chf.bmin[2] + (rcReal(z)+0.5f)*chf.cs; 

					if (pointInPoly(nverts, verts, p))
					{
						chf.areas[i] = areaId;
					}
				}
			}
		}
	}

	ctx->stopTimer(RC_TIMER_MARK_CONVEXPOLY_AREA);
}

int rcOffsetPoly(const rcReal* verts, const int nverts, const rcReal offset,
				 rcReal* outVerts, const int maxOutVerts)
{
	const rcReal	MITER_LIMIT = 1.20f;

	int n = 0;

	for (int i = 0; i < nverts; i++)
	{
		const int a = (i+nverts-1) % nverts;
		const int b = i;
		const int c = (i+1) % nverts;
		const rcReal* va = &verts[a*3];
		const rcReal* vb = &verts[b*3];
		const rcReal* vc = &verts[c*3];
		rcReal dx0 = vb[0] - va[0];
		rcReal dy0 = vb[2] - va[2];
		rcReal d0 = dx0*dx0 + dy0*dy0;
		if (d0 > 1e-6f)
		{
			d0 = 1.0f/rcSqrt(d0);
			dx0 *= d0;
			dy0 *= d0;
		}
		rcReal dx1 = vc[0] - vb[0];
		rcReal dy1 = vc[2] - vb[2];
		rcReal d1 = dx1*dx1 + dy1*dy1;
		if (d1 > 1e-6f)
		{
			d1 = 1.0f/rcSqrt(d1);
			dx1 *= d1;
			dy1 *= d1;
		}
		const rcReal dlx0 = -dy0;
		const rcReal dly0 = dx0;
		const rcReal dlx1 = -dy1;
		const rcReal dly1 = dx1;
		rcReal cross = dx1*dy0 - dx0*dy1;
		rcReal dmx = (dlx0 + dlx1) * 0.5f;
		rcReal dmy = (dly0 + dly1) * 0.5f;
		rcReal dmr2 = dmx*dmx + dmy*dmy;
		bool bevel = dmr2 * MITER_LIMIT*MITER_LIMIT < 1.0f;
		if (dmr2 > 1e-6f)
		{
			const rcReal scale = 1.0f / dmr2;
			dmx *= scale;
			dmy *= scale;
		}

		if (bevel && cross < 0.0f)
		{
			if (n+2 >= maxOutVerts)
				return 0;
			rcReal d = (1.0f - (dx0*dx1 + dy0*dy1))*0.5f;
			outVerts[n*3+0] = vb[0] + (-dlx0+dx0*d)*offset;
			outVerts[n*3+1] = vb[1];
			outVerts[n*3+2] = vb[2] + (-dly0+dy0*d)*offset;
			n++;
			outVerts[n*3+0] = vb[0] + (-dlx1-dx1*d)*offset;
			outVerts[n*3+1] = vb[1];
			outVerts[n*3+2] = vb[2] + (-dly1-dy1*d)*offset;
			n++;
		}
		else
		{
			if (n+1 >= maxOutVerts)
				return 0;
			outVerts[n*3+0] = vb[0] - dmx*offset;
			outVerts[n*3+1] = vb[1];
			outVerts[n*3+2] = vb[2] - dmy*offset;
			n++;
		}
	}
	
	return n;
}


/// @par
///
/// The value of spacial parameters are in world units.
/// 
/// @see rcCompactHeightfield, rcMedianFilterWalkableArea
void rcMarkCylinderArea(rcContext* ctx, const rcReal* pos,
						const rcReal r, const rcReal h, unsigned char areaId,
						rcCompactHeightfield& chf)
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_MARK_CYLINDER_AREA);
	
	rcReal bmin[3], bmax[3];
	bmin[0] = pos[0] - r;
	bmin[1] = pos[1];
	bmin[2] = pos[2] - r;
	bmax[0] = pos[0] + r;
	bmax[1] = pos[1] + h;
	bmax[2] = pos[2] + r;
	const rcReal r2 = r*r;
	
	int minx = (int)((bmin[0]-chf.bmin[0])/chf.cs);
	int miny = (int)((bmin[1]-chf.bmin[1])/chf.ch);
	int minz = (int)((bmin[2]-chf.bmin[2])/chf.cs);
	int maxx = (int)((bmax[0]-chf.bmin[0])/chf.cs);
	int maxy = (int)((bmax[1]-chf.bmin[1])/chf.ch);
	int maxz = (int)((bmax[2]-chf.bmin[2])/chf.cs);
	
	if (maxx < 0) return;
	if (minx >= chf.width) return;
	if (maxz < 0) return;
	if (minz >= chf.height) return;
	
	if (minx < 0) minx = 0;
	if (maxx >= chf.width) maxx = chf.width-1;
	if (minz < 0) minz = 0;
	if (maxz >= chf.height) maxz = chf.height-1;	
	
	
	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			const rcCompactCell& c = chf.cells[x+z*chf.width];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				rcCompactSpan& s = chf.spans[i];
				
				if (chf.areas[i] == RC_NULL_AREA)
					continue;
				
				if ((int)s.y >= miny && (int)s.y <= maxy)
				{
					const rcReal sx = chf.bmin[0] + (rcReal(x)+0.5f)*chf.cs;
					const rcReal sz = chf.bmin[2] + (rcReal(z)+0.5f)*chf.cs;
					const rcReal dx = sx - pos[0];
					const rcReal dz = sz - pos[2];
					
					if (dx*dx + dz*dz < r2)
					{
						chf.areas[i] = areaId;
					}
				}
			}
		}
	}
	
	ctx->stopTimer(RC_TIMER_MARK_CYLINDER_AREA);
}

/// @par
///
/// The value of spacial parameters are in world units.
/// 
/// @see rcCompactHeightfield, rcMedianFilterWalkableArea
void rcReplaceBoxArea(rcContext* ctx, const rcReal* bmin, const rcReal* bmax,
	unsigned char areaId, unsigned char filterAreaId,
	rcCompactHeightfield& chf)
{
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_MARK_BOX_AREA);

	int minx = (int)((bmin[0] - chf.bmin[0]) / chf.cs);
	int miny = (int)((bmin[1] - chf.bmin[1]) / chf.ch);
	int minz = (int)((bmin[2] - chf.bmin[2]) / chf.cs);
	int maxx = (int)((bmax[0] - chf.bmin[0]) / chf.cs);
	int maxy = (int)((bmax[1] - chf.bmin[1]) / chf.ch);
	int maxz = (int)((bmax[2] - chf.bmin[2]) / chf.cs);

	if (maxx < 0) return;
	if (minx >= chf.width) return;
	if (maxz < 0) return;
	if (minz >= chf.height) return;

	if (minx < 0) minx = 0;
	if (maxx >= chf.width) maxx = chf.width - 1;
	if (minz < 0) minz = 0;
	if (maxz >= chf.height) maxz = chf.height - 1;

	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			const rcCompactCell& c = chf.cells[x + z*chf.width];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				rcCompactSpan& s = chf.spans[i];
				if ((int)s.y >= miny && (int)s.y <= maxy)
				{
					if (chf.areas[i] == filterAreaId)
						chf.areas[i] = areaId;
				}
			}
		}
	}

	ctx->stopTimer(RC_TIMER_MARK_BOX_AREA);

}

/// @par
///
/// The value of spacial parameters are in world units.
/// 
/// The y-values of the polygon vertices are ignored. So the polygon is effectively 
/// projected onto the xz-plane at @p hmin, then extruded to @p hmax.
/// 
/// @see rcCompactHeightfield, rcMedianFilterWalkableArea
void rcReplaceConvexPolyArea(rcContext* ctx, const rcReal* verts, const int nverts,
	const rcReal hmin, const rcReal hmax, unsigned char areaId, unsigned char filterAreaId,
	rcCompactHeightfield& chf)
{
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_MARK_CONVEXPOLY_AREA);

	rcReal bmin[3], bmax[3];
	rcVcopy(bmin, verts);
	rcVcopy(bmax, verts);
	for (int i = 1; i < nverts; ++i)
	{
		rcVmin(bmin, &verts[i * 3]);
		rcVmax(bmax, &verts[i * 3]);
	}
	bmin[1] = hmin;
	bmax[1] = hmax;

	int minx = (int)((bmin[0] - chf.bmin[0]) / chf.cs);
	int miny = (int)((bmin[1] - chf.bmin[1]) / chf.ch);
	int minz = (int)((bmin[2] - chf.bmin[2]) / chf.cs);
	int maxx = (int)((bmax[0] - chf.bmin[0]) / chf.cs);
	int maxy = (int)((bmax[1] - chf.bmin[1]) / chf.ch);
	int maxz = (int)((bmax[2] - chf.bmin[2]) / chf.cs);

	if (maxx < 0) return;
	if (minx >= chf.width) return;
	if (maxz < 0) return;
	if (minz >= chf.height) return;

	if (minx < 0) minx = 0;
	if (maxx >= chf.width) maxx = chf.width - 1;
	if (minz < 0) minz = 0;
	if (maxz >= chf.height) maxz = chf.height - 1;


	// TODO: Optimize.
	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			const rcCompactCell& c = chf.cells[x + z*chf.width];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				rcCompactSpan& s = chf.spans[i];
				if (chf.areas[i] != filterAreaId)
					continue;
				if ((int)s.y >= miny && (int)s.y <= maxy)
				{
					rcReal p[3];
					p[0] = chf.bmin[0] + (rcReal(x) + 0.5f)*chf.cs;
					p[1] = 0;
					p[2] = chf.bmin[2] + (rcReal(z) + 0.5f)*chf.cs;

					if (pointInPoly(nverts, verts, p))
					{
						chf.areas[i] = areaId;
					}
				}
			}
		}
	}

	ctx->stopTimer(RC_TIMER_MARK_CONVEXPOLY_AREA);
}

/// @par
///
/// The value of spacial parameters are in world units.
/// 
/// @see rcCompactHeightfield, rcMedianFilterWalkableArea
void rcReplaceCylinderArea(rcContext* ctx, const rcReal* pos,
	const rcReal r, const rcReal h, unsigned char areaId, unsigned char filterAreaId,
	rcCompactHeightfield& chf)
{
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_MARK_CYLINDER_AREA);

	rcReal bmin[3], bmax[3];
	bmin[0] = pos[0] - r;
	bmin[1] = pos[1];
	bmin[2] = pos[2] - r;
	bmax[0] = pos[0] + r;
	bmax[1] = pos[1] + h;
	bmax[2] = pos[2] + r;
	const rcReal r2 = r*r;

	int minx = (int)((bmin[0] - chf.bmin[0]) / chf.cs);
	int miny = (int)((bmin[1] - chf.bmin[1]) / chf.ch);
	int minz = (int)((bmin[2] - chf.bmin[2]) / chf.cs);
	int maxx = (int)((bmax[0] - chf.bmin[0]) / chf.cs);
	int maxy = (int)((bmax[1] - chf.bmin[1]) / chf.ch);
	int maxz = (int)((bmax[2] - chf.bmin[2]) / chf.cs);

	if (maxx < 0) return;
	if (minx >= chf.width) return;
	if (maxz < 0) return;
	if (minz >= chf.height) return;

	if (minx < 0) minx = 0;
	if (maxx >= chf.width) maxx = chf.width - 1;
	if (minz < 0) minz = 0;
	if (maxz >= chf.height) maxz = chf.height - 1;


	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			const rcCompactCell& c = chf.cells[x + z*chf.width];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				rcCompactSpan& s = chf.spans[i];

				if (chf.areas[i] != filterAreaId)
					continue;

				if ((int)s.y >= miny && (int)s.y <= maxy)
				{
					const rcReal sx = chf.bmin[0] + (rcReal(x) + 0.5f)*chf.cs;
					const rcReal sz = chf.bmin[2] + (rcReal(z) + 0.5f)*chf.cs;
					const rcReal dx = sx - pos[0];
					const rcReal dz = sz - pos[2];

					if (dx*dx + dz*dz < r2)
					{
						chf.areas[i] = areaId;
					}
				}
			}
		}
	}

	ctx->stopTimer(RC_TIMER_MARK_CYLINDER_AREA);
}
