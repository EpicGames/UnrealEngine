// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAlloc.h"
#include "Recast/RecastAssert.h"


static int getCornerHeight(int x, int y, int i, int dir,
						   const rcCompactHeightfield& chf,
						   bool& isBorderVertex)
{
	const rcCompactSpan& s = chf.spans[i];
	int ch = (int)s.y;
	int dirp = (dir+1) & 0x3;
	
	unsigned int regs[4] = {0,0,0,0};
	
	// Combine region and area codes in order to prevent
	// border vertices which are in between two areas to be removed. 
	regs[0] = chf.spans[i].reg | (chf.areas[i] << 16);
	
	if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
	{
		const int ax = x + rcGetDirOffsetX(dir);
		const int ay = y + rcGetDirOffsetY(dir);
		const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(s, dir);
		const rcCompactSpan& as = chf.spans[ai];
		ch = rcMax(ch, (int)as.y);
		regs[1] = chf.spans[ai].reg | (chf.areas[ai] << 16);
		if (rcGetCon(as, dirp) != RC_NOT_CONNECTED)
		{
			const int ax2 = ax + rcGetDirOffsetX(dirp);
			const int ay2 = ay + rcGetDirOffsetY(dirp);
			const int ai2 = (int)chf.cells[ax2+ay2*chf.width].index + rcGetCon(as, dirp);
			const rcCompactSpan& as2 = chf.spans[ai2];
			ch = rcMax(ch, (int)as2.y);
			regs[2] = chf.spans[ai2].reg | (chf.areas[ai2] << 16);
		}
	}
	if (rcGetCon(s, dirp) != RC_NOT_CONNECTED)
	{
		const int ax = x + rcGetDirOffsetX(dirp);
		const int ay = y + rcGetDirOffsetY(dirp);
		const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(s, dirp);
		const rcCompactSpan& as = chf.spans[ai];
		ch = rcMax(ch, (int)as.y);
		regs[3] = chf.spans[ai].reg | (chf.areas[ai] << 16);
		if (rcGetCon(as, dir) != RC_NOT_CONNECTED)
		{
			const int ax2 = ax + rcGetDirOffsetX(dir);
			const int ay2 = ay + rcGetDirOffsetY(dir);
			const int ai2 = (int)chf.cells[ax2+ay2*chf.width].index + rcGetCon(as, dir);
			const rcCompactSpan& as2 = chf.spans[ai2];
			ch = rcMax(ch, (int)as2.y);
			regs[2] = chf.spans[ai2].reg | (chf.areas[ai2] << 16);
		}
	}

	// Check if the vertex is special edge vertex, these vertices will be removed later.
	for (int j = 0; j < 4; ++j)
	{
		const int a = j;
		const int b = (j+1) & 0x3;
		const int c = (j+2) & 0x3;
		const int d = (j+3) & 0x3;
		
		// The vertex is a border vertex there are two same exterior cells in a row,
		// followed by two interior cells and none of the regions are out of bounds.
		const bool twoSameExts = (regs[a] & regs[b] & RC_BORDER_REG) != 0 && regs[a] == regs[b];
		const bool twoInts = ((regs[c] | regs[d]) & RC_BORDER_REG) == 0;
		const bool intsSameArea = (regs[c]>>16) == (regs[d]>>16);
		const bool noZeros = regs[a] != 0 && regs[b] != 0 && regs[c] != 0 && regs[d] != 0;
		if (twoSameExts && twoInts && intsSameArea && noZeros)
		{
			isBorderVertex = true;
			break;
		}
	}
	
	return ch;
}

static void walkContour(int x, int y, int i,
						rcCompactHeightfield& chf,
						unsigned char* flags, rcIntArray& points)
{
	// Choose the first non-connected edge
	unsigned char dir = 0;
	while ((flags[i] & (1 << dir)) == 0)
		dir++;
	
	unsigned char startDir = dir;
	int starti = i;
	
	const unsigned char area = chf.areas[i];
	
	int iter = 0;
	while (++iter < 40000)
	{
		if (flags[i] & (1 << dir))
		{
			// Choose the edge corner
			bool isBorderVertex = false;
			bool isAreaBorder = false;
			int px = x;
			int py = getCornerHeight(x, y, i, dir, chf, isBorderVertex);
			int pz = y;
			switch(dir)
			{
				case 0: pz++; break;
				case 1: px++; pz++; break;
				case 2: px++; break;
			}
			int r = 0;
			const rcCompactSpan& s = chf.spans[i];
			if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
			{
				const int ax = x + rcGetDirOffsetX(dir);
				const int ay = y + rcGetDirOffsetY(dir);
				const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(s, dir);
				r = (int)chf.spans[ai].reg;
				if (area != chf.areas[ai])
					isAreaBorder = true;
			}
			if (isBorderVertex)
				r |= RC_BORDER_VERTEX;
			if (isAreaBorder)
				r |= RC_AREA_BORDER;
			points.push(px);
			points.push(py);
			points.push(pz);
			points.push(r);
			
			flags[i] &= ~(1 << dir); // Remove visited edges
			dir = (dir+1) & 0x3;  // Rotate CW
		}
		else
		{
			int ni = -1;
			const int nx = x + rcGetDirOffsetX(dir);
			const int ny = y + rcGetDirOffsetY(dir);
			const rcCompactSpan& s = chf.spans[i];
			if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
			{
				const rcCompactCell& nc = chf.cells[nx+ny*chf.width];
				ni = (int)nc.index + rcGetCon(s, dir);
			}
			if (ni == -1)
			{
				// Should not happen.
				return;
			}
			x = nx;
			y = ny;
			i = ni;
			dir = (dir+3) & 0x3;	// Rotate CCW
		}
		
		if (starti == i && startDir == dir)
		{
			break;
		}
	}
}

static rcReal distancePtSeg(const int x, const int z,
						   const int px, const int pz,
						   const int qx, const int qz)
{
/*	rcReal pqx = (rcReal)(qx - px);
	rcReal pqy = (rcReal)(qy - py);
	rcReal pqz = (rcReal)(qz - pz);
	rcReal dx = (rcReal)(x - px);
	rcReal dy = (rcReal)(y - py);
	rcReal dz = (rcReal)(z - pz);
	rcReal d = pqx*pqx + pqy*pqy + pqz*pqz;
	rcReal t = pqx*dx + pqy*dy + pqz*dz;
	if (d > 0)
		t /= d;
	if (t < 0)
		t = 0;
	else if (t > 1)
		t = 1;
	
	dx = px + t*pqx - x;
	dy = py + t*pqy - y;
	dz = pz + t*pqz - z;
	
	return dx*dx + dy*dy + dz*dz;*/

	rcReal pqx = (rcReal)(qx - px);
	rcReal pqz = (rcReal)(qz - pz);
	rcReal dx = (rcReal)(x - px);
	rcReal dz = (rcReal)(z - pz);
	rcReal d = pqx*pqx + pqz*pqz;
	rcReal t = pqx*dx + pqz*dz;
	if (d > 0)
		t /= d;
	if (t < 0)
		t = 0;
	else if (t > 1)
		t = 1;
	
	dx = px + t*pqx - x;
	dz = pz + t*pqz - z;
	
	return dx*dx + dz*dz;
}

static void simplifyContour(rcIntArray& points, rcIntArray& simplified,
							const rcReal maxError, const int maxEdgeLen, const int buildFlags)
{
	// Add initial points.
	bool hasConnections = false;
	for (int i = 0; i < points.size(); i += 4)
	{
		if ((points[i+3] & RC_CONTOUR_REG_MASK) != 0)
		{
			hasConnections = true;
			break;
		}
	}
	
	if (hasConnections)
	{
		// The contour has some portals to other regions.
		// Add a new point to every location where the region changes.
		for (int i = 0, ni = points.size()/4; i < ni; ++i)
		{
			int ii = (i+1) % ni;
			const bool differentRegs = (points[i*4+3] & RC_CONTOUR_REG_MASK) != (points[ii*4+3] & RC_CONTOUR_REG_MASK);
			const bool areaBorders = (points[i*4+3] & RC_AREA_BORDER) != (points[ii*4+3] & RC_AREA_BORDER);
			if (differentRegs || areaBorders)
			{
				simplified.push(points[i*4+0]);
				simplified.push(points[i*4+1]);
				simplified.push(points[i*4+2]);
				simplified.push(i);
			}
		}       
	}
	
	if (simplified.size() == 0)
	{
		// If there is no connections at all,
		// create some initial points for the simplification process. 
		// Find lower-left and upper-right vertices of the contour.
		int llx = points[0];
		int lly = points[1];
		int llz = points[2];
		int lli = 0;
		int urx = points[0];
		int ury = points[1];
		int urz = points[2];
		int uri = 0;
		for (int i = 0; i < points.size(); i += 4)
		{
			int x = points[i+0];
			int y = points[i+1];
			int z = points[i+2];
			if (x < llx || (x == llx && z < llz))
			{
				llx = x;
				lly = y;
				llz = z;
				lli = i/4;
			}
			if (x > urx || (x == urx && z > urz))
			{
				urx = x;
				ury = y;
				urz = z;
				uri = i/4;
			}
		}
		simplified.push(llx);
		simplified.push(lly);
		simplified.push(llz);
		simplified.push(lli);
		
		simplified.push(urx);
		simplified.push(ury);
		simplified.push(urz);
		simplified.push(uri);
	}
	
	// Add points until all raw points are within
	// error tolerance to the simplified shape.
	const int pn = points.size()/4;
	for (int i = 0; i < simplified.size()/4; )
	{
		int ii = (i+1) % (simplified.size()/4);
		
		const int ax = simplified[i*4+0];
		const int az = simplified[i*4+2];
		const int ai = simplified[i*4+3];
		
		const int bx = simplified[ii*4+0];
		const int bz = simplified[ii*4+2];
		const int bi = simplified[ii*4+3];

		// Find maximum deviation from the segment.
		rcReal maxd = 0;
		int maxi = -1;
		int ci, cinc, endi;
		
		// Traverse the segment in lexilogical order so that the
		// max deviation is calculated similarly when traversing
		// opposite segments.
		if (bx > ax || (bx == ax && bz > az))
		{
			cinc = 1;
			ci = (ai+cinc) % pn;
			endi = bi;
		}
		else
		{
			cinc = pn-1;
			ci = (bi+cinc) % pn;
			endi = ai;
		}
		
		// Tessellate only outer edges or edges between areas.
		if ((points[ci*4+3] & RC_CONTOUR_REG_MASK) == 0 ||
			(points[ci*4+3] & RC_AREA_BORDER))
		{
			while (ci != endi)
			{
				rcReal d = distancePtSeg(points[ci*4+0], points[ci*4+2], ax, az, bx, bz);
				if (d > maxd)
				{
					maxd = d;
					maxi = ci;
				}
				ci = (ci+cinc) % pn;
			}
		}
		
		
		// If the max deviation is larger than accepted error,
		// add new point, else continue to next segment.
		if (maxi != -1 && maxd > (maxError*maxError))
		{
			// Add space for the new point.
			simplified.resize(simplified.size()+4);
			const int n = simplified.size()/4;
			for (int j = n-1; j > i; --j)
			{
				simplified[j*4+0] = simplified[(j-1)*4+0];
				simplified[j*4+1] = simplified[(j-1)*4+1];
				simplified[j*4+2] = simplified[(j-1)*4+2];
				simplified[j*4+3] = simplified[(j-1)*4+3];
			}
			// Add the point.
			simplified[(i+1)*4+0] = points[maxi*4+0];
			simplified[(i+1)*4+1] = points[maxi*4+1];
			simplified[(i+1)*4+2] = points[maxi*4+2];
			simplified[(i+1)*4+3] = maxi;
		}
		else
		{
			++i;
		}
	}
	
	// Split too long edges.
	if (maxEdgeLen > 0 && (buildFlags & (RC_CONTOUR_TESS_WALL_EDGES|RC_CONTOUR_TESS_AREA_EDGES)) != 0)
	{
		for (int i = 0; i < simplified.size()/4; )
		{
			const int ii = (i+1) % (simplified.size()/4);
			
			const int ax = simplified[i*4+0];
			const int az = simplified[i*4+2];
			const int ai = simplified[i*4+3];
			
			const int bx = simplified[ii*4+0];
			const int bz = simplified[ii*4+2];
			const int bi = simplified[ii*4+3];

			// Find maximum deviation from the segment.
			int maxi = -1;
			int ci = (ai+1) % pn;

			// Tessellate only outer edges or edges between areas.
			bool tess = false;
			// Wall edges.
			if ((buildFlags & RC_CONTOUR_TESS_WALL_EDGES) && (points[ci*4+3] & RC_CONTOUR_REG_MASK) == 0)
				tess = true;
			// Edges between areas.
			if ((buildFlags & RC_CONTOUR_TESS_AREA_EDGES) && (points[ci*4+3] & RC_AREA_BORDER))
				tess = true;
			
			if (tess)
			{
				int dx = bx - ax;
				int dz = bz - az;
				if (dx*dx + dz*dz > maxEdgeLen*maxEdgeLen)
				{
					// Round based on the segments in lexilogical order so that the
					// max tesselation is consistent regardles in which direction
					// segments are traversed.
					const int n = bi < ai ? (bi+pn - ai) : (bi - ai);
					if (n > 1)
					{
						if (bx > ax || (bx == ax && bz > az))
							maxi = (ai + n/2) % pn;
						else
							maxi = (ai + (n+1)/2) % pn;
					}
				}
			}
			
			// If the max deviation is larger than accepted error,
			// add new point, else continue to next segment.
			if (maxi != -1)
			{
				// Add space for the new point.
				simplified.resize(simplified.size()+4);
				const int n = simplified.size()/4;
				for (int j = n-1; j > i; --j)
				{
					simplified[j*4+0] = simplified[(j-1)*4+0];
					simplified[j*4+1] = simplified[(j-1)*4+1];
					simplified[j*4+2] = simplified[(j-1)*4+2];
					simplified[j*4+3] = simplified[(j-1)*4+3];
				}
				// Add the point.
				simplified[(i+1)*4+0] = points[maxi*4+0];
				simplified[(i+1)*4+1] = points[maxi*4+1];
				simplified[(i+1)*4+2] = points[maxi*4+2];
				simplified[(i+1)*4+3] = maxi;
			}
			else
			{
				++i;
			}
		}
	}
	
	for (int i = 0; i < simplified.size()/4; ++i)
	{
		// The edge vertex flag is take from the current raw point,
		// and the neighbour region is take from the next raw point.
		const int ai = (simplified[i*4+3]+1) % pn;
		const int bi = simplified[i*4+3];
		simplified[i*4+3] = (points[ai*4+3] & (RC_CONTOUR_REG_MASK|RC_AREA_BORDER)) | (points[bi*4+3] & RC_BORDER_VERTEX);
	}
	
}

static void removeDegenerateSegments(rcIntArray& simplified)
{
	// Remove adjacent vertices which are equal on xz-plane,
	// or else the triangulator will get confused.
	for (int i = 0; i < simplified.size()/4; ++i)
	{
		int ni = i+1;
		if (ni >= (simplified.size()/4))
			ni = 0;
			
		if (simplified[i*4+0] == simplified[ni*4+0] &&
			simplified[i*4+2] == simplified[ni*4+2])
		{
			// Degenerate segment, remove.
			for (int j = i; j < simplified.size()/4-1; ++j)
			{
				simplified[j*4+0] = simplified[(j+1)*4+0];
				simplified[j*4+1] = simplified[(j+1)*4+1];
				simplified[j*4+2] = simplified[(j+1)*4+2];
				simplified[j*4+3] = simplified[(j+1)*4+3];
			}
			simplified.resize(simplified.size()-4);
		}
	}
}

static int calcAreaOfPolygon2D(const int* verts, const int nverts)
{
	int area = 0;
	for (int i = 0, j = nverts-1; i < nverts; j=i++)
	{
		const int* vi = &verts[i*4];
		const int* vj = &verts[j*4];
		area += vi[0] * vj[2] - vj[0] * vi[2];
	}
	return (area+1) / 2;
}

inline bool ileft(const int* a, const int* b, const int* c)
{
	return (b[0] - a[0]) * (c[2] - a[2]) - (c[0] - a[0]) * (b[2] - a[2]) <= 0;
}

static void getClosestIndices(const int* vertsa, const int nvertsa,
							  const int* vertsb, const int nvertsb,
							  int& ia, int& ib)
{
	int closestDist = 0xfffffff;
	ia = -1, ib = -1;
	for (int i = 0; i < nvertsa; ++i)
	{
		const int in = (i+1) % nvertsa;
		const int ip = (i+nvertsa-1) % nvertsa;
		const int* va = &vertsa[i*4];
		const int* van = &vertsa[in*4];
		const int* vap = &vertsa[ip*4];
		
		for (int j = 0; j < nvertsb; ++j)
		{
			const int* vb = &vertsb[j*4];
			// vb must be "infront" of va.
			if (ileft(vap,va,vb) && ileft(va,van,vb))
			{
				const int dx = vb[0] - va[0];
				const int dz = vb[2] - va[2];
				const int d = dx*dx + dz*dz;
				if (d < closestDist)
				{
					ia = i;
					ib = j;
					closestDist = d;
				}
			}
		}
	}
}

static bool mergeContours(rcContour& ca, rcContour& cb, int ia, int ib)
{
	check(ca.nverts > 0 && cb.nverts > 0);
	const int maxVerts = ca.nverts + cb.nverts + 2;
	int* verts = (int*)rcAlloc(sizeof(int)*maxVerts*4, RC_ALLOC_PERM);
	if (!verts)
		return false;

	int nv = 0;

	// Copy contour A.
	for (int i = 0; i <= ca.nverts; ++i)
	{
		int* dst = &verts[nv*4];
		const int* src = &ca.verts[((ia+i)%ca.nverts)*4];
		dst[0] = src[0];
		dst[1] = src[1];
		dst[2] = src[2];
		dst[3] = src[3];
		nv++;
	}

	// Copy contour B
	for (int i = 0; i <= cb.nverts; ++i)
	{
		int* dst = &verts[nv*4];
		const int* src = &cb.verts[((ib+i)%cb.nverts)*4];
		dst[0] = src[0];
		dst[1] = src[1];
		dst[2] = src[2];
		dst[3] = src[3];
		nv++;
	}
	
	rcFree(ca.verts);
	ca.verts = verts;
	ca.nverts = nv;

	rcFree(cb.verts);
	cb.verts = 0;
	cb.nverts = 0;
	
	return true;
}

/// @par
///
/// The raw contours will match the region outlines exactly. The @p maxError and @p maxEdgeLen
/// parameters control how closely the simplified contours will match the raw contours.
///
/// Simplified contours are generated such that the vertices for portals between areas match up. 
/// (They are considered mandatory vertices.)
///
/// Setting @p maxEdgeLength to zero will disable the edge length feature.
/// 
/// See the #rcConfig documentation for more information on the configuration parameters.
/// 
/// @see rcAllocContourSet, rcCompactHeightfield, rcContourSet, rcConfig
bool rcBuildContours(rcContext* ctx, rcCompactHeightfield& chf,
					 const rcReal maxError, const int maxEdgeLen,
					 rcContourSet& cset, const int buildFlags)
{
	rcAssert(ctx);
	
	const int w = chf.width;
	const int h = chf.height;
	const rcBorderSize borderSize = chf.borderSize;		//@UE
	
	ctx->startTimer(RC_TIMER_BUILD_CONTOURS);
	
	rcVcopy(cset.bmin, chf.bmin);
	rcVcopy(cset.bmax, chf.bmax);
	if (borderSize.low > 0 || borderSize.high > 0)		//@UE
	{
		// If the heightfield was build with bordersize, remove the offset.
		const rcReal pad = borderSize.low*chf.cs;		//@UE
		cset.bmin[0] += pad;
		cset.bmin[2] += pad;
		
		const rcReal padHigh = borderSize.high*chf.cs;	//@UE
		cset.bmax[0] -= padHigh;						//@UE
		cset.bmax[2] -= padHigh;						//@UE
	}
	cset.cs = chf.cs;
	cset.ch = chf.ch;
	cset.width = chf.width - (chf.borderSize.low+chf.borderSize.high);		//@UE
	cset.height = chf.height - (chf.borderSize.low+chf.borderSize.high);	//@UE
	cset.borderSize = chf.borderSize;
	
	int maxContours = rcMax((int)chf.maxRegions, 8);
	cset.conts = (rcContour*)rcAlloc(sizeof(rcContour)*maxContours, RC_ALLOC_PERM);
	if (!cset.conts)
		return false;
	cset.nconts = 0;
	
	rcScopedDelete<unsigned char> flags = (unsigned char*)rcAlloc(sizeof(unsigned char)*chf.spanCount, RC_ALLOC_TEMP);
	if (!flags)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildContours: Out of memory 'flags' (%d).", chf.spanCount);
		return false;
	}
	
	ctx->startTimer(RC_TIMER_BUILD_CONTOURS_TRACE);
	
	// Mark boundaries.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				unsigned char res = 0;
				const rcCompactSpan& s = chf.spans[i];
				if (!chf.spans[i].reg || (chf.spans[i].reg & RC_BORDER_REG))
				{
					flags[i] = 0;
					continue;
				}
				for (int dir = 0; dir < 4; ++dir)
				{
					unsigned short r = 0;
					if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
					{
						const int ax = x + rcGetDirOffsetX(dir);
						const int ay = y + rcGetDirOffsetY(dir);
						const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, dir);
						r = chf.spans[ai].reg;
					}
					if (r == chf.spans[i].reg)
						res |= (1 << dir);
				}
				flags[i] = res ^ 0xf; // Inverse, mark non connected edges.
			}
		}
	}
	
	ctx->stopTimer(RC_TIMER_BUILD_CONTOURS_TRACE);
	
	rcIntArray verts(256);
	rcIntArray simplified(64);
	
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				if (flags[i] == 0 || flags[i] == 0xf)
				{
					flags[i] = 0;
					continue;
				}
				const unsigned short reg = chf.spans[i].reg;
				if (!reg || (reg & RC_BORDER_REG))
					continue;
				const unsigned char area = chf.areas[i];
				
				verts.resize(0);
				simplified.resize(0);

				ctx->startTimer(RC_TIMER_BUILD_CONTOURS_TRACE);
				walkContour(x, y, i, chf, flags, verts);
				ctx->stopTimer(RC_TIMER_BUILD_CONTOURS_TRACE);

				ctx->startTimer(RC_TIMER_BUILD_CONTOURS_SIMPLIFY);
				simplifyContour(verts, simplified, maxError, maxEdgeLen, buildFlags);
				removeDegenerateSegments(simplified);
				ctx->stopTimer(RC_TIMER_BUILD_CONTOURS_SIMPLIFY);
				

				// Store region->contour remap info.
				// Create contour.
				if (simplified.size()/4 >= 3)
				{
					if (cset.nconts >= maxContours)
					{
						// Allocate more contours.
						// This can happen when there are tiny holes in the heightfield.
						const int oldMax = maxContours;
						maxContours *= 2;
						rcContour* newConts = (rcContour*)rcAlloc(sizeof(rcContour)*maxContours, RC_ALLOC_PERM);
						for (int j = 0; j < cset.nconts; ++j)
						{
							newConts[j] = cset.conts[j];
							// Reset source pointers to prevent data deletion.
							cset.conts[j].verts = 0;
							cset.conts[j].rverts = 0;
						}
						rcFree(cset.conts);
						cset.conts = newConts;
					
						ctx->log(RC_LOG_WARNING, "rcBuildContours: Expanding max contours from %d to %d.", oldMax, maxContours);
					}
						
					rcContour* cont = &cset.conts[cset.nconts++];
					
					cont->nverts = simplified.size()/4;
					cont->verts = (int*)rcAlloc(sizeof(int)*cont->nverts*4, RC_ALLOC_PERM);
					if (!cont->verts)
					{
						ctx->log(RC_LOG_ERROR, "rcBuildContours: Out of memory 'verts' (%d).", cont->nverts);
						return false;
					}
					memcpy(cont->verts, &simplified[0], sizeof(int)*cont->nverts*4);
					if (borderSize.low > 0)				//@UE
					{
						// If the heightfield was build with bordersize, remove the offset.
						for (int j = 0; j < cont->nverts; ++j)
						{
							int* v = &cont->verts[j*4];
							v[0] -= borderSize.low;		//@UE
							v[2] -= borderSize.low;		//@UE
						}
					}
					
					cont->nrverts = verts.size()/4;
					cont->rverts = (int*)rcAlloc(sizeof(int)*cont->nrverts*4, RC_ALLOC_PERM);
					if (!cont->rverts)
					{
						ctx->log(RC_LOG_ERROR, "rcBuildContours: Out of memory 'rverts' (%d).", cont->nrverts);
						return false;
					}
					memcpy(cont->rverts, &verts[0], sizeof(int)*cont->nrverts*4);
					if (borderSize.low > 0)				//@UE
					{
						// If the heightfield was build with bordersize, remove the offset.
						for (int j = 0; j < cont->nrverts; ++j)
						{
							int* v = &cont->rverts[j*4];
							v[0] -= borderSize.low;		//@UE
							v[2] -= borderSize.low;		//@UE
						}
					}
					
					cont->reg = reg;
					cont->area = area;
				}
			}
		}
	}
	
	// Check and merge droppings.
	// Sometimes the previous algorithms can fail and create several contours
	// per area. This pass will try to merge the holes into the main region.
	for (int i = 0; i < cset.nconts; ++i)
	{
		rcContour& cont = cset.conts[i];
		// Check if the contour is would backwards.
		if (calcAreaOfPolygon2D(cont.verts, cont.nverts) < 0)
		{
			// Find another contour which has the same region ID.
			int mergeIdx = -1;
			for (int j = 0; j < cset.nconts; ++j)
			{
				if (i == j) continue;
				if (cset.conts[j].nverts && cset.conts[j].reg == cont.reg)
				{
					// Make sure the polygon is correctly oriented.
					if (calcAreaOfPolygon2D(cset.conts[j].verts, cset.conts[j].nverts))
					{
						mergeIdx = j;
						break;
					}
				}
			}
			if (mergeIdx == -1)
			{
				ctx->log(RC_LOG_WARNING, "rcBuildContours: Could not find merge target for bad contour %d.", i);
			}
			else
			{
				rcContour& mcont = cset.conts[mergeIdx];
				// Merge by closest points.
				int ia = 0, ib = 0;
				getClosestIndices(mcont.verts, mcont.nverts, cont.verts, cont.nverts, ia, ib);
				if (ia == -1 || ib == -1)
				{
					ctx->log(RC_LOG_WARNING, "rcBuildContours: Failed to find merge points for %d and %d.", i, mergeIdx);
					continue;
				}
				if (!mergeContours(mcont, cont, ia, ib))
				{
					ctx->log(RC_LOG_WARNING, "rcBuildContours: Failed to merge contours %d and %d.", i, mergeIdx);
					continue;
				}
			}
		}
	}
	
	ctx->stopTimer(RC_TIMER_BUILD_CONTOURS);
	
	return true;
}

static void getContourCenter(const rcContour* cont, const rcReal* orig, rcReal cs, rcReal ch, rcReal* center)
{
	center[0] = 0;
	center[1] = 0;
	center[2] = 0;
	if (!cont->nverts)
		return;
	for (int i = 0; i < cont->nverts; ++i)
	{
		const int* v = &cont->verts[i*4];
		center[0] += (rcReal)v[0];
		center[1] += (rcReal)v[1];
		center[2] += (rcReal)v[2];
	}
	const rcReal s = rcReal(1.) / cont->nverts;
	center[0] *= s * cs;
	center[1] *= s * ch;
	center[2] *= s * cs;
	center[0] += orig[0];
	center[1] += orig[1] + 4*ch;
	center[2] += orig[2];
}

static int findContourFromSet(const rcContourSet& cset, unsigned short reg)
{
	for (int i = 0; i < cset.nconts; ++i)
	{
		if (cset.conts[i].reg == reg)
			return i;
	}
	return -1;
}

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
bool rcBuildClusters(rcContext* ctx, rcContourSet& cset, rcClusterSet& clusters)
{
	ctx->startTimer(RC_TIMER_BUILD_CLUSTERS);

	clusters.nclusters = cset.nconts + 1;

	clusters.center = (rcReal*)rcAlloc(sizeof(rcReal)*3*clusters.nclusters, RC_ALLOC_PERM);
	clusters.nlinks = (unsigned short*)rcAlloc(sizeof(unsigned short)*clusters.nclusters, RC_ALLOC_PERM);
	if (!clusters.center || !clusters.nlinks)
		return false;

	memset(clusters.center, 0, sizeof(rcReal)*3*clusters.nclusters);
	memset(clusters.nlinks, 0, sizeof(unsigned short)*clusters.nclusters);

	rcScopedDelete<unsigned short> clusterLinks(clusters.nclusters*clusters.nclusters);
	memset(clusterLinks, 0xffff, sizeof(unsigned short)*clusters.nclusters*clusters.nclusters);
	unsigned short nlinks = 0;

	for (int i = 1; i < clusters.nclusters; i++)
	{
		int ic = findContourFromSet(cset, (unsigned short)i);
		if (ic < 0) 
			continue;

		rcContour& cont = cset.conts[ic];
		unsigned short& nnei = clusters.nlinks[i];

		getContourCenter(&cont, cset.bmin, cset.cs, cset.ch, &clusters.center[i*3]);
		unsigned short firstLink = nlinks;

		for (int j = 0; j < cont.nverts; j++)
		{
			unsigned short neiReg = (unsigned short)cont.verts[j*4+3];
			const int neiCont = findContourFromSet(cset, neiReg);
			if (neiCont < 0) continue;

			const unsigned short neiCluster = neiReg;
			bool bFound = false;
			for (unsigned short k = firstLink; k < nlinks; k++)
			{
				if (clusterLinks[k] == neiCluster)
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				clusterLinks[nlinks] = neiCluster;
				nlinks++;
				nnei++;
			}
		}
	}

	if (nlinks)
	{
		clusters.links = (unsigned short*)rcAlloc(sizeof(unsigned short)*nlinks, RC_ALLOC_PERM);
		memcpy(clusters.links, clusterLinks, sizeof(unsigned short)*nlinks);
	}

	ctx->stopTimer(RC_TIMER_BUILD_CLUSTERS);
	return true;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END