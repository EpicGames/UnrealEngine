// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAssert.h"

/// @par
///
/// Allows the formation of walkable regions that will flow over low lying 
/// objects such as curbs, and up structures such as stairways. 
/// 
/// Two neighboring spans are walkable if: <tt>rcAbs(currentSpan.smax - neighborSpan.smax) < waklableClimb</tt>
/// 
/// @warning Will override the effect of #rcFilterLedgeSpans.  So if both filters are used, call
/// #rcFilterLedgeSpans after calling this filter. 
///
/// @see rcHeightfield, rcConfig
void rcFilterLowHangingWalkableObstacles(rcContext* ctx, const int walkableClimb, rcHeightfield& solid)
{
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_FILTER_LOW_OBSTACLES);
	
	const int w = solid.width;
	const int h = solid.height;
	
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			rcSpan* ps = 0;
			bool previousWalkable = false;
			unsigned char previousArea = RC_NULL_AREA;
			
			for (rcSpan* s = solid.spans[x + y*w]; s; ps = s, s = s->next)
			{
				const bool walkable = s->data.area != RC_NULL_AREA;
				// If current span is not walkable, but there is walkable
				// span just below it, mark the span above it walkable too.
				if (!walkable && previousWalkable)
				{
					if (rcAbs((int)s->data.smax - (int)ps->data.smax) <= walkableClimb)
						s->data.area = previousArea;
				}
				// Copy walkable flag so that it cannot propagate
				// past multiple non-walkable objects.
				previousWalkable = walkable;
				previousArea = s->data.area;
			}
		}
	}

	ctx->stopTimer(RC_TIMER_FILTER_LOW_OBSTACLES);
}

void rcFilterLedgeSpansImp(rcContext* ctx, const int walkableHeight, const int walkableClimb,
                           const rcNeighborSlopeFilterMode neighborSlopeFilterMode, const rcReal maxStepFromWalkableSlope, const rcReal ch,	// UE
                           const int filterLedgeSpansAtY, rcHeightfield& solid)
{
	rcAssert(ctx);

	const int w = solid.width;
	const int h = solid.height;
	const int MAX_HEIGHT = RC_SPAN_MAX_HEIGHT;

	const int maxStepFor2CellsVx = rcCeil(2*maxStepFromWalkableSlope / ch);	// UE

	// Mark border spans.
	for (int x = 0; x < w; ++x)
	{
		for (rcSpan* s = solid.spans[x + filterLedgeSpansAtY*w]; s; s = s->next)
		{
			// Skip non walkable spans.
			if (s->data.area == RC_NULL_AREA)
				continue;

			const int bot = (int)(s->data.smax);
			const int top = s->next ? (int)(s->next->data.smin) : MAX_HEIGHT;

			// Find neighbours minimum height.
			int minh = MAX_HEIGHT;

			// Min and max height of accessible neighbours.
			int asmin = s->data.smax;
			int asmax = s->data.smax;

			for (int dir = 0; dir < 4; ++dir)
			{
				int dx = x + rcGetDirOffsetX(dir);
				int dy = filterLedgeSpansAtY + rcGetDirOffsetY(dir);
				// Skip neighbours which are out of bounds.
				if (dx < 0 || dy < 0 || dx >= w || dy >= h)
				{
					minh = rcMin(minh, -walkableClimb - bot);
					continue;
				}

				// From minus infinity to the first span.
				rcSpan* ns = solid.spans[dx + dy*w];
				int nbot = -walkableClimb;
				int ntop = ns ? (int)ns->data.smin : MAX_HEIGHT;
				// Skip neightbour if the gap between the spans is too small.
				if (rcMin(top, ntop) - rcMax(bot, nbot) > walkableHeight)
					minh = rcMin(minh, nbot - bot);

				// Rest of the spans.
				for (ns = solid.spans[dx + dy*w]; ns; ns = ns->next)
				{
					nbot = (int)ns->data.smax;
					ntop = ns->next ? (int)ns->next->data.smin : MAX_HEIGHT;
					// Skip neightbour if the gap between the spans is too small.
					if (rcMin(top, ntop) - rcMax(bot, nbot) > walkableHeight)
					{
						minh = rcMin(minh, nbot - bot);

						// Find min/max accessible neighbour height. 
						if (neighborSlopeFilterMode != RC_SLOPE_FILTER_NONE && rcAbs(nbot - bot) <= walkableClimb)	//UE
						{
							if (nbot < asmin) asmin = nbot;
							if (nbot > asmax) asmax = nbot;
						}
					}
				}
			}

			// The current span is close to a ledge if the drop to any
			// neighbour span is less than the walkableClimb.
			if (minh < -walkableClimb)
			{
				s->data.area = RC_NULL_AREA;
			}
			// If the difference between all neighbours is too large,
			// we are at steep slope, mark the span as ledge.
//@UE BEGIN
			// This test is meant to reject step/slope combinations that can't be climbed by the agent.
			// At the span level, we detect the walkable climb height between neighbor spans. At the agent level,
			// the radius of the agent can mean that we should consider multiple neighbor spans to detect what would be
			// the actual height of the step to climb. Because of that, depending on the number of voxel represented by
			// the agent radius, the following test might cause more problems than improvements.
			// That's why we decided to allow to control it via the filterNeighborSlope parameter
			else if (neighborSlopeFilterMode == RC_SLOPE_FILTER_RECAST && (asmax - asmin) > walkableClimb)

			{
				s->data.area = RC_NULL_AREA;
			}
			// Compare the step between the min and the max with the maximum step made by a uniform slope for 2 spans (since min and max can be 2 cells apart).
			else if (neighborSlopeFilterMode == RC_SLOPE_FILTER_USE_HEIGHT_FROM_WALKABLE_SLOPE && (asmax - asmin) > maxStepFor2CellsVx)
			{
				s->data.area = RC_NULL_AREA;
			}
//@UE END			
		}
	}
}

/// @par
///
/// A ledge is a span with one or more neighbors whose maximum is further away than @p walkableClimb
/// from the current span's maximum.
/// This method removes the impact of the overestimation of conservative voxelization 
/// so the resulting mesh will not have regions hanging in the air over ledges.
/// 
/// A span is a ledge if: <tt>rcAbs(currentSpan.smax - neighborSpan.smax) > walkableClimb</tt>
/// 
/// @see rcHeightfield, rcConfig
void rcFilterLedgeSpans(rcContext* ctx, const int walkableHeight, const int walkableClimb,
						const rcNeighborSlopeFilterMode neighborSlopeFilterMode, const rcReal maxStepFromWalkableSlope, const rcReal ch,	//UE
						rcHeightfield& solid)
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_FILTER_BORDER);

	const int h = solid.height;
	
	// Mark border spans.
	for (int y = 0; y < h; ++y)
	{
		rcFilterLedgeSpansImp(ctx, walkableHeight, walkableClimb, neighborSlopeFilterMode, maxStepFromWalkableSlope, ch, y, solid);	//UE
	}

	ctx->stopTimer(RC_TIMER_FILTER_BORDER);
}	


/// @see rcHeightfield, rcConfig
void rcFilterLedgeSpans(rcContext* ctx, const int walkableHeight, const int walkableClimb,
	const rcNeighborSlopeFilterMode neighborSlopeFilterMode, const rcReal maxStepFromWalkableSlope, const rcReal ch,	// UE
	const int yStart, const int maxYProcess, //UE
	rcHeightfield& solid)
{
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_FILTER_BORDER);

	const int h = rcMin(yStart + maxYProcess, solid.height);

	for (int y = yStart; y < h; ++y)
	{
		rcFilterLedgeSpansImp(ctx, walkableHeight, walkableClimb, neighborSlopeFilterMode, maxStepFromWalkableSlope, ch, y, solid);
	}

	ctx->stopTimer(RC_TIMER_FILTER_BORDER);
}

/// @par
///
/// For this filter, the clearance above the span is the distance from the span's 
/// maximum to the next higher span's minimum. (Same grid column.)
/// 
/// @see rcHeightfield, rcConfig
void rcFilterWalkableLowHeightSpans(rcContext* ctx, int walkableHeight, rcHeightfield& solid)
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_FILTER_WALKABLE);
	
	const int w = solid.width;
	const int h = solid.height;
	const int MAX_HEIGHT = RC_SPAN_MAX_HEIGHT;
	
	// Remove walkable flag from spans which do not have enough
	// space above them for the agent to stand there.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			for (rcSpan* s = solid.spans[x + y*w]; s; s = s->next)
			{
				const int bot = (int)(s->data.smax);
				const int top = s->next ? (int)(s->next->data.smin) : MAX_HEIGHT;
				if ((top - bot) < walkableHeight)	// UE
					s->data.area = RC_NULL_AREA;
			}
		}
	}
	
	ctx->stopTimer(RC_TIMER_FILTER_WALKABLE);
}

void rcFilterWalkableLowHeightSpansSequences(rcContext* ctx, int walkableHeight, rcHeightfield& solid)
{
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_FILTER_WALKABLE);

	const int w = solid.width;
	const int h = solid.height;

	const int32 MaxSpans = 64;
	rcCompactSpan SpanList[MaxSpans];
	int32 NumSpans;
	memset(SpanList, 0, sizeof(SpanList));

	// UE: leave only single low span below valid one (null area doesn't count) or after leaving walkableHeight space between them

	// Remove walkable flag from spans which do not have enough
	// space above them for the agent to stand there.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			// build compact span list, we need to iterate from top to bottom
			NumSpans = 0;
			for (rcSpan* s = solid.spans[x + y*w]; s; s = s->next)
			{
				const rcSpanUInt bot = s->data.smax;
				const rcSpanUInt top = s->next ? s->next->data.smin : RC_SPAN_MAX_HEIGHT;
				SpanList[NumSpans].y = rcClamp(bot, 0, RC_SPAN_MAX_HEIGHT);
				SpanList[NumSpans].h = (unsigned char)rcClamp(top - bot, 0, 0xff);
				SpanList[NumSpans].reg = s->data.area;
				
				NumSpans++;
				if (NumSpans >= MaxSpans)
				{
					break;
				}
			}

			rcSpanUInt NextAllowedBase = RC_SPAN_MAX_HEIGHT;
			for (int32 Idx = NumSpans - 1; Idx >= 0; Idx--)
			{
				if (SpanList[Idx].h < walkableHeight)
				{
					if (SpanList[Idx].y < NextAllowedBase)
					{
						NextAllowedBase = rcMax(0, (int)SpanList[Idx].y - walkableHeight);
					}
					else
					{
						SpanList[Idx].reg = RC_NULL_AREA;
					}
				}
				else if (SpanList[Idx].reg != RC_NULL_AREA)
				{
					NextAllowedBase = SpanList[Idx].y;
				}
			}

			int32 SpanIdx = 0;
			for (rcSpan* s = solid.spans[x + y*w]; s; s = s->next)
			{
				if (SpanIdx < MaxSpans)
				{
					s->data.area = SpanList[SpanIdx].reg;
				}

				SpanIdx++;
			}
		}
	}

	ctx->stopTimer(RC_TIMER_FILTER_WALKABLE);
}
