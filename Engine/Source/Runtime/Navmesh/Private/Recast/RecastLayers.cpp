// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAlloc.h"
#include "Recast/RecastAssert.h"

struct rcLayerRegionMonotone
{
	int chunkId;
	rcIntArray neis;
	rcIntArray layers;
	rcSpanUInt ymin, ymax;
	unsigned short layerId;		// Layer ID
	unsigned char base : 1;		// Flag indicating if the region is the base of merged regions.
	unsigned char remap : 1;
};

static void rcFreeLayerRegionMonotones(rcLayerRegionMonotone* regs, int nregs)
{
	// destroy all elements to free internal rcIntArray allocations
	for (int i = 0; i < nregs; i++)
	{
		regs[i].~rcLayerRegionMonotone();
	}

	rcFree(regs);
}

static void addUnique(rcIntArray& a, int v)
{
	if (!a.contains(v))
	{
		a.push(v);
	}
}

inline bool overlapRange(const rcSpanUInt amin, const rcSpanUInt amax,
						 const rcSpanUInt bmin, const rcSpanUInt bmax)
{
	return (amin > bmax || amax < bmin) ? false : true;
}

static void fixLayerConnections(rcHeightfieldLayer* layer)
{
	// [UE: break one directional connections, contour tracing gets stuck in infinite loop]
	const int lw = layer->width;
	const int lh = layer->height;

	for (int y = 0; y < lh; ++y)
	{
		for (int x = 0; x < lw; ++x)
		{
			const int idx = x + y*lw;
			const int con = layer->cons[idx];

			for (int dir = 0; dir < 4; ++dir)
			{
				if ((con & (1 << dir)) == 0)
				{
					const int nx = x + rcGetDirOffsetX(dir);
					const int ny = y + rcGetDirOffsetY(dir);
					if (nx >= 0 && ny >= 0 && nx < lw && ny < lh)
					{
						const int nidx = nx + ny*lw;
						const int oppDir = (dir + 2) % 4;
						layer->cons[nidx] &= ~(1 << oppDir);
					}
				}
			}
		}
	}
}

struct rcLayerSweepSpan
{
	unsigned short ns;	// number samples
	unsigned short id;	// region id
	unsigned short nei;	// neighbour id
};

static bool CollectLayerRegionsMonotone(rcContext* ctx, rcCompactHeightfield& chf, const rcBorderSize borderSize,	//@UE
	unsigned short* srcReg, rcLayerRegionMonotone*& regs, int& nregs)
{
	const int w = chf.width;
	const int h = chf.height;

	// assume 8 unique layers on each place along row
	const int32 MaxSweeps = w * 8;

	rcScopedDelete<rcLayerSweepSpan> sweeps(MaxSweeps);
	if (!sweeps)
	{
		ctx->log(RC_LOG_ERROR, "CollectLayerRegionsMonotone: Out of memory 'sweeps' (%d).", MaxSweeps);
		return false;
	}

	// Partition walkable area into monotone regions.
	rcIntArray prev(256);
	unsigned short regId = 0;

	for (int y = borderSize.low; y < h - borderSize.high; ++y)		//@UE
	{
		prev.resize(regId + 1);
		memset(&prev[0], 0, sizeof(int)*regId);
		unsigned short sweepId = 0;
		unsigned int MaxSpanCount = 0;

		for (int x = borderSize.low; x < w - borderSize.high; ++x)		//@UE
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			MaxSpanCount = rcMax(MaxSpanCount, c.count);

			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				if (chf.areas[i] == RC_NULL_AREA) continue;

				unsigned short sid = 0xffff;

				// -x
				if (rcGetCon(s, 0) != RC_NOT_CONNECTED)
				{
					const int ax = x + rcGetDirOffsetX(0);
					const int ay = y + rcGetDirOffsetY(0);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 0);

					if (chf.areas[ai] != RC_NULL_AREA && srcReg[ai] != 0xffff)
						sid = srcReg[ai];
				}

				if (sid == 0xffff)
				{
					sid = sweepId++;
					if (sid < MaxSweeps)
					{
						sweeps[sid].nei = 0xffff;
						sweeps[sid].ns = 0;
					}
					else
					{
						ctx->log(RC_LOG_ERROR, "CollectLayerRegionsMonotone: Layer split is too complex, skipping tile! x:%d y:%d spansTotal:%d spansCurrent:%d spansMax:%d", x, y, chf.spanCount, c.count, MaxSpanCount);
						return false;
					}
				}

				// -y
				if (rcGetCon(s, 3) != RC_NOT_CONNECTED)
				{
					const int ax = x + rcGetDirOffsetX(3);
					const int ay = y + rcGetDirOffsetY(3);
					const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, 3);

					const unsigned short nr = srcReg[ai];
					if (nr != 0xffff)
					{
						// Set neighbour when first valid neighbour is encoutered.
						if (sweeps[sid].ns == 0)
							sweeps[sid].nei = nr;

						if (sweeps[sid].nei == nr)
						{
							// Update existing neighbour
							sweeps[sid].ns++;
							prev[nr]++;
						}
						else
						{
							// This is hit if there is nore than one neighbour.
							// Invalidate the neighbour.
							sweeps[sid].nei = 0xffff;
						}
					}
				}

				srcReg[i] = sid;
			}
		}

		// Create unique ID.
		for (int i = 0; i < sweepId; ++i)
		{
			// If the neighbour is set and there is only one continuous connection to it,
			// the sweep will be merged with the previous one, else new region is created.
			if (sweeps[i].nei != 0xffff && prev[sweeps[i].nei] == sweeps[i].ns)
			{
				sweeps[i].id = sweeps[i].nei;
			}
			else
			{
				sweeps[i].id = regId++;
			}
		}

		// Remap local sweep ids to region ids.
		for (int x = borderSize.low; x < w - borderSize.high; ++x)	//@UE
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				if (srcReg[i] != 0xffff)
					srcReg[i] = sweeps[srcReg[i]].id;
			}
		}
	}

	// Allocate and init layer regions.
	nregs = (int)regId;

// @UE BEGIN: special handling of "no regions"
	if (nregs == 0)
	{
		regs = 0;
		// treating this as success because we successfully generated 0 regions,
		// no issues occurred, everything was good. Just no regions. 
		return true;
	}
// @UE END

	regs = (rcLayerRegionMonotone*)rcAlloc(sizeof(rcLayerRegionMonotone)*nregs, RC_ALLOC_TEMP);
	if (!regs)
	{
		ctx->log(RC_LOG_ERROR, "CollectLayerRegionsMonotone: Out of memory 'regs' (%d).", nregs);
		return false;
	}
	memset((void*)regs, 0, sizeof(rcLayerRegionMonotone)*nregs);
	for (int i = 0; i < nregs; ++i)
	{
		regs[i].layerId = 0xffff;
		regs[i].ymin = RC_SPAN_MAX_HEIGHT;
		regs[i].ymax = 0;
	}

	rcIntArray lregs(64);

	// Find region neighbours and overlapping regions.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x + y*w];
			lregs.resize(0);

			for (int i = (int)c.index, ni = (int)(c.index + c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				const unsigned short ri = srcReg[i];
				if (ri == 0xffff) continue;

				regs[ri].ymin = rcMin(regs[ri].ymin, s.y);
				regs[ri].ymax = rcMax(regs[ri].ymax, s.y);

				// Collect all region layers.
				lregs.push(ri);

				// Update neighbours
				for (int dir = 0; dir < 4; ++dir)
				{
					if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
					{
						const int ax = x + rcGetDirOffsetX(dir);
						const int ay = y + rcGetDirOffsetY(dir);
						const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, dir);

						const unsigned short rai = srcReg[ai];
						if (rai != 0xffff && rai != ri)
							addUnique(regs[ri].neis, rai);
					}
				}

			}

			// Update overlapping regions.
			const int nlregs = lregs.size();
			for (int i = 0; i < nlregs - 1; ++i)
			{
				for (int j = i + 1; j < nlregs; ++j)
				{
					if (lregs[i] != lregs[j])
					{
						rcLayerRegionMonotone& ri = regs[lregs[i]];
						rcLayerRegionMonotone& rj = regs[lregs[j]];
						addUnique(ri.layers, lregs[j]);
						addUnique(rj.layers, lregs[i]);
					}
				}
			}
		}
	}

	return true;
}

static bool CollectLayerRegionsChunky(rcContext* ctx, rcCompactHeightfield& chf,
									  const rcBorderSize borderSize, const int chunkSize,		//@UE
									  unsigned short* srcReg, rcLayerRegionMonotone*& regs, int& nregs)
{
	const int w = chf.width;
	const int h = chf.height;

	rcScopedDelete<rcLayerSweepSpan> sweeps(chunkSize);
	if (!sweeps)
	{
		ctx->log(RC_LOG_ERROR, "CollectLayerRegionsChunky: Out of memory 'sweeps' (%d).", chunkSize);
		return false;
	}

	// Partition walkable area into monotone regions.
	rcIntArray prev(256);
	unsigned short regId = 0;

	for (int chunkx = borderSize.low; chunkx < w-borderSize.high; chunkx += chunkSize)		//@UE
	{
		for (int chunky = borderSize.low; chunky < h-borderSize.high; chunky += chunkSize)		//@UE
		{
			const int maxx = rcMin(chunkx + chunkSize, w-borderSize.high);		//@UE
			const int maxy = rcMin(chunky + chunkSize, h-borderSize.high);		//@UE

			for (int y = chunky; y < maxy; ++y)
			{
				prev.resize(regId+1);
				memset(&prev[0],0,sizeof(int)*regId);
				unsigned short sweepId = 0;

				for (int x = chunkx; x < maxx; ++x)
				{
					const rcCompactCell& c = chf.cells[x+y*w];

					for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
					{
						const rcCompactSpan& s = chf.spans[i];
						if (chf.areas[i] == RC_NULL_AREA) continue;

						unsigned short sid = 0xffff;

						// -x
						if (rcGetCon(s, 0) != RC_NOT_CONNECTED && x > chunkx)
						{
							const int ax = x + rcGetDirOffsetX(0);
							const int ay = y + rcGetDirOffsetY(0);
							const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 0);
							if (chf.areas[ai] != RC_NULL_AREA && srcReg[ai] != 0xffff)
								sid = srcReg[ai];
						}

						if (sid == 0xffff)
						{
							sid = sweepId++;
							// UE: multiple spans per single X row may result in more sweeps than originally allocated
							if (sweeps.resizeGrow(sid + 1))
							{
								sweeps[sid].nei = 0xffff;
								sweeps[sid].ns = 0;
							}
							else
							{
								ctx->log(RC_LOG_ERROR, "CollectLayerRegionsChunky: Out of memory 'sweeps' resize (%d).", sid + 1);
								return false;
							}
						}

						// -y
						if (rcGetCon(s,3) != RC_NOT_CONNECTED && y > chunky)
						{
							const int ax = x + rcGetDirOffsetX(3);
							const int ay = y + rcGetDirOffsetY(3);
							const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 3);
							const unsigned short nr = srcReg[ai];
							if (nr != 0xffff)
							{
								// Set neighbour when first valid neighbour is encoutered.
								if (sweeps[sid].ns == 0)
									sweeps[sid].nei = nr;

								if (sweeps[sid].nei == nr)
								{
									// Update existing neighbour
									sweeps[sid].ns++;
									prev[nr]++;
								}
								else
								{
									// This is hit if there is nore than one neighbour.
									// Invalidate the neighbour.
									sweeps[sid].nei = 0xffff;
								}
							}
						}

						srcReg[i] = sid;
					}
				}

				// Create unique ID.
				for (int i = 0; i < sweepId; ++i)
				{
					// If the neighbour is set and there is only one continuous connection to it,
					// the sweep will be merged with the previous one, else new region is created.
					if (sweeps[i].nei != 0xffff && prev[sweeps[i].nei] == sweeps[i].ns)
					{
						sweeps[i].id = sweeps[i].nei;
					}
					else
					{
						sweeps[i].id = regId++;
					}
				}

				// Remap local sweep ids to region ids.
				for (int x = chunkx; x < maxx; ++x)
				{
					const rcCompactCell& c = chf.cells[x+y*w];
					for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
					{
						if (srcReg[i] != 0xffff)
							srcReg[i] = sweeps[srcReg[i]].id;
					}
				}
			}
		}
	}

	// Allocate and init layer regions.
	nregs = (int)regId;

// @UE BEGIN: special handling of "no regions"
	if (nregs == 0)
	{
		regs = 0;
		// treating this as success because we successfully generated 0 regions,
		// no issues occurred, everything was good. Just no regions. 
		return true;
	}
// @UE END

	regs = (rcLayerRegionMonotone*)rcAlloc(sizeof(rcLayerRegionMonotone)*nregs, RC_ALLOC_TEMP);
	if (!regs)
	{
		ctx->log(RC_LOG_ERROR, "CollectLayerRegionsChunky: Out of memory 'regs' (%d).", nregs);
		return false;
	}
	memset((void*)regs, 0, sizeof(rcLayerRegionMonotone)*nregs);
	for (int i = 0; i < nregs; ++i)
	{
		regs[i].layerId = 0xffff;
		regs[i].ymin = RC_SPAN_MAX_HEIGHT;
		regs[i].ymax = 0;
	}

	rcIntArray lregs(64);

	// Find region neighbours and overlapping regions.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			lregs.resize(0);

			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				const unsigned short ri = srcReg[i];
				if (ri == 0xffff) continue;

				regs[ri].ymin = rcMin(regs[ri].ymin, s.y);
				regs[ri].ymax = rcMax(regs[ri].ymax, s.y);
				regs[ri].chunkId = (x / chunkSize) + (y / chunkSize) * chunkSize;

				// Collect all region layers.
				lregs.push(ri);

				// Update neighbours
				for (int dir = 0; dir < 4; ++dir)
				{
					if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
					{
						const int ax = x + rcGetDirOffsetX(dir);
						const int ay = y + rcGetDirOffsetY(dir);
						const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, dir);
						const unsigned short rai = srcReg[ai];
						if (rai != 0xffff && rai != ri)
							addUnique(regs[ri].neis, rai);
					}
				}

			}

			// Update overlapping regions.
			const int nlregs = lregs.size();
			for (int i = 0; i < nlregs-1; ++i)
			{
				for (int j = i+1; j < nlregs; ++j)
				{
					if (lregs[i] != lregs[j])
					{
						rcLayerRegionMonotone& ri = regs[lregs[i]];
						rcLayerRegionMonotone& rj = regs[lregs[j]];
						addUnique(ri.layers, lregs[j]);
						addUnique(rj.layers, lregs[i]);
					}
				}
			}
		}
	}

	return true;
}

static bool SplitAndStoreLayerRegions(rcContext* ctx, rcCompactHeightfield& chf,
									  const rcBorderSize borderSize, const int walkableHeight,		//@UE
									  unsigned short* srcReg, rcLayerRegionMonotone* regs, const int nregs,
									  rcHeightfieldLayerSet& lset)
{
	// Create 2D layers from regions.
	unsigned short layerId = 0;

	rcIntArray stack(64);
	stack.resize(0);

	for (int i = 0; i < nregs; ++i)
	{
		rcLayerRegionMonotone& root = regs[i];
		// Skip already visited.
		if (root.layerId != 0xffff)
			continue;

		// Start search.
		root.layerId = layerId;
		root.base = 1;
		stack.push(i);

		while (stack.size())
		{
			// Pop front
			rcLayerRegionMonotone& reg = regs[stack[0]];
			for (int j = 1; j < stack.size(); ++j)
				stack[j - 1] = stack[j];
			stack.pop();

			const int nneis = (int)reg.neis.size();
			for (int j = 0; j < nneis; ++j)
			{
				const int nei = reg.neis[j];
				rcLayerRegionMonotone& regn = regs[nei];
				// Skip already visited.
				if (regn.layerId != 0xffff)
					continue;
				// Skip if the neighbour is overlapping root region.
				if (root.layers.contains(nei))
					continue;
				// Skip if the height range would become too large.
				const rcSpanUInt ymin = rcMin(root.ymin, regn.ymin);
				const rcSpanUInt ymax = rcMax(root.ymax, regn.ymax);
				if (((int)ymax - (int)ymin) >= 255)
					continue;

				// Deepen
				stack.push(nei);

				// Mark layer id
				regn.layerId = layerId;
				// Merge current layers to root.
				for (int k = 0; k < regn.layers.size(); ++k)
					addUnique(root.layers, regn.layers[k]);
				root.ymin = rcMin(root.ymin, regn.ymin);
				root.ymax = rcMax(root.ymax, regn.ymax);
			}
		}

		layerId++;
	}

	// Merge non-overlapping regions that are close in height.
	const unsigned short mergeHeight = (unsigned short)walkableHeight * 4;

	for (int i = 0; i < nregs; ++i)
	{
		rcLayerRegionMonotone& ri = regs[i];
		if (!ri.base) continue;

		unsigned short newId = ri.layerId;

		for (;;)
		{
			unsigned short oldId = 0xffff;

			for (int j = 0; j < nregs; ++j)
			{
				if (i == j) continue;
				rcLayerRegionMonotone& rj = regs[j];
				if (!rj.base) continue;

				// Skip if the regions are not close to each other.
				if (!overlapRange(ri.ymin,ri.ymax+mergeHeight, rj.ymin,rj.ymax+mergeHeight))
					continue;
				// Skip if the height range would become too large.
				const rcSpanUInt ymin = rcMin(ri.ymin, rj.ymin);
				const rcSpanUInt ymax = rcMax(ri.ymax, rj.ymax);
				if (((int)ymax - (int)ymin) >= 255)
					continue;

				// Make sure that there is no overlap when mergin 'ri' and 'rj'.
				bool overlap = false;
				// Iterate over all regions which have the same layerId as 'rj'
				for (int k = 0; k < nregs; ++k)
				{
					if (regs[k].layerId != rj.layerId)
						continue;
					// Check if region 'k' is overlapping region 'ri'
					// Index to 'regs' is the same as region id.
					if (ri.layers.contains(k))
					{
						overlap = true;
						break;
					}
				}
				// Cannot merge of regions overlap.
				if (overlap)
					continue;

				// Can merge i and j.
				oldId = rj.layerId;
				break;
			}

			// Could not find anything to merge with, stop.
			if (oldId == 0xffff)
				break;

			// Merge
			for (int j = 0; j < nregs; ++j)
			{
				rcLayerRegionMonotone& rj = regs[j];
				if (rj.layerId == oldId)
				{
					rj.base = 0;
					// Remap layerIds.
					rj.layerId = newId;
					// Add overlaid layers from 'rj' to 'ri'.
					for (int k = 0; k < rj.layers.size(); ++k)
						addUnique(ri.layers, rj.layers[k]);
					// Update heigh bounds.
					ri.ymin = rcMin(ri.ymin, rj.ymin);
					ri.ymax = rcMax(ri.ymax, rj.ymax);
				}
			}
		}
	}

	// Compact layerIds
	layerId = 0;
	if (nregs < 256)
	{
		// Compact ids.
		unsigned short remap[256];
		memset(remap, 0, sizeof(unsigned short)*256);
		// Find number of unique regions.
		for (int i = 0; i < nregs; ++i)
			remap[regs[i].layerId] = 1;
		for (int i = 0; i < 256; ++i)
			if (remap[i])
				remap[i] = layerId++;
		// Remap ids.
		for (int i = 0; i < nregs; ++i)
			regs[i].layerId = remap[regs[i].layerId];
	}
	else
	{
		for (int i = 0; i < nregs; ++i)
			regs[i].remap = true;

		for (int i = 0; i < nregs; ++i)
		{
			if (!regs[i].remap)
				continue;
			unsigned short oldId = regs[i].layerId;
			unsigned short newId = ++layerId;
			for (int j = i; j < nregs; ++j)
			{
				if (regs[j].remap && regs[j].layerId == oldId)
				{
					regs[j].layerId = newId;
					regs[j].remap = false;
				}
			}
		}
	}

	// No layers, return empty.
	if (layerId == 0)
	{
		ctx->stopTimer(RC_TIMER_BUILD_LAYERS);
		return true;
	}

	// Create layers.
	rcAssert(lset.layers == 0);

	const int w = chf.width;
	const int h = chf.height;
	const int lw = w - (borderSize.low + borderSize.high);		//@UE
	const int lh = h - (borderSize.low + borderSize.high);		//@UE

	// Build contracted bbox for layers.
	rcReal bmin[3], bmax[3];
	rcVcopy(bmin, chf.bmin);
	rcVcopy(bmax, chf.bmax);
	bmin[0] += borderSize.low*chf.cs;		//@UE
	bmin[2] += borderSize.low*chf.cs;		//@UE
	bmax[0] -= borderSize.high*chf.cs;		//@UE
	bmax[2] -= borderSize.high*chf.cs;		//@UE

	lset.nlayers = (int)layerId;

	lset.layers = (rcHeightfieldLayer*)rcAlloc(sizeof(rcHeightfieldLayer)*lset.nlayers, RC_ALLOC_PERM);
	if (!lset.layers)
	{
		ctx->log(RC_LOG_ERROR, "SplitAndStoreLayerRegions: Out of memory 'layers' (%d).", lset.nlayers);
		return false;
	}
	memset(lset.layers, 0, sizeof(rcHeightfieldLayer)*lset.nlayers);


	// Store layers.
	for (int i = 0; i < lset.nlayers; ++i)
	{
		unsigned short curId = (unsigned short)i;

		// Allocate memory for the current layer.
		rcHeightfieldLayer* layer = &lset.layers[i];
		memset(layer, 0, sizeof(rcHeightfieldLayer));

		const int gridSize = sizeof(unsigned char)*lw*lh;
		const int gridSize2 = sizeof(unsigned short)*lw*lh;

		layer->heights = (unsigned short*)rcAlloc(gridSize2, RC_ALLOC_PERM);
		if (!layer->heights)
		{
			ctx->log(RC_LOG_ERROR, "SplitAndStoreLayerRegions: Out of memory 'heights' (%d).", gridSize2);
			return false;
		}
		memset(layer->heights, 0xff, gridSize2);

		layer->areas = (unsigned char*)rcAlloc(gridSize, RC_ALLOC_PERM);
		if (!layer->areas)
		{
			ctx->log(RC_LOG_ERROR, "SplitAndStoreLayerRegions: Out of memory 'areas' (%d).", gridSize);
			return false;
		}
		memset(layer->areas, 0, gridSize);

		layer->cons = (unsigned char*)rcAlloc(gridSize, RC_ALLOC_PERM);
		if (!layer->cons)
		{
			ctx->log(RC_LOG_ERROR, "SplitAndStoreLayerRegions: Out of memory 'cons' (%d).", gridSize);
			return false;
		}
		memset(layer->cons, 0, gridSize);

		// Find layer height bounds.
		rcSpanUInt hmin = 0, hmax = 0;
		for (int j = 0; j < nregs; ++j)
		{
			if (regs[j].base && regs[j].layerId == curId)
			{
				hmin = regs[j].ymin;
				hmax = regs[j].ymax;
			}
		}

		layer->width = lw;
		layer->height = lh;
		layer->cs = chf.cs;
		layer->ch = chf.ch;

		// Adjust the bbox to fit the heighfield.
		rcVcopy(layer->bmin, bmin);
		rcVcopy(layer->bmax, bmax);
		layer->bmin[1] = bmin[1] + hmin*chf.ch;
		layer->bmax[1] = bmin[1] + hmax*chf.ch;

		// Update usable data region.
		layer->minx = layer->width;
		layer->maxx = 0;
		layer->miny = layer->height;
		layer->maxy = 0;

		// Copy height and area from compact heighfield. 
		for (int y = 0; y < lh; ++y)
		{
			for (int x = 0; x < lw; ++x)
			{
				const int cx = borderSize.low+x;	//@UE
				const int cy = borderSize.low+y;	//@UE
				const rcCompactCell& c = chf.cells[cx+cy*w];
				for (int j = (int)c.index, nj = (int)(c.index+c.count); j < nj; ++j)
				{
					const rcCompactSpan& s = chf.spans[j];
					// Skip unassigned regions.
					if (srcReg[j] == 0xffff)
						continue;
					// Skip of does nto belong to current layer.
					unsigned short lid = regs[srcReg[j]].layerId;
					if (lid != curId)
						continue;

					// Update data bounds.
					layer->minx = rcMin(layer->minx, x);
					layer->maxx = rcMax(layer->maxx, x);
					layer->miny = rcMin(layer->miny, y);
					layer->maxy = rcMax(layer->maxy, y);

					// Store height and area type.
					const int idx = x+y*lw;
					layer->heights[idx] = (unsigned short)(s.y - hmin);
					layer->areas[idx] = chf.areas[j];

					// Check connection.
					unsigned char portal = 0;
					unsigned char con = 0;
					for (int dir = 0; dir < 4; ++dir)
					{
						if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
						{
							const int ax = cx + rcGetDirOffsetX(dir);
							const int ay = cy + rcGetDirOffsetY(dir);
							const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, dir);
							unsigned short alid = srcReg[ai] != 0xffff ? regs[srcReg[ai]].layerId : 0xffff;
							// Portal mask
							if (chf.areas[ai] != RC_NULL_AREA && lid != alid)
							{
								portal |= (unsigned char)(1<<dir);
								// Update height so that it matches on both sides of the portal.
								const rcCompactSpan& as = chf.spans[ai];
								if (as.y > hmin)
									layer->heights[idx] = rcMax(layer->heights[idx], (unsigned short)(as.y - hmin));
							}
							// Valid connection mask
							if (chf.areas[ai] != RC_NULL_AREA && lid == alid)
							{
								const int nx = ax - borderSize.low;		//@UE
								const int ny = ay - borderSize.low;		//@UE
								if (nx >= 0 && ny >= 0 && nx < lw && ny < lh)
								{
									con |= (unsigned char)(1 << dir);
								}
							}
						}
					}

					layer->cons[idx] |= (portal << 4) | con;
				}
			}
		}

		fixLayerConnections(layer);

		if (layer->minx > layer->maxx)
			layer->minx = layer->maxx = 0;
		if (layer->miny > layer->maxy)
			layer->miny = layer->maxy = 0;
	}

	return true;
}


/// @par
/// 
/// See the #rcConfig documentation for more information on the configuration parameters.
/// 
/// @see rcAllocHeightfieldLayerSet, rcCompactHeightfield, rcHeightfieldLayerSet, rcConfig
bool rcBuildHeightfieldLayersMonotone(rcContext* ctx, rcCompactHeightfield& chf,
							  const rcBorderSize borderSize, const int walkableHeight,		//@UE
							  rcHeightfieldLayerSet& lset)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_BuildHeightfieldLayersMonotone);
	
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_BUILD_LAYERS);
	
	rcScopedDelete<unsigned short> srcReg = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount, RC_ALLOC_TEMP);
	if (!srcReg)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildHeightfieldLayers: Out of memory 'srcReg' (%d).", chf.spanCount);
		return false;
	}
	memset(srcReg,0xff,sizeof(unsigned short)*chf.spanCount);
	
	rcLayerRegionMonotone* regs = NULL;
	int nregs = 0;
	
	const bool bHasRegions = CollectLayerRegionsMonotone(ctx, chf, borderSize, srcReg, regs, nregs);
	if (!bHasRegions)
	{
		// no allocations yet, but just to be safe...
		rcFreeLayerRegionMonotones(regs, nregs);
		return false;
	}

	const bool bHasSaved = SplitAndStoreLayerRegions(ctx, chf, borderSize, walkableHeight, srcReg, regs, nregs, lset);
	rcFreeLayerRegionMonotones(regs, nregs);

	if (!bHasSaved)
	{
		return false;
	}
	
	ctx->stopTimer(RC_TIMER_BUILD_LAYERS);
	
	return true;
}

bool rcBuildHeightfieldLayersChunky(rcContext* ctx, rcCompactHeightfield& chf,
									const rcBorderSize borderSize, const int walkableHeight,	//@UE
									const int chunkSize,
									rcHeightfieldLayerSet& lset)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_BuildHeightfieldLayersChunky);
	
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_BUILD_LAYERS);

	rcScopedDelete<unsigned short> srcReg = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount, RC_ALLOC_TEMP);
	if (!srcReg)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildHeightfieldLayers: Out of memory 'srcReg' (%d).", chf.spanCount);
		return false;
	}
	memset(srcReg,0xff,sizeof(unsigned short)*chf.spanCount);

	rcLayerRegionMonotone* regs = NULL;
	int nregs = 0;

	const bool bHasRegions = CollectLayerRegionsChunky(ctx, chf, borderSize, chunkSize, srcReg, regs, nregs);
	if (!bHasRegions)
	{
		// no allocations yet, but just to be safe...
		rcFreeLayerRegionMonotones(regs, nregs);
		return false;
	}

	const bool bHasSaved = SplitAndStoreLayerRegions(ctx, chf, borderSize, walkableHeight, srcReg, regs, nregs, lset);
	rcFreeLayerRegionMonotones(regs, nregs);

	if (!bHasSaved)
	{
		return false;
	}

	ctx->stopTimer(RC_TIMER_BUILD_LAYERS);

	return true;
}

/// helper function from RecastRegion.cpp, requires distance data in compact height field
bool rcGatherRegionsNoFilter(rcContext* ctx, rcCompactHeightfield& chf, const rcBorderSize borderSize, unsigned short* spanBuf4, unsigned short*& resultBufPtr);	//@UE

struct rcLayerRegion
{
	rcIntArray layers;
	rcIntArray connections;
	unsigned short layerId;
	rcSpanUInt ymin, ymax;
	unsigned char remap : 1;
	unsigned char visited : 1;
	unsigned char base : 1;
	unsigned char hasSpans : 1;
};

static void addUniqueLayerRegion(rcLayerRegion& reg, int n)
{
	if (!reg.layers.contains(n))
	{
		reg.layers.push(n);
	}
}

static bool isSolidEdge(rcCompactHeightfield& chf, unsigned short* srcReg, int x, int y, int i, int dir)
{
	const rcCompactSpan& s = chf.spans[i];
	unsigned short r = 0;
	if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
	{
		const int ax = x + rcGetDirOffsetX(dir);
		const int ay = y + rcGetDirOffsetY(dir);
		const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(s, dir);
		r = srcReg[ai];
	}
	if (r == srcReg[i])
		return false;
	return true;
}

static void walkContour(int x, int y, int i, int dir, rcCompactHeightfield& chf, unsigned short* srcReg, rcIntArray& cont)
{
	int startDir = dir;
	int starti = i;

	const rcCompactSpan& ss = chf.spans[i];
	unsigned short curReg = 0;
	if (rcGetCon(ss, dir) != RC_NOT_CONNECTED)
	{
		const int ax = x + rcGetDirOffsetX(dir);
		const int ay = y + rcGetDirOffsetY(dir);
		const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(ss, dir);
		curReg = srcReg[ai];
	}
	cont.push(curReg);

	int iter = 0;
	while (++iter < 40000)
	{
		const rcCompactSpan& s = chf.spans[i];

		if (isSolidEdge(chf, srcReg, x, y, i, dir))
		{
			// Choose the edge corner
			unsigned short r = 0;
			if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
			{
				const int ax = x + rcGetDirOffsetX(dir);
				const int ay = y + rcGetDirOffsetY(dir);
				const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(s, dir);
				r = srcReg[ai];
			}
			if (r != curReg)
			{
				curReg = r;
				cont.push(curReg);
			}

			dir = (dir+1) & 0x3;  // Rotate CW
		}
		else
		{
			int ni = -1;
			const int nx = x + rcGetDirOffsetX(dir);
			const int ny = y + rcGetDirOffsetY(dir);
			if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
			{
				const rcCompactCell& nc = chf.cells[nx+ny*chf.width];
				ni = (int)nc.index + rcGetCon(s, dir);
			}
			if (ni == -1)
			{
				// Should not happen.
				return;
			}
			x = nx;
			y = ny;
			i = ni;
			dir = (dir+3) & 0x3;	// Rotate CCW
		}

		if (starti == i && startDir == dir)
		{
			break;
		}
	}

	// Remove adjacent duplicates.
	if (cont.size() > 1)
	{
		for (int j = 0; j < cont.size(); )
		{
			int nj = (j+1) % cont.size();
			if (cont[j] == cont[nj])
			{
				for (int k = j; k < cont.size()-1; ++k)
					cont[k] = cont[k+1];
				cont.pop();
			}
			else
				++j;
		}
	}
}

/// @par
/// 
/// See the #rcConfig documentation for more information on the configuration parameters.
/// 
/// @see rcAllocHeightfieldLayerSet, rcCompactHeightfield, rcHeightfieldLayerSet, rcConfig
bool rcBuildHeightfieldLayers(rcContext* ctx, rcCompactHeightfield& chf,
	const rcBorderSize borderSize, const int walkableHeight,		//@UE
	rcHeightfieldLayerSet& lset)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_BuildHeightfieldLayers);
	
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_BUILD_LAYERS);

	rcScopedDelete<unsigned short> spanBuf4 = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount*4, RC_ALLOC_TEMP);
	if (!spanBuf4)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildHeightfieldLayers: Out of memory 'spanBuf4' (%d).", chf.spanCount*4);
		return false;
	}

	ctx->startTimer(RC_TIMER_BUILD_REGIONS_WATERSHED);

	unsigned short* srcReg = spanBuf4;
	// srcReg might be updated and point at a different location.
	if (!rcGatherRegionsNoFilter(ctx, chf, borderSize, spanBuf4, srcReg))	//@UE
		return false;

	ctx->stopTimer(RC_TIMER_BUILD_REGIONS_WATERSHED);
	ctx->startTimer(RC_TIMER_BUILD_REGIONS_FILTER);

	const int w = chf.width;
	const int h = chf.height;
	const int nreg = chf.maxRegions + 1;
	rcScopedStructArrayDelete<rcLayerRegion> regions(nreg);
	if (!regions)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildHeightfieldLayers: Out of memory 'regions' (%d).", nreg);
		return false;
	}

	// Construct regions
	memset((void*)regions, 0, sizeof(rcLayerRegion)*nreg);
	for (int i = 0; i < nreg; ++i)
	{
		regions[i].layerId = (unsigned short)i;
		regions[i].ymax = 0;
		regions[i].ymin = RC_SPAN_MAX_HEIGHT;
	}

	// Find region neighbours and overlapping regions.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				const unsigned short ri = srcReg[i];
				if (ri == 0 || ri >= nreg)
					continue;

				rcLayerRegion& reg = regions[ri];
				reg.ymin = rcMin(reg.ymin, s.y);
				reg.ymax = rcMax(reg.ymax, s.y);
				reg.hasSpans = true;

				// Collect all region layers.
				for (int j = (int)c.index; j < ni; ++j)
				{
					unsigned short nri = srcReg[j];
					if (nri == 0 || nri >= nreg)
						continue;

					if (nri != ri)
					{
						addUniqueLayerRegion(reg, nri);
					}
				}

				// Have found contour
				if (reg.connections.size() > 0)
					continue;

				// Check if this cell is next to a border.
				int ndir = -1;
				for (int dir = 0; dir < 4; ++dir)
				{
					if (isSolidEdge(chf, srcReg, x, y, i, dir))
					{
						ndir = dir;
						break;
					}
				}

				if (ndir != -1)
				{
					// The cell is at border.
					// Walk around the contour to find all the neighbors.
					walkContour(x, y, i, ndir, chf, srcReg, reg.connections);
				}
			}
		}
	} 

	// Create 2D layers from regions. 
	unsigned short layerId = 0;
	rcIntArray stack(64);
	for (int i = 0; i < nreg; i++)
	{
		rcLayerRegion& reg = regions[i];
		if (reg.visited || !reg.hasSpans)
			continue;

		reg.layerId = layerId;
		reg.visited = true;
		reg.base = true;

		stack.resize(0);
		stack.push(i);

		while (stack.size())
		{
			int ri = stack.pop();
			rcLayerRegion& creg = regions[ri];
			for (int j = 0; j < creg.connections.size(); j++)
			{
				const unsigned short nei = (unsigned short)creg.connections[j];
				if (nei & RC_BORDER_REG)
					continue;

				rcLayerRegion& regn = regions[nei];
				// Skip already visited.
				if (regn.visited)
					continue;
				// Skip if the neighbor is overlapping root region.
				if (reg.layers.contains(nei))
					continue;
				// Skip if the height range would become too large.
				const rcSpanUInt ymin = rcMin(reg.ymin, regn.ymin);
				const rcSpanUInt ymax = rcMax(reg.ymax, regn.ymax);
				if (((int)ymax - (int)ymin) >= 255)
					continue;

				// visit
				stack.push(nei);
				regn.visited = true;
				regn.layerId = layerId;
				// add layers to root
				for (int k = 0; k < regn.layers.size(); k++)
					addUniqueLayerRegion(reg, regn.layers[k]);
				reg.ymin = rcMin(reg.ymin, regn.ymin);
				reg.ymax = rcMax(reg.ymax, regn.ymax);
			}
		}

		layerId++;
	}

	// Merge non-overlapping regions that are close in height.
	const unsigned short mergeHeight = (unsigned short)walkableHeight * 4; 
	for (int i = 0; i < nreg; i++)
	{
		rcLayerRegion& ri = regions[i];
		if (!ri.base) continue;

		unsigned short newId = ri.layerId;
		for (;;)
		{
			unsigned short oldId = 0xffff;
			for (int j = 0; j < nreg; j++)
			{
				if (i == j) continue;
				rcLayerRegion& rj = regions[j];
				if (!rj.base) continue;

				// Skip if the regions are not close to each other.
				if (!overlapRange(ri.ymin,ri.ymax+mergeHeight, rj.ymin,rj.ymax+mergeHeight))
					continue;
				// Skip if the height range would become too large.
				const rcSpanUInt ymin = rcMin(ri.ymin, rj.ymin);
				const rcSpanUInt ymax = rcMax(ri.ymax, rj.ymax);
				if (((int)ymax - (int)ymin) >= 255)
					continue;

				// Make sure that there is no overlap when mergin 'ri' and 'rj'.
				bool overlap = false;
				// Iterate over all regions which have the same layerId as 'rj'
				for (int k = 0; k < nreg; ++k)
				{
					if (regions[k].layerId != rj.layerId)
						continue;
					// Check if region 'k' is overlapping region 'ri'
					// Index to 'regs' is the same as region id.
					if (ri.layers.contains(k))
					{
						overlap = true;
						break;
					}
				}
				// Cannot merge of regions overlap.
				if (overlap)
					continue;

				// Can merge i and j.
				oldId = rj.layerId;
				break;
			}

			// Could not find anything to merge with, stop.
			if (oldId == 0xffff)
				break;

			// Merge
			for (int j = 0; j < nreg; ++j)
			{
				rcLayerRegion& rj = regions[j];
				if (rj.layerId == oldId)
				{
					rj.base = 0;
					// Remap layerIds.
					rj.layerId = newId;
					// Add overlaid layers from 'rj' to 'ri'.
					for (int k = 0; k < rj.layers.size(); ++k)
						addUniqueLayerRegion(ri, rj.layers[k]);
					// Update height bounds.
					ri.ymin = rcMin(ri.ymin, rj.ymin);
					ri.ymax = rcMax(ri.ymax, rj.ymax);
				}
			}
		}
	}

	// Compress layer Ids.
	for (int i = 0; i < nreg; ++i)
	{
		regions[i].remap = regions[i].hasSpans;
		if (!regions[i].hasSpans)
		{
			regions[i].layerId = 0xffff;
		}
	}

	unsigned short maxLayerId = 0;
	for (int i = 0; i < nreg; ++i)
	{
		if (!regions[i].remap)
			continue;
		unsigned short oldId = regions[i].layerId;
		unsigned short newId = maxLayerId;
		for (int j = i; j < nreg; ++j)
		{
			if (regions[j].layerId == oldId)
			{
				regions[j].layerId = newId;
				regions[j].remap = false;
			}
		}
		maxLayerId++;
	}

	ctx->stopTimer(RC_TIMER_BUILD_REGIONS_FILTER);

	if (maxLayerId == 0)
	{
		ctx->stopTimer(RC_TIMER_BUILD_LAYERS);
		return true;
	}

	// Create layers.
	rcAssert(lset.layers == 0);

	const int lw = w - (borderSize.low+borderSize.high);	//@UE
	const int lh = h - (borderSize.low+borderSize.high);	//@UE

	// Build contracted bbox for layers.
	rcReal bmin[3], bmax[3];
	rcVcopy(bmin, chf.bmin);
	rcVcopy(bmax, chf.bmax);
	bmin[0] += borderSize.low*chf.cs;		//@UE
	bmin[2] += borderSize.low*chf.cs;		//@UE
	bmax[0] -= borderSize.high*chf.cs;		//@UE
	bmax[2] -= borderSize.high*chf.cs;		//@UE

	lset.nlayers = (int)maxLayerId;

	lset.layers = (rcHeightfieldLayer*)rcAlloc(sizeof(rcHeightfieldLayer)*lset.nlayers, RC_ALLOC_PERM);
	if (!lset.layers)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildHeightfieldLayers: Out of memory 'layers' (%d).", lset.nlayers);
		return false;
	}
	memset(lset.layers, 0, sizeof(rcHeightfieldLayer)*lset.nlayers);


	// Store layers.
	for (int i = 0; i < lset.nlayers; ++i)
	{
		unsigned short curId = (unsigned short)i;

		// Allocate memory for the current layer.
		rcHeightfieldLayer* layer = &lset.layers[i];
		memset(layer, 0, sizeof(rcHeightfieldLayer));

		const int gridSize = sizeof(unsigned char)*lw*lh;
		const int gridSize2 = sizeof(unsigned short)*lw*lh;

		layer->heights = (unsigned short*)rcAlloc(gridSize2, RC_ALLOC_PERM);
		if (!layer->heights)
		{
			ctx->log(RC_LOG_ERROR, "rcBuildHeightfieldLayers: Out of memory 'heights' (%d).", gridSize2);
			return false;
		}
		memset(layer->heights, 0xff, gridSize2);

		layer->areas = (unsigned char*)rcAlloc(gridSize, RC_ALLOC_PERM);
		if (!layer->areas)
		{
			ctx->log(RC_LOG_ERROR, "rcBuildHeightfieldLayers: Out of memory 'areas' (%d).", gridSize);
			return false;
		}
		memset(layer->areas, 0, gridSize);

		layer->cons = (unsigned char*)rcAlloc(gridSize, RC_ALLOC_PERM);
		if (!layer->cons)
		{
			ctx->log(RC_LOG_ERROR, "rcBuildHeightfieldLayers: Out of memory 'cons' (%d).", gridSize);
			return false;
		}
		memset(layer->cons, 0, gridSize);

		// Find layer height bounds.
		rcSpanUInt hmin = 0, hmax = 0;
		for (int j = 0; j < nreg; ++j)
		{
			if (regions[j].base && regions[j].layerId == curId)
			{
				hmin = regions[j].ymin;
				hmax = regions[j].ymax;
			}
		}

		layer->width = lw;
		layer->height = lh;
		layer->cs = chf.cs;
		layer->ch = chf.ch;

		// Adjust the bbox to fit the heighfield.
		rcVcopy(layer->bmin, bmin);
		rcVcopy(layer->bmax, bmax);
		layer->bmin[1] = bmin[1] + hmin*chf.ch;
		layer->bmax[1] = bmin[1] + hmax*chf.ch;

		// Update usable data region.
		layer->minx = layer->width;
		layer->maxx = 0;
		layer->miny = layer->height;
		layer->maxy = 0;

		// Copy height and area from compact heighfield. 
		for (int y = 0; y < lh; ++y)
		{
			for (int x = 0; x < lw; ++x)
			{
				const int cx = borderSize.low+x;		//@UE
				const int cy = borderSize.low+y;		//@UE
				const rcCompactCell& c = chf.cells[cx+cy*w];
				for (int j = (int)c.index, nj = (int)(c.index+c.count); j < nj; ++j)
				{
					const rcCompactSpan& s = chf.spans[j];
					// Skip unassigned regions.
					if (srcReg[j] == 0 || srcReg[j] >= nreg)
						continue;
					// Skip of does not belong to current layer.
					unsigned short lid = regions[srcReg[j]].layerId;
					if (lid != curId)
						continue;

					// Update data bounds.
					layer->minx = rcMin(layer->minx, x);
					layer->maxx = rcMax(layer->maxx, x);
					layer->miny = rcMin(layer->miny, y);
					layer->maxy = rcMax(layer->maxy, y);

					// Store height and area type.
					const int idx = x+y*lw;
					layer->heights[idx] = (unsigned short)(s.y - hmin);
					layer->areas[idx] = chf.areas[j];

					// Check connection.
					unsigned char portal = 0;
					unsigned char con = 0;
					for (int dir = 0; dir < 4; ++dir)
					{
						if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
						{
							const int ax = cx + rcGetDirOffsetX(dir);
							const int ay = cy + rcGetDirOffsetY(dir);
							const int ai = (int)chf.cells[ax + ay*w].index + rcGetCon(s, dir);
							unsigned short alid = (srcReg[ai] < nreg) ? regions[srcReg[ai]].layerId : 0xffff;
							// Portal mask
							if (chf.areas[ai] != RC_NULL_AREA && lid != alid)
							{
								portal |= (unsigned char)(1 << dir);
								// Update height so that it matches on both sides of the portal.
								const rcCompactSpan& as = chf.spans[ai];
								if (as.y > hmin)
									layer->heights[idx] = rcMax(layer->heights[idx], (unsigned short)(as.y - hmin));
							}
							// Valid connection mask
							if (chf.areas[ai] != RC_NULL_AREA && lid == alid)
							{
								const int nx = ax - borderSize.low;		//@UE
								const int ny = ay - borderSize.low;		//@UE
								if (nx >= 0 && ny >= 0 && nx < lw && ny < lh)
								{
									con |= (unsigned char)(1 << dir);
								}
							}
						}
					}

					layer->cons[idx] = (portal << 4) | con;
				}
			}
		}

		fixLayerConnections(layer);

		if (layer->minx > layer->maxx)
			layer->minx = layer->maxx = 0;
		if (layer->miny > layer->maxy)
			layer->miny = layer->maxy = 0;
	}

	ctx->stopTimer(RC_TIMER_BUILD_LAYERS);
	return true;
}
