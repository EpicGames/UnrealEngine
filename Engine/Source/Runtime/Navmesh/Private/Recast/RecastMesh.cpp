// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAlloc.h"
#include "Recast/RecastAssert.h"

struct rcEdge
{
	unsigned short vert[2];
	unsigned short polyEdge[2];
	unsigned short poly[2];
};

static bool buildMeshAdjacency(unsigned short* polys, const int npolys,
							   const int nverts, const int vertsPerPoly)
{
	// Based on code by Eric Lengyel from:
	// http://www.terathon.com/code/edges.php
	
	int maxEdgeCount = npolys*vertsPerPoly;
	unsigned short* firstEdge = (unsigned short*)rcAlloc(sizeof(unsigned short)*(nverts + maxEdgeCount), RC_ALLOC_TEMP);
	if (!firstEdge)
		return false;
	unsigned short* nextEdge = firstEdge + nverts;
	int edgeCount = 0;
	
	rcEdge* edges = (rcEdge*)rcAlloc(sizeof(rcEdge)*maxEdgeCount, RC_ALLOC_TEMP);
	if (!edges)
	{
		rcFree(firstEdge);
		return false;
	}
	
	for (int i = 0; i < nverts; i++)
		firstEdge[i] = RC_MESH_NULL_IDX;
	
	for (int i = 0; i < npolys; ++i)
	{
		unsigned short* t = &polys[i*vertsPerPoly*2];
		for (int j = 0; j < vertsPerPoly; ++j)
		{
			if (t[j] == RC_MESH_NULL_IDX) break;
			unsigned short v0 = t[j];
			unsigned short v1 = (j+1 >= vertsPerPoly || t[j+1] == RC_MESH_NULL_IDX) ? t[0] : t[j+1];
			if (v0 < v1)
			{
				rcEdge& edge = edges[edgeCount];
				edge.vert[0] = v0;
				edge.vert[1] = v1;
				edge.poly[0] = (unsigned short)i;
				edge.polyEdge[0] = (unsigned short)j;
				edge.poly[1] = (unsigned short)i;
				edge.polyEdge[1] = 0;
				// Insert edge
				nextEdge[edgeCount] = firstEdge[v0];
				firstEdge[v0] = (unsigned short)edgeCount;
				edgeCount++;
			}
		}
	}
	
	for (int i = 0; i < npolys; ++i)
	{
		unsigned short* t = &polys[i*vertsPerPoly*2];
		for (int j = 0; j < vertsPerPoly; ++j)
		{
			if (t[j] == RC_MESH_NULL_IDX) break;
			unsigned short v0 = t[j];
			unsigned short v1 = (j+1 >= vertsPerPoly || t[j+1] == RC_MESH_NULL_IDX) ? t[0] : t[j+1];
			if (v0 > v1)
			{
				for (unsigned short e = firstEdge[v1]; e != RC_MESH_NULL_IDX; e = nextEdge[e])
				{
					rcEdge& edge = edges[e];
					if (edge.vert[1] == v0 && edge.poly[0] == edge.poly[1])
					{
						edge.poly[1] = (unsigned short)i;
						edge.polyEdge[1] = (unsigned short)j;
						break;
					}
				}
			}
		}
	}
	
	// Store adjacency
	for (int i = 0; i < edgeCount; ++i)
	{
		const rcEdge& e = edges[i];
		if (e.poly[0] != e.poly[1])
		{
			unsigned short* p0 = &polys[e.poly[0]*vertsPerPoly*2];
			unsigned short* p1 = &polys[e.poly[1]*vertsPerPoly*2];
			p0[vertsPerPoly + e.polyEdge[0]] = e.poly[1];
			p1[vertsPerPoly + e.polyEdge[1]] = e.poly[0];
		}
	}
	
	rcFree(firstEdge);
	rcFree(edges);
	
	return true;
}


static const int VERTEX_BUCKET_COUNT = (1<<12);

inline int computeVertexHash(int x, int y, int z)
{
	const unsigned int h1 = 0x8da6b343; // Large multiplicative constants;
	const unsigned int h2 = 0xd8163841; // here arbitrarily chosen primes
	const unsigned int h3 = 0xcb1ab31f;
	unsigned int n = h1 * x + h2 * y + h3 * z;
	return (int)(n & (VERTEX_BUCKET_COUNT-1));
}

static unsigned short addVertex(unsigned short x, unsigned short y, unsigned short z,
								unsigned short* verts, int* firstVert, int* nextVert, int& nv)
{
	int bucket = computeVertexHash(x, 0, z);
	int i = firstVert[bucket];
	
	while (i != -1)
	{
		const unsigned short* v = &verts[i*3];
		if (v[0] == x && (rcAbs(v[1] - y) <= 2) && v[2] == z)
			return (unsigned short)i;
		i = nextVert[i]; // next
	}
	
	// Could not find, create new.
	i = nv; nv++;
	unsigned short* v = &verts[i*3];
	v[0] = x;
	v[1] = y;
	v[2] = z;
	nextVert[i] = firstVert[bucket];
	firstVert[bucket] = i;
	
	return (unsigned short)i;
}

inline int prev(int i, int n) { return i-1 >= 0 ? i-1 : n-1; }
inline int next(int i, int n) { return i+1 < n ? i+1 : 0; }

inline int area2(const int* a, const int* b, const int* c)
{
	return (b[0] - a[0]) * (c[2] - a[2]) - (c[0] - a[0]) * (b[2] - a[2]);
}

//	Exclusive or: true iff exactly one argument is true.
//	The arguments are negated to ensure that they are 0/1
//	values.  Then the bitwise Xor operator may apply.
//	(This idea is due to Michael Baldwin.)
inline bool xorb(bool x, bool y)
{
	return !x ^ !y;
}

// Returns true iff c is strictly to the left of the directed
// line through a to b.
inline bool left(const int* a, const int* b, const int* c)
{
	return area2(a, b, c) < 0;
}

inline bool leftOn(const int* a, const int* b, const int* c)
{
	return area2(a, b, c) <= 0;
}

inline bool collinear(const int* a, const int* b, const int* c)
{
	return area2(a, b, c) == 0;
}

//	Returns true iff ab properly intersects cd: they share
//	a point interior to both segments.  The properness of the
//	intersection is ensured by using strict leftness.
static bool intersectProp(const int* a, const int* b, const int* c, const int* d)
{
	// Eliminate improper cases.
	if (collinear(a,b,c) || collinear(a,b,d) ||
		collinear(c,d,a) || collinear(c,d,b))
		return false;
	
	return xorb(left(a,b,c), left(a,b,d)) && xorb(left(c,d,a), left(c,d,b));
}

// Returns T iff (a,b,c) are collinear and point c lies 
// on the closed segement ab.
static bool between(const int* a, const int* b, const int* c)
{
	if (!collinear(a, b, c))
		return false;
	// If ab not vertical, check betweenness on x; else on y.
	if (a[0] != b[0])
		return	((a[0] <= c[0]) && (c[0] <= b[0])) || ((a[0] >= c[0]) && (c[0] >= b[0]));
	else
		return	((a[2] <= c[2]) && (c[2] <= b[2])) || ((a[2] >= c[2]) && (c[2] >= b[2]));
}

// Returns true iff segments ab and cd intersect, properly or improperly.
static bool intersect(const int* a, const int* b, const int* c, const int* d)
{
	if (intersectProp(a, b, c, d))
		return true;
	else if (between(a, b, c) || between(a, b, d) ||
			 between(c, d, a) || between(c, d, b))
		return true;
	else
		return false;
}

static bool vequal(const int* a, const int* b)
{
	return a[0] == b[0] && a[2] == b[2];
}

// Returns T iff (v_i, v_j) is a proper internal *or* external
// diagonal of P, *ignoring edges incident to v_i and v_j*.
static bool diagonalie(int i, int j, int n, const int* verts, int* indices)
{
	const int* d0 = &verts[(indices[i] & 0x0fffffff) * 4];
	const int* d1 = &verts[(indices[j] & 0x0fffffff) * 4];
	
	// For each edge (k,k+1) of P
	for (int k = 0; k < n; k++)
	{
		int k1 = next(k, n);
		// Skip edges incident to i or j
		if (!((k == i) || (k1 == i) || (k == j) || (k1 == j)))
		{
			const int* p0 = &verts[(indices[k] & 0x0fffffff) * 4];
			const int* p1 = &verts[(indices[k1] & 0x0fffffff) * 4];

			if (vequal(d0, p0) || vequal(d1, p0) || vequal(d0, p1) || vequal(d1, p1))
				continue;
			
			if (intersect(d0, d1, p0, p1))
				return false;
		}
	}
	return true;
}

// Returns true iff the diagonal (i,j) is strictly internal to the 
// polygon P in the neighborhood of the i endpoint.
static bool	inCone(int i, int j, int n, const int* verts, int* indices)
{
	const int* vi = &verts[(indices[i] & 0x0fffffff) * 4];
	const int* vj = &verts[(indices[j] & 0x0fffffff) * 4];
	const int* vi1 = &verts[(indices[next(i, n)] & 0x0fffffff) * 4];
	const int* vin1 = &verts[(indices[prev(i, n)] & 0x0fffffff) * 4];

	// If P[i] is a convex vertex [ i+1 left or on (i-1,i) ].
	if (leftOn(vin1, vi, vi1))
		return left(vi, vj, vin1) && left(vj, vi, vi1);
	// Assume (i-1,i,i+1) not collinear.
	// else P[i] is reflex.
	return !(leftOn(vi, vj, vi1) && leftOn(vj, vi, vin1));
}

// Returns T iff (v_i, v_j) is a proper internal
// diagonal of P.
static bool diagonal(int i, int j, int n, const int* verts, int* indices)
{
	return inCone(i, j, n, verts, indices) && diagonalie(i, j, n, verts, indices);
}

static int triangulate(int n, const int* verts, int* indices, int* tris)
{
	int ntris = 0;
	int* dst = tris;
	
	// The last bit of the index is used to indicate if the vertex can be removed.
	for (int i = 0; i < n; i++)
	{
		int i1 = next(i, n);
		int i2 = next(i1, n);
		if (diagonal(i, i2, n, verts, indices))
			indices[i1] |= 0x80000000;
	}
	
	while (n > 3)
	{
		int minLen = -1;
		int mini = -1;
		for (int i = 0; i < n; i++)
		{
			int i1 = next(i, n);
			if (indices[i1] & 0x80000000)
			{
				const int* p0 = &verts[(indices[i] & 0x0fffffff) * 4];
				const int* p2 = &verts[(indices[next(i1, n)] & 0x0fffffff) * 4];
				
				int dx = p2[0] - p0[0];
				int dy = p2[2] - p0[2];
				int len = dx*dx + dy*dy;
				
				if (minLen < 0 || len < minLen)
				{
					minLen = len;
					mini = i;
				}
			}
		}
		
		if (mini == -1)
		{
			// Should not happen.
/*			printf("mini == -1 ntris=%d n=%d\n", ntris, n);
			for (int i = 0; i < n; i++)
			{
				printf("%d ", indices[i] & 0x0fffffff);
			}
			printf("\n");*/
			return -ntris;
		}
		
		int i = mini;
		int i1 = next(i, n);
		int i2 = next(i1, n);
		
		*dst++ = indices[i] & 0x0fffffff;
		*dst++ = indices[i1] & 0x0fffffff;
		*dst++ = indices[i2] & 0x0fffffff;
		ntris++;
		
		// Removes P[i1] by copying P[i+1]...P[n-1] left one index.
		n--;
		for (int k = i1; k < n; k++)
			indices[k] = indices[k+1];
		
		if (i1 >= n) i1 = 0;
		i = prev(i1,n);
		// Update diagonal flags.
		if (diagonal(prev(i, n), i1, n, verts, indices))
			indices[i] |= 0x80000000;
		else
			indices[i] &= 0x0fffffff;
		
		if (diagonal(i, next(i1, n), n, verts, indices))
			indices[i1] |= 0x80000000;
		else
			indices[i1] &= 0x0fffffff;
	}
	
	// Append the remaining triangle.
	*dst++ = indices[0] & 0x0fffffff;
	*dst++ = indices[1] & 0x0fffffff;
	*dst++ = indices[2] & 0x0fffffff;
	ntris++;
	
	return ntris;
}

static int countPolyVerts(const unsigned short* p, const int nvp)
{
	for (int i = 0; i < nvp; ++i)
		if (p[i] == RC_MESH_NULL_IDX)
			return i;
	return nvp;
}

inline bool uleft(const unsigned short* a, const unsigned short* b, const unsigned short* c)
{
	return ((int)b[0] - (int)a[0]) * ((int)c[2] - (int)a[2]) -
		   ((int)c[0] - (int)a[0]) * ((int)b[2] - (int)a[2]) < 0;
}

static int getPolyMergeValue(unsigned short* pa, unsigned short* pb,
							 const unsigned short* verts, int& ea, int& eb,
							 const int nvp)
{
	const int na = countPolyVerts(pa, nvp);
	const int nb = countPolyVerts(pb, nvp);
	
	// If the merged polygon would be too big, do not merge.
	if (na+nb-2 > nvp)
		return -1;
	
	// Check if the polygons share an edge.
	ea = -1;
	eb = -1;
	
	for (int i = 0; i < na; ++i)
	{
		unsigned short va0 = pa[i];
		unsigned short va1 = pa[(i+1) % na];
		if (va0 > va1)
			rcSwap(va0, va1);
		for (int j = 0; j < nb; ++j)
		{
			unsigned short vb0 = pb[j];
			unsigned short vb1 = pb[(j+1) % nb];
			if (vb0 > vb1)
				rcSwap(vb0, vb1);
			if (va0 == vb0 && va1 == vb1)
			{
				ea = i;
				eb = j;
				break;
			}
		}
	}
	
	// No common edge, cannot merge.
	if (ea == -1 || eb == -1)
		return -1;
	
	// Check to see if the merged polygon would be convex.
	unsigned short va, vb, vc;
	
	va = pa[(ea+na-1) % na];
	vb = pa[ea];
	vc = pb[(eb+2) % nb];
	if (!uleft(&verts[va*3], &verts[vb*3], &verts[vc*3]))
		return -1;
	
	va = pb[(eb+nb-1) % nb];
	vb = pb[eb];
	vc = pa[(ea+2) % na];
	if (!uleft(&verts[va*3], &verts[vb*3], &verts[vc*3]))
		return -1;
	
	va = pa[ea];
	vb = pa[(ea+1)%na];
	
	int dx = (int)verts[va*3+0] - (int)verts[vb*3+0];
	int dy = (int)verts[va*3+2] - (int)verts[vb*3+2];
	
	return dx*dx + dy*dy;
}

static void mergePolys(unsigned short* pa, unsigned short* pb, int ea, int eb,
					   unsigned short* tmp, const int nvp)
{
	const int na = countPolyVerts(pa, nvp);
	const int nb = countPolyVerts(pb, nvp);
	
	// Merge polygons.
	memset(tmp, 0xff, sizeof(unsigned short)*nvp);
	int n = 0;
	// Add pa
	for (int i = 0; i < na-1; ++i)
		tmp[n++] = pa[(ea+1+i) % na];
	// Add pb
	for (int i = 0; i < nb-1; ++i)
		tmp[n++] = pb[(eb+1+i) % nb];
	
	memcpy(pa, tmp, sizeof(unsigned short)*nvp);
}


static void pushFront(int v, int* arr, int& an)
{
	an++;
	for (int i = an-1; i > 0; --i) arr[i] = arr[i-1];
	arr[0] = v;
}

static void pushBack(int v, int* arr, int& an)
{
	arr[an] = v;
	an++;
}

static bool canRemoveVertex(rcContext* ctx, rcPolyMesh& mesh, const unsigned short rem)
{
	const int nvp = mesh.nvp;
	
	// Count number of polygons to remove.
	int numRemovedVerts = 0;
	int numTouchedVerts = 0;
	int numRemainingEdges = 0;
	for (int i = 0; i < mesh.npolys; ++i)
	{
		unsigned short* p = &mesh.polys[i*nvp*2];
		const int nv = countPolyVerts(p, nvp);
		int numRemoved = 0;
		int numVerts = 0;
		for (int j = 0; j < nv; ++j)
		{
			if (p[j] == rem)
			{
				numTouchedVerts++;
				numRemoved++;
			}
			numVerts++;
		}
		if (numRemoved)
		{
			numRemovedVerts += numRemoved;
			numRemainingEdges += numVerts-(numRemoved+1);
		}
	}
	
	// There would be too few edges remaining to create a polygon.
	// This can happen for example when a tip of a triangle is marked
	// as deletion, but there are no other polys that share the vertex.
	// In this case, the vertex should not be removed.
	if (numRemainingEdges <= 2)
		return false;
	
	// Find edges which share the removed vertex.
	const int maxEdges = numTouchedVerts*2;
	int nedges = 0;
	rcScopedDelete<int> edges = (int*)rcAlloc(sizeof(int)*maxEdges*3, RC_ALLOC_TEMP);
	if (!edges)
	{
		ctx->log(RC_LOG_WARNING, "canRemoveVertex: Out of memory 'edges' (%d).", maxEdges*3);
		return false;
	}
		
	for (int i = 0; i < mesh.npolys; ++i)
	{
		unsigned short* p = &mesh.polys[i*nvp*2];
		const int nv = countPolyVerts(p, nvp);

		// Collect edges which touches the removed vertex.
		for (int j = 0, k = nv-1; j < nv; k = j++)
		{
			if (p[j] == rem || p[k] == rem)
			{
				// Arrange edge so that a=rem.
				int a = p[j], b = p[k];
				if (b == rem)
					rcSwap(a,b);
					
				// Check if the edge exists
				bool exists = false;
				for (int m = 0; m < nedges; ++m)
				{
					int* e = &edges[m*3];
					if (e[1] == b)
					{
						// Exists, increment vertex share count.
						e[2]++;
						exists = true;
					}
				}
				// Add new edge.
				if (!exists)
				{
					int* e = &edges[nedges*3];
					e[0] = a;
					e[1] = b;
					e[2] = 1;
					nedges++;
				}
			}
		}
	}

	// There should be no more than 2 open edges.
	// This catches the case that two non-adjacent polygons
	// share the removed vertex. In that case, do not remove the vertex.
	int numOpenEdges = 0;
	for (int i = 0; i < nedges; ++i)
	{
		if (edges[i*3+2] < 2)
			numOpenEdges++;
	}
	if (numOpenEdges > 2)
		return false;
	
	return true;
}

static bool removeVertex(rcContext* ctx, rcPolyMesh& mesh, const unsigned short rem, const int maxTris)
{
	const int nvp = mesh.nvp;

	// Count number of polygons to remove.
	int numRemovedVerts = 0;
	for (int i = 0; i < mesh.npolys; ++i)
	{
		unsigned short* p = &mesh.polys[i*nvp*2];
		const int nv = countPolyVerts(p, nvp);
		for (int j = 0; j < nv; ++j)
		{
			if (p[j] == rem)
				numRemovedVerts++;
		}
	}
	
	int nedges = 0;
	rcScopedDelete<int> edges = (int*)rcAlloc(sizeof(int)*numRemovedVerts*nvp*4, RC_ALLOC_TEMP);
	if (!edges)
	{
		ctx->log(RC_LOG_WARNING, "removeVertex: Out of memory 'edges' (%d).", numRemovedVerts*nvp*4);
		return false;
	}

	int nhole = 0;
	rcScopedDelete<int> hole = (int*)rcAlloc(sizeof(int)*numRemovedVerts*nvp, RC_ALLOC_TEMP);
	if (!hole)
	{
		ctx->log(RC_LOG_WARNING, "removeVertex: Out of memory 'hole' (%d).", numRemovedVerts*nvp);
		return false;
	}
	
	int nhreg = 0;
	rcScopedDelete<int> hreg = (int*)rcAlloc(sizeof(int)*numRemovedVerts*nvp, RC_ALLOC_TEMP);
	if (!hreg)
	{
		ctx->log(RC_LOG_WARNING, "removeVertex: Out of memory 'hreg' (%d).", numRemovedVerts*nvp);
		return false;
	}

	int nharea = 0;
	rcScopedDelete<int> harea = (int*)rcAlloc(sizeof(int)*numRemovedVerts*nvp, RC_ALLOC_TEMP);
	if (!harea)
	{
		ctx->log(RC_LOG_WARNING, "removeVertex: Out of memory 'harea' (%d).", numRemovedVerts*nvp);
		return false;
	}
	
	for (int i = 0; i < mesh.npolys; ++i)
	{
		unsigned short* p = &mesh.polys[i*nvp*2];
		const int nv = countPolyVerts(p, nvp);
		bool hasRem = false;
		for (int j = 0; j < nv; ++j)
			if (p[j] == rem) hasRem = true;
		if (hasRem)
		{
			// Collect edges which does not touch the removed vertex.
			for (int j = 0, k = nv-1; j < nv; k = j++)
			{
				if (p[j] != rem && p[k] != rem)
				{
					int* e = &edges[nedges*4];
					e[0] = p[k];
					e[1] = p[j];
					e[2] = mesh.regs[i];
					e[3] = mesh.areas[i];
					nedges++;
				}
			}
			// Remove the polygon.
			unsigned short* p2 = &mesh.polys[(mesh.npolys-1)*nvp*2];
			memcpy(p,p2,sizeof(unsigned short)*nvp);
			memset(p+nvp,0xff,sizeof(unsigned short)*nvp);
			mesh.regs[i] = mesh.regs[mesh.npolys-1];
			mesh.areas[i] = mesh.areas[mesh.npolys-1];
			mesh.npolys--;
			--i;
		}
	}
	
	// Remove vertex.
	for (int i = (int)rem; i < mesh.nverts; ++i)
	{
		mesh.verts[i*3+0] = mesh.verts[(i+1)*3+0];
		mesh.verts[i*3+1] = mesh.verts[(i+1)*3+1];
		mesh.verts[i*3+2] = mesh.verts[(i+1)*3+2];
	}
	mesh.nverts--;

	// Adjust indices to match the removed vertex layout.
	for (int i = 0; i < mesh.npolys; ++i)
	{
		unsigned short* p = &mesh.polys[i*nvp*2];
		const int nv = countPolyVerts(p, nvp);
		for (int j = 0; j < nv; ++j)
			if (p[j] > rem) p[j]--;
	}
	for (int i = 0; i < nedges; ++i)
	{
		if (edges[i*4+0] > rem) edges[i*4+0]--;
		if (edges[i*4+1] > rem) edges[i*4+1]--;
	}

	if (nedges == 0)
		return true;

	// Start with one vertex, keep appending connected
	// segments to the start and end of the hole.
	pushBack(edges[0], hole, nhole);
	pushBack(edges[2], hreg, nhreg);
	pushBack(edges[3], harea, nharea);
	
	while (nedges)
	{
		bool match = false;
		
		for (int i = 0; i < nedges; ++i)
		{
			const int ea = edges[i*4+0];
			const int eb = edges[i*4+1];
			const int r = edges[i*4+2];
			const int a = edges[i*4+3];
			bool add = false;
			if (hole[0] == eb)
			{
				// The segment matches the beginning of the hole boundary.
				pushFront(ea, hole, nhole);
				pushFront(r, hreg, nhreg);
				pushFront(a, harea, nharea);
				add = true;
			}
			else if (hole[nhole-1] == ea)
			{
				// The segment matches the end of the hole boundary.
				pushBack(eb, hole, nhole);
				pushBack(r, hreg, nhreg);
				pushBack(a, harea, nharea);
				add = true;
			}
			if (add)
			{
				// The edge segment was added, remove it.
				edges[i*4+0] = edges[(nedges-1)*4+0];
				edges[i*4+1] = edges[(nedges-1)*4+1];
				edges[i*4+2] = edges[(nedges-1)*4+2];
				edges[i*4+3] = edges[(nedges-1)*4+3];
				--nedges;
				match = true;
				--i;
			}
		}
		
		if (!match)
			break;
	}

	rcScopedDelete<int> tris = (int*)rcAlloc(sizeof(int)*nhole*3, RC_ALLOC_TEMP);
	if (!tris)
	{
		ctx->log(RC_LOG_WARNING, "removeVertex: Out of memory 'tris' (%d).", nhole*3);
		return false;
	}

	rcScopedDelete<int> tverts = (int*)rcAlloc(sizeof(int)*nhole*4, RC_ALLOC_TEMP);
	if (!tverts)
	{
		ctx->log(RC_LOG_WARNING, "removeVertex: Out of memory 'tverts' (%d).", nhole*4);
		return false;
	}

	rcScopedDelete<int> thole = (int*)rcAlloc(sizeof(int)*nhole, RC_ALLOC_TEMP);
	if (!tverts)
	{
		ctx->log(RC_LOG_WARNING, "removeVertex: Out of memory 'thole' (%d).", nhole);
		return false;
	}

	// Generate temp vertex array for triangulation.
	for (int i = 0; i < nhole; ++i)
	{
		const int holeVertIndex = hole[i];
		tverts[i*4+0] = mesh.verts[holeVertIndex*3+0];
		tverts[i*4+1] = mesh.verts[holeVertIndex*3+1];
		tverts[i*4+2] = mesh.verts[holeVertIndex*3+2];
		tverts[i*4+3] = 0;
		thole[i] = i;
	}

	// Triangulate the hole.
	int ntris = triangulate(nhole, &tverts[0], &thole[0], tris);
	if (ntris < 0)
	{
		ntris = -ntris;
		ctx->log(RC_LOG_WARNING, "removeVertex: triangulate() returned bad results.");
	}
	
	// Merge the hole triangles back to polygons.
	rcScopedDelete<unsigned short> polys = (unsigned short*)rcAlloc(sizeof(unsigned short)*(ntris+1)*nvp, RC_ALLOC_TEMP);
	if (!polys)
	{
		ctx->log(RC_LOG_ERROR, "removeVertex: Out of memory 'polys' (%d).", (ntris+1)*nvp);
		return false;
	}
	rcScopedDelete<unsigned short> pregs = (unsigned short*)rcAlloc(sizeof(unsigned short)*ntris, RC_ALLOC_TEMP);
	if (!pregs)
	{
		ctx->log(RC_LOG_ERROR, "removeVertex: Out of memory 'pregs' (%d).", ntris);
		return false;
	}
	rcScopedDelete<unsigned char> pareas = (unsigned char*)rcAlloc(sizeof(unsigned char)*ntris, RC_ALLOC_TEMP);
	if (!pregs)
	{
		ctx->log(RC_LOG_ERROR, "removeVertex: Out of memory 'pareas' (%d).", ntris);
		return false;
	}
	
	unsigned short* tmpPoly = &polys[ntris*nvp];
			
	// Build initial polygons.
	int npolys = 0;
	memset(polys, 0xff, ntris*nvp*sizeof(unsigned short));
	for (int j = 0; j < ntris; ++j)
	{
		int* t = &tris[j*3];
		if (t[0] != t[1] && t[0] != t[2] && t[1] != t[2])
		{
			polys[npolys*nvp+0] = (unsigned short)hole[t[0]];
			polys[npolys*nvp+1] = (unsigned short)hole[t[1]];
			polys[npolys*nvp+2] = (unsigned short)hole[t[2]];
			pregs[npolys] = (unsigned short)hreg[t[0]];
			pareas[npolys] = (unsigned char)harea[t[0]];
			npolys++;
		}
	}
	if (!npolys)
		return true;
	
	// Merge polygons.
	if (nvp > 3)
	{
		for (;;)
		{
			// Find best polygons to merge.
			int bestMergeVal = 0;
			int bestPa = 0, bestPb = 0, bestEa = 0, bestEb = 0;
			
			for (int j = 0; j < npolys-1; ++j)
			{
				unsigned short* pj = &polys[j*nvp];
				for (int k = j+1; k < npolys; ++k)
				{
					unsigned short* pk = &polys[k*nvp];
					int ea, eb;
					int v = getPolyMergeValue(pj, pk, mesh.verts, ea, eb, nvp);
					if (v > bestMergeVal)
					{
						bestMergeVal = v;
						bestPa = j;
						bestPb = k;
						bestEa = ea;
						bestEb = eb;
					}
				}
			}
			
			if (bestMergeVal > 0)
			{
				// Found best, merge.
				unsigned short* pa = &polys[bestPa*nvp];
				unsigned short* pb = &polys[bestPb*nvp];
				mergePolys(pa, pb, bestEa, bestEb, tmpPoly, nvp);
				memcpy(pb, &polys[(npolys-1)*nvp], sizeof(unsigned short)*nvp);
				pregs[bestPb] = pregs[npolys-1];
				pareas[bestPb] = pareas[npolys-1];
				npolys--;
			}
			else
			{
				// Could not merge any polygons, stop.
				break;
			}
		}
	}
	
	// Store polygons.
	for (int i = 0; i < npolys; ++i)
	{
		if (mesh.npolys >= maxTris) break;
		unsigned short* p = &mesh.polys[mesh.npolys*nvp*2];
		memset(p,0xff,sizeof(unsigned short)*nvp*2);
		for (int j = 0; j < nvp; ++j)
			p[j] = polys[i*nvp+j];
		mesh.regs[mesh.npolys] = pregs[i];
		mesh.areas[mesh.npolys] = pareas[i];
		mesh.npolys++;
		if (mesh.npolys > maxTris)
		{
			ctx->log(RC_LOG_ERROR, "removeVertex: Too many polygons %d (max:%d).", mesh.npolys, maxTris);
			return false;
		}
	}
	
	return true;
}

/// @par
///
/// @note If the mesh data is to be used to construct a Detour navigation mesh, then the upper 
/// limit must be retricted to <= #DT_VERTS_PER_POLYGON.
///
/// @see rcAllocPolyMesh, rcContourSet, rcPolyMesh, rcConfig
bool rcBuildPolyMesh(rcContext* ctx, rcContourSet& cset, const int nvp, rcPolyMesh& mesh)
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_BUILD_POLYMESH);

	rcVcopy(mesh.bmin, cset.bmin);
	rcVcopy(mesh.bmax, cset.bmax);
	mesh.cs = cset.cs;
	mesh.ch = cset.ch;
	mesh.borderSize = cset.borderSize;
	
	int maxVertices = 0;
	int maxTris = 0;
	int maxVertsPerCont = 0;
	for (int i = 0; i < cset.nconts; ++i)
	{
		// Skip null contours.
		if (cset.conts[i].nverts < 3) continue;
		maxVertices += cset.conts[i].nverts;
		maxTris += cset.conts[i].nverts - 2;
		maxVertsPerCont = rcMax(maxVertsPerCont, cset.conts[i].nverts);
	}
	
	if (maxVertices >= 0xfffe)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Too many vertices %d.", maxVertices);
		return false;
	}
		
	rcScopedDelete<unsigned char> vflags = (unsigned char*)rcAlloc(sizeof(unsigned char)*maxVertices, RC_ALLOC_TEMP);
	if (!vflags)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'vflags' (%d).", maxVertices);
		return false;
	}
	memset(vflags, 0, maxVertices);
	
	mesh.verts = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxVertices*3, RC_ALLOC_PERM);
	if (!mesh.verts)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'mesh.verts' (%d).", maxVertices);
		return false;
	}
	mesh.polys = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxTris*nvp*2, RC_ALLOC_PERM);
	if (!mesh.polys)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'mesh.polys' (%d).", maxTris*nvp*2);
		return false;
	}
	mesh.regs = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxTris, RC_ALLOC_PERM);
	if (!mesh.regs)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'mesh.regs' (%d).", maxTris);
		return false;
	}
	mesh.areas = (unsigned char*)rcAlloc(sizeof(unsigned char)*maxTris, RC_ALLOC_PERM);
	if (!mesh.areas)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'mesh.areas' (%d).", maxTris);
		return false;
	}
	
	mesh.nverts = 0;
	mesh.npolys = 0;
	mesh.nvp = nvp;
	mesh.maxpolys = maxTris;
	
	memset(mesh.verts, 0, sizeof(unsigned short)*maxVertices*3);
	memset(mesh.polys, 0xff, sizeof(unsigned short)*maxTris*nvp*2);
	memset(mesh.regs, 0, sizeof(unsigned short)*maxTris);
	memset(mesh.areas, 0, sizeof(unsigned char)*maxTris);
	
	rcScopedDelete<int> nextVert = (int*)rcAlloc(sizeof(int)*maxVertices, RC_ALLOC_TEMP);
	if (!nextVert)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'nextVert' (%d).", maxVertices);
		return false;
	}
	memset(nextVert, 0, sizeof(int)*maxVertices);
	
	rcScopedDelete<int> firstVert = (int*)rcAlloc(sizeof(int)*VERTEX_BUCKET_COUNT, RC_ALLOC_TEMP);
	if (!firstVert)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'firstVert' (%d).", VERTEX_BUCKET_COUNT);
		return false;
	}
	for (int i = 0; i < VERTEX_BUCKET_COUNT; ++i)
		firstVert[i] = -1;
	
	rcScopedDelete<int> indices = (int*)rcAlloc(sizeof(int)*maxVertsPerCont, RC_ALLOC_TEMP);
	if (!indices)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'indices' (%d).", maxVertsPerCont);
		return false;
	}
	rcScopedDelete<int> tris = (int*)rcAlloc(sizeof(int)*maxVertsPerCont*3, RC_ALLOC_TEMP);
	if (!tris)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'tris' (%d).", maxVertsPerCont*3);
		return false;
	}
	rcScopedDelete<unsigned short> polys = (unsigned short*)rcAlloc(sizeof(unsigned short)*(maxVertsPerCont+1)*nvp, RC_ALLOC_TEMP);
	if (!polys)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'polys' (%d).", maxVertsPerCont*nvp);
		return false;
	}
	unsigned short* tmpPoly = &polys[maxVertsPerCont*nvp];

	for (int i = 0; i < cset.nconts; ++i)
	{
		rcContour& cont = cset.conts[i];
		
		// Skip null contours.
		if (cont.nverts < 3)
			continue;
		
		// Triangulate contour
		for (int j = 0; j < cont.nverts; ++j)
			indices[j] = j;
			
		int ntris = triangulate(cont.nverts, cont.verts, &indices[0], &tris[0]);
		if (ntris <= 0)
		{
			// Bad triangulation, should not happen.
/*			printf("\tconst rcReal bmin[3] = {%ff,%ff,%ff};\n", cset.bmin[0], cset.bmin[1], cset.bmin[2]);
			printf("\tconst rcReal cs = %ff;\n", cset.cs);
			printf("\tconst rcReal ch = %ff;\n", cset.ch);
			printf("\tconst int verts[] = {\n");
			for (int k = 0; k < cont.nverts; ++k)
			{
				const int* v = &cont.verts[k*4];
				printf("\t\t%d,%d,%d,%d,\n", v[0], v[1], v[2], v[3]);
			}
			printf("\t};\n\tconst int nverts = sizeof(verts)/(sizeof(int)*4);\n");*/
			ctx->log(RC_LOG_WARNING, "rcBuildPolyMesh: Bad triangulation Contour %d.", i);
			ntris = -ntris;
		}
				
		// Add and merge vertices.
		for (int j = 0; j < cont.nverts; ++j)
		{
			const int* v = &cont.verts[j*4];
			indices[j] = addVertex((unsigned short)v[0], (unsigned short)v[1], (unsigned short)v[2],
								   mesh.verts, firstVert, nextVert, mesh.nverts);
			if (v[3] & RC_BORDER_VERTEX)
			{
				// This vertex should be removed.
				vflags[indices[j]] = 1;
			}
		}

		// Build initial polygons.
		int npolys = 0;
		memset(polys, 0xff, maxVertsPerCont*nvp*sizeof(unsigned short));
		for (int j = 0; j < ntris; ++j)
		{
			int* t = &tris[j*3];
			if (t[0] != t[1] && t[0] != t[2] && t[1] != t[2])
			{
				polys[npolys*nvp+0] = (unsigned short)indices[t[0]];
				polys[npolys*nvp+1] = (unsigned short)indices[t[1]];
				polys[npolys*nvp+2] = (unsigned short)indices[t[2]];
				npolys++;
			}
		}
		if (!npolys)
			continue;
		
		// Merge polygons.
		if (nvp > 3)
		{
			for(;;)
			{
				// Find best polygons to merge.
				int bestMergeVal = 0;
				int bestPa = 0, bestPb = 0, bestEa = 0, bestEb = 0;
				
				for (int j = 0; j < npolys-1; ++j)
				{
					unsigned short* pj = &polys[j*nvp];
					for (int k = j+1; k < npolys; ++k)
					{
						unsigned short* pk = &polys[k*nvp];
						int ea, eb;
						int v = getPolyMergeValue(pj, pk, mesh.verts, ea, eb, nvp);
						if (v > bestMergeVal)
						{
							bestMergeVal = v;
							bestPa = j;
							bestPb = k;
							bestEa = ea;
							bestEb = eb;
						}
					}
				}
				
				if (bestMergeVal > 0)
				{
					// Found best, merge.
					unsigned short* pa = &polys[bestPa*nvp];
					unsigned short* pb = &polys[bestPb*nvp];
					mergePolys(pa, pb, bestEa, bestEb, tmpPoly, nvp);
					memcpy(pb, &polys[(npolys-1)*nvp], sizeof(unsigned short)*nvp);
					npolys--;
				}
				else
				{
					// Could not merge any polygons, stop.
					break;
				}
			}
		}
		
		// Store polygons.
		for (int j = 0; j < npolys; ++j)
		{
			unsigned short* p = &mesh.polys[mesh.npolys*nvp*2];
			unsigned short* q = &polys[j*nvp];
			for (int k = 0; k < nvp; ++k)
				p[k] = q[k];
			mesh.regs[mesh.npolys] = cont.reg;
			mesh.areas[mesh.npolys] = cont.area;
			mesh.npolys++;
			if (mesh.npolys > maxTris)
			{
				ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Too many polygons %d (max:%d).", mesh.npolys, maxTris);
				return false;
			}
		}
	}
	
	
	// Remove edge vertices.
	for (int i = 0; i < mesh.nverts; ++i)
	{
		if (vflags[i])
		{
			if (!canRemoveVertex(ctx, mesh, (unsigned short)i))
				continue;
			if (!removeVertex(ctx, mesh, (unsigned short)i, maxTris))
			{
				// Failed to remove vertex
				ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Failed to remove edge vertex %d.", i);
				return false;
			}
			// Remove vertex
			// Note: mesh.nverts is already decremented inside removeVertex()!
			// Fixup vertex flags
			for (int j = i; j < mesh.nverts; ++j)
				vflags[j] = vflags[j+1];
			--i;
		}
	}
	
	// Calculate adjacency.
	if (!buildMeshAdjacency(mesh.polys, mesh.npolys, mesh.nverts, nvp))
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Adjacency failed.");
		return false;
	}
	
	// Find portal edges
	if (mesh.borderSize.low > 0 || mesh.borderSize.high > 0)	//@UE
	{
		const int w = cset.width;
		const int h = cset.height;
		for (int i = 0; i < mesh.npolys; ++i)
		{
			unsigned short* p = &mesh.polys[i*2*nvp];
			for (int j = 0; j < nvp; ++j)
			{
				if (p[j] == RC_MESH_NULL_IDX) break;
				// Skip connected edges.
				if (p[nvp+j] != RC_MESH_NULL_IDX)
					continue;
				int nj = j+1;
				if (nj >= nvp || p[nj] == RC_MESH_NULL_IDX) nj = 0;
				const unsigned short* va = &mesh.verts[p[j]*3];
				const unsigned short* vb = &mesh.verts[p[nj]*3];

				if ((int)va[0] == 0 && (int)vb[0] == 0)
					p[nvp+j] = 0x8000 | 0;
				else if ((int)va[2] == h && (int)vb[2] == h)
					p[nvp+j] = 0x8000 | 1;
				else if ((int)va[0] == w && (int)vb[0] == w)
					p[nvp+j] = 0x8000 | 2;
				else if ((int)va[2] == 0 && (int)vb[2] == 0)
					p[nvp+j] = 0x8000 | 3;
			}
		}
	}

	// Just allocate the mesh flags array. The user is resposible to fill it.
	mesh.flags = (unsigned short*)rcAlloc(sizeof(unsigned short)*mesh.npolys, RC_ALLOC_PERM);
	if (!mesh.flags)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: Out of memory 'mesh.flags' (%d).", mesh.npolys);
		return false;
	}
	memset(mesh.flags, 0, sizeof(unsigned short) * mesh.npolys);
	
	if (mesh.nverts > 0xffff)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: The resulting mesh has too many vertices %d (max %d). Data can be corrupted.", mesh.nverts, 0xffff);
	}
	if (mesh.npolys > 0xffff)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMesh: The resulting mesh has too many polygons %d (max %d). Data can be corrupted.", mesh.npolys, 0xffff);
	}
	
	ctx->stopTimer(RC_TIMER_BUILD_POLYMESH);
	
	return true;
}

/// @see rcAllocPolyMesh, rcPolyMesh
bool rcMergePolyMeshes(rcContext* ctx, rcPolyMesh** meshes, const int nmeshes, rcPolyMesh& mesh)
{
	rcAssert(ctx);
	
	if (!nmeshes || !meshes)
		return true;

	ctx->startTimer(RC_TIMER_MERGE_POLYMESH);

	mesh.nvp = meshes[0]->nvp;
	mesh.cs = meshes[0]->cs;
	mesh.ch = meshes[0]->ch;
	rcVcopy(mesh.bmin, meshes[0]->bmin);
	rcVcopy(mesh.bmax, meshes[0]->bmax);

	int maxVerts = 0;
	int maxPolys = 0;
	int maxVertsPerMesh = 0;
	for (int i = 0; i < nmeshes; ++i)
	{
		rcVmin(mesh.bmin, meshes[i]->bmin);
		rcVmax(mesh.bmax, meshes[i]->bmax);
		maxVertsPerMesh = rcMax(maxVertsPerMesh, meshes[i]->nverts);
		maxVerts += meshes[i]->nverts;
		maxPolys += meshes[i]->npolys;
	}
	
	mesh.nverts = 0;
	mesh.verts = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxVerts*3, RC_ALLOC_PERM);
	if (!mesh.verts)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Out of memory 'mesh.verts' (%d).", maxVerts*3);
		return false;
	}

	mesh.npolys = 0;
	mesh.polys = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxPolys*2*mesh.nvp, RC_ALLOC_PERM);
	if (!mesh.polys)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Out of memory 'mesh.polys' (%d).", maxPolys*2*mesh.nvp);
		return false;
	}
	memset(mesh.polys, 0xff, sizeof(unsigned short)*maxPolys*2*mesh.nvp);

	mesh.regs = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxPolys, RC_ALLOC_PERM);
	if (!mesh.regs)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Out of memory 'mesh.regs' (%d).", maxPolys);
		return false;
	}
	memset(mesh.regs, 0, sizeof(unsigned short)*maxPolys);

	mesh.areas = (unsigned char*)rcAlloc(sizeof(unsigned char)*maxPolys, RC_ALLOC_PERM);
	if (!mesh.areas)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Out of memory 'mesh.areas' (%d).", maxPolys);
		return false;
	}
	memset(mesh.areas, 0, sizeof(unsigned char)*maxPolys);

	mesh.flags = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxPolys, RC_ALLOC_PERM);
	if (!mesh.flags)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Out of memory 'mesh.flags' (%d).", maxPolys);
		return false;
	}
	memset(mesh.flags, 0, sizeof(unsigned short)*maxPolys);
	
	rcScopedDelete<int> nextVert = (int*)rcAlloc(sizeof(int)*maxVerts, RC_ALLOC_TEMP);
	if (!nextVert)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Out of memory 'nextVert' (%d).", maxVerts);
		return false;
	}
	memset(nextVert, 0, sizeof(int)*maxVerts);
	
	rcScopedDelete<int> firstVert = (int*)rcAlloc(sizeof(int)*VERTEX_BUCKET_COUNT, RC_ALLOC_TEMP);
	if (!firstVert)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Out of memory 'firstVert' (%d).", VERTEX_BUCKET_COUNT);
		return false;
	}
	for (int i = 0; i < VERTEX_BUCKET_COUNT; ++i)
		firstVert[i] = -1;

	rcScopedDelete<unsigned short> vremap = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxVertsPerMesh, RC_ALLOC_PERM);
	if (!vremap)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Out of memory 'vremap' (%d).", maxVertsPerMesh);
		return false;
	}
	memset(vremap, 0, sizeof(unsigned short)*maxVertsPerMesh);
	
	for (int i = 0; i < nmeshes; ++i)
	{
		const rcPolyMesh* pmesh = meshes[i];
		
		const unsigned short ox = (unsigned short)rcFloor((pmesh->bmin[0]-mesh.bmin[0])/mesh.cs+0.5f);
		const unsigned short oz = (unsigned short)rcFloor((pmesh->bmin[2]-mesh.bmin[2])/mesh.cs+0.5f);
		
		for (int j = 0; j < pmesh->nverts; ++j)
		{
			unsigned short* v = &pmesh->verts[j*3];
			vremap[j] = addVertex(v[0]+ox, v[1], v[2]+oz,
								  mesh.verts, firstVert, nextVert, mesh.nverts);
		}
		
		for (int j = 0; j < pmesh->npolys; ++j)
		{
			unsigned short* tgt = &mesh.polys[mesh.npolys*2*mesh.nvp];
			unsigned short* src = &pmesh->polys[j*2*mesh.nvp];
			mesh.regs[mesh.npolys] = pmesh->regs[j];
			mesh.areas[mesh.npolys] = pmesh->areas[j];
			mesh.flags[mesh.npolys] = pmesh->flags[j];
			mesh.npolys++;
			for (int k = 0; k < mesh.nvp; ++k)
			{
				if (src[k] == RC_MESH_NULL_IDX) break;
				tgt[k] = vremap[src[k]];
			}
		}
	}

	// Calculate adjacency.
	if (!buildMeshAdjacency(mesh.polys, mesh.npolys, mesh.nverts, mesh.nvp))
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: Adjacency failed.");
		return false;
	}

	if (mesh.nverts > 0xffff)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: The resulting mesh has too many vertices %d (max %d). Data can be corrupted.", mesh.nverts, 0xffff);
	}
	if (mesh.npolys > 0xffff)
	{
		ctx->log(RC_LOG_ERROR, "rcMergePolyMeshes: The resulting mesh has too many polygons %d (max %d). Data can be corrupted.", mesh.npolys, 0xffff);
	}
	
	ctx->stopTimer(RC_TIMER_MERGE_POLYMESH);
	
	return true;
}

bool rcCopyPolyMesh(rcContext* ctx, const rcPolyMesh& src, rcPolyMesh& dst)
{
	rcAssert(ctx);
	
	// Destination must be empty.
	rcAssert(dst.verts == 0);
	rcAssert(dst.polys == 0);
	rcAssert(dst.regs == 0);
	rcAssert(dst.areas == 0);
	rcAssert(dst.flags == 0);
	
	dst.nverts = src.nverts;
	dst.npolys = src.npolys;
	dst.maxpolys = src.npolys;
	dst.nvp = src.nvp;
	rcVcopy(dst.bmin, src.bmin);
	rcVcopy(dst.bmax, src.bmax);
	dst.cs = src.cs;
	dst.ch = src.ch;
	dst.borderSize = src.borderSize;
	
	dst.verts = (unsigned short*)rcAlloc(sizeof(unsigned short)*src.nverts*3, RC_ALLOC_PERM);
	if (!dst.verts)
	{
		ctx->log(RC_LOG_ERROR, "rcCopyPolyMesh: Out of memory 'dst.verts' (%d).", src.nverts*3);
		return false;
	}
	memcpy(dst.verts, src.verts, sizeof(unsigned short)*src.nverts*3);
	
	dst.polys = (unsigned short*)rcAlloc(sizeof(unsigned short)*src.npolys*2*src.nvp, RC_ALLOC_PERM);
	if (!dst.polys)
	{
		ctx->log(RC_LOG_ERROR, "rcCopyPolyMesh: Out of memory 'dst.polys' (%d).", src.npolys*2*src.nvp);
		return false;
	}
	memcpy(dst.polys, src.polys, sizeof(unsigned short)*src.npolys*2*src.nvp);
	
	dst.regs = (unsigned short*)rcAlloc(sizeof(unsigned short)*src.npolys, RC_ALLOC_PERM);
	if (!dst.regs)
	{
		ctx->log(RC_LOG_ERROR, "rcCopyPolyMesh: Out of memory 'dst.regs' (%d).", src.npolys);
		return false;
	}
	memcpy(dst.regs, src.regs, sizeof(unsigned short)*src.npolys);
	
	dst.areas = (unsigned char*)rcAlloc(sizeof(unsigned char)*src.npolys, RC_ALLOC_PERM);
	if (!dst.areas)
	{
		ctx->log(RC_LOG_ERROR, "rcCopyPolyMesh: Out of memory 'dst.areas' (%d).", src.npolys);
		return false;
	}
	memcpy(dst.areas, src.areas, sizeof(unsigned char)*src.npolys);
	
	dst.flags = (unsigned short*)rcAlloc(sizeof(unsigned short)*src.npolys, RC_ALLOC_PERM);
	if (!dst.flags)
	{
		ctx->log(RC_LOG_ERROR, "rcCopyPolyMesh: Out of memory 'dst.flags' (%d).", src.npolys);
		return false;
	}
	memcpy(dst.flags, src.flags, sizeof(unsigned char)*src.npolys);
	
	return true;
}
