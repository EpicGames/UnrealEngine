// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "CoreMinimal.h"
#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAlloc.h"
#include "Recast/RecastAssert.h"


static const unsigned RC_UNSET_HEIGHT = 0xffff;

struct rcHeightPatch
{
	inline rcHeightPatch() : data(0), xmin(0), ymin(0), width(0), height(0) {}
	inline ~rcHeightPatch() { rcFree(data); }
	unsigned short* data;
	int xmin, ymin, width, height;
};


inline rcReal vdot2(const rcReal* a, const rcReal* b)
{
	return a[0]*b[0] + a[2]*b[2];
}

inline rcReal vdistSq2(const rcReal* p, const rcReal* q)
{
	const rcReal dx = q[0] - p[0];
	const rcReal dy = q[2] - p[2];
	return dx*dx + dy*dy;
}

inline rcReal vdist2(const rcReal* p, const rcReal* q)
{
	return rcSqrt(vdistSq2(p,q));
}

inline rcReal vcross2(const rcReal* p1, const rcReal* p2, const rcReal* p3)
{ 
	const rcReal u1 = p2[0] - p1[0];
	const rcReal v1 = p2[2] - p1[2];
	const rcReal u2 = p3[0] - p1[0];
	const rcReal v2 = p3[2] - p1[2];
	return u1 * v2 - v1 * u2;
}

static bool circumCircle(const rcReal* p1, const rcReal* p2, const rcReal* p3,
						 rcReal* c, rcReal& r)
{
	static const rcReal EPS = 1e-6f;
	
	const rcReal cp = vcross2(p1, p2, p3);
	if (rcAbs(cp) > EPS)
	{
		const rcReal p1Sq = vdot2(p1,p1);
		const rcReal p2Sq = vdot2(p2,p2);
		const rcReal p3Sq = vdot2(p3,p3);
		c[0] = (p1Sq*(p2[2]-p3[2]) + p2Sq*(p3[2]-p1[2]) + p3Sq*(p1[2]-p2[2])) / (2*cp);
		c[2] = (p1Sq*(p3[0]-p2[0]) + p2Sq*(p1[0]-p3[0]) + p3Sq*(p2[0]-p1[0])) / (2*cp);
		r = vdist2(c, p1);
		return true;
	}

	c[0] = p1[0];
	c[2] = p1[2];
	r = 0;
	return false;
}

static rcReal distPtTri(const rcReal* p, const rcReal* a, const rcReal* b, const rcReal* c)
{
	rcReal v0[3], v1[3], v2[3];
	rcVsub(v0, c,a);
	rcVsub(v1, b,a);
	rcVsub(v2, p,a);

	const rcReal dot00 = vdot2(v0, v0);
	const rcReal dot01 = vdot2(v0, v1);
	const rcReal dot02 = vdot2(v0, v2);
	const rcReal dot11 = vdot2(v1, v1);
	const rcReal dot12 = vdot2(v1, v2);
	
	// Compute barycentric coordinates
	const rcReal invDenom = 1.0f / (dot00 * dot11 - dot01 * dot01);
	const rcReal u = (dot11 * dot02 - dot01 * dot12) * invDenom;
	rcReal v = (dot00 * dot12 - dot01 * dot02) * invDenom;
	
	// If point lies inside the triangle, return interpolated y-coord.
	static const rcReal EPS = 1e-4f;
	if (u >= -EPS && v >= -EPS && (u+v) <= 1+EPS)
	{
		const rcReal y = a[1] + v0[1]*u + v1[1]*v;
		return rcAbs(y-p[1]);
	}
	return RC_REAL_MAX;
}

static rcReal distancePtSeg(const rcReal* pt, const rcReal* p, const rcReal* q)
{
	rcReal pqx = q[0] - p[0];
	rcReal pqy = q[1] - p[1];
	rcReal pqz = q[2] - p[2];
	rcReal dx = pt[0] - p[0];
	rcReal dy = pt[1] - p[1];
	rcReal dz = pt[2] - p[2];
	rcReal d = pqx*pqx + pqy*pqy + pqz*pqz;
	rcReal t = pqx*dx + pqy*dy + pqz*dz;
	if (d > 0)
		t /= d;
	if (t < 0)
		t = 0;
	else if (t > 1)
		t = 1;
	
	dx = p[0] + t*pqx - pt[0];
	dy = p[1] + t*pqy - pt[1];
	dz = p[2] + t*pqz - pt[2];
	
	return dx*dx + dy*dy + dz*dz;
}

static rcReal distancePtSeg2d(const rcReal* pt, const rcReal* p, const rcReal* q)
{
	rcReal pqx = q[0] - p[0];
	rcReal pqz = q[2] - p[2];
	rcReal dx = pt[0] - p[0];
	rcReal dz = pt[2] - p[2];
	rcReal d = pqx*pqx + pqz*pqz;
	rcReal t = pqx*dx + pqz*dz;
	if (d > 0)
		t /= d;
	if (t < 0)
		t = 0;
	else if (t > 1)
		t = 1;
	
	dx = p[0] + t*pqx - pt[0];
	dz = p[2] + t*pqz - pt[2];
	
	return dx*dx + dz*dz;
}

static rcReal distToTriMesh(const rcReal* p, const rcReal* verts, const int /*nverts*/, const int* tris, const int ntris)
{
	rcReal dmin = RC_REAL_MAX;
	for (int i = 0; i < ntris; ++i)
	{
		const rcReal* va = &verts[tris[i*4+0]*3];
		const rcReal* vb = &verts[tris[i*4+1]*3];
		const rcReal* vc = &verts[tris[i*4+2]*3];
		rcReal d = distPtTri(p, va,vb,vc);
		if (d < dmin)
			dmin = d;
	}
	if (dmin == RC_REAL_MAX) return -1;
	return dmin;
}

static rcReal distToPoly(int nvert, const rcReal* verts, const rcReal* p)
{

	rcReal dmin = RC_REAL_MAX;
	int i, j, c = 0;
	for (i = 0, j = nvert-1; i < nvert; j = i++)
	{
		const rcReal* vi = &verts[i*3];
		const rcReal* vj = &verts[j*3];
		if (((vi[2] > p[2]) != (vj[2] > p[2])) &&
			(p[0] < (vj[0]-vi[0]) * (p[2]-vi[2]) / (vj[2]-vi[2]) + vi[0]) )
			c = !c;
		dmin = rcMin(dmin, distancePtSeg2d(p, vj, vi));
	}
	return c ? -dmin : dmin;
}


static unsigned short getHeight(const rcReal fx, const rcReal fy, const rcReal fz,
								const rcReal /*cs*/, const rcReal ics, const rcReal ch,
								const rcHeightPatch& hp)
{
	int ix = (int)rcFloor(fx*ics + 0.01f);
	int iz = (int)rcFloor(fz*ics + 0.01f);
	//@UE BEGIN: clamp in range 0..width-1 and 0..height-1 (breaks array addressing without -1)
	ix = rcClamp(ix-hp.xmin, 0, hp.width-1);
	iz = rcClamp(iz-hp.ymin, 0, hp.height-1);
	//@UE END
	unsigned short h = hp.data[ix+iz*hp.width];
	if (h == RC_UNSET_HEIGHT)
	{
		//@UE BEGIN
		// setting fallback value in case proper height is not found
		h = (unsigned short)rcFloor(fy/ch);
		//@UE END

		// Special case when data might be bad.
		// Find nearest neighbour pixel which has valid height.
		const int off[8*2] = { -1,0, -1,-1, 0,-1, 1,-1, 1,0, 1,1, 0,1, -1,1};
		rcReal dmin = RC_REAL_MAX;
		for (int i = 0; i < 8; ++i)
		{
			const int nx = ix+off[i*2+0];
			const int nz = iz+off[i*2+1];
			if (nx < 0 || nz < 0 || nx >= hp.width || nz >= hp.height) continue;
			const unsigned short nh = hp.data[nx+nz*hp.width];
			if (nh == RC_UNSET_HEIGHT) continue;

			const rcReal d = rcAbs(nh*ch - fy);
			if (d < dmin)
			{
				h = nh;
				dmin = d;
			}
			
/*			const rcReal dx = (rcReal(nx)+0.5f)*cs - fx; 
			const rcReal dz = (rcReal(nz)+0.5f)*cs - fz;
			const rcReal d = dx*dx+dz*dz;
			if (d < dmin)
			{
				h = nh;
				dmin = d;
			} */
		}
	}
	return h;
}


enum EdgeValues
{
	UNDEF = -1,
	HULL = -2,
};

static int findEdge(const int* edges, int nedges, int s, int t)
{
	for (int i = 0; i < nedges; i++)
	{
		const int* e = &edges[i*4];
		if ((e[0] == s && e[1] == t) || (e[0] == t && e[1] == s))
			return i;
	}
	return UNDEF;
}

static int addEdge(rcContext* ctx, int* edges, int& nedges, const int maxEdges, int s, int t, int l, int r)
{
	if (nedges >= maxEdges)
	{
		ctx->log(RC_LOG_ERROR, "addEdge: Too many edges (%d/%d).", nedges, maxEdges);
		return UNDEF;
	}
	
	// Add edge if not already in the triangulation. 
	int e = findEdge(edges, nedges, s, t);
	if (e == UNDEF)
	{
		int* edge = &edges[nedges*4];
		edge[0] = s;
		edge[1] = t;
		edge[2] = l;
		edge[3] = r;
		return nedges++;
	}
	else
	{
		return UNDEF;
	}
}

static void updateLeftFace(int* e, int s, int t, int f)
{
	if (e[0] == s && e[1] == t && e[2] == UNDEF)
		e[2] = f;
	else if (e[1] == s && e[0] == t && e[3] == UNDEF)
		e[3] = f;
}	

static int overlapSegSeg2d(const rcReal* a, const rcReal* b, const rcReal* c, const rcReal* d)
{
	const rcReal a1 = vcross2(a, b, d);
	const rcReal a2 = vcross2(a, b, c);
	if (a1*a2 < 0.0f)
	{
		rcReal a3 = vcross2(c, d, a);
		rcReal a4 = a3 + a2 - a1;
		if (a3 * a4 < 0.0f)
			return 1;
	}	
	return 0;
}

static bool overlapEdges(const rcReal* pts, const int* edges, int nedges, int s1, int t1)
{
	for (int i = 0; i < nedges; ++i)
	{
		const int s0 = edges[i*4+0];
		const int t0 = edges[i*4+1];
		// Same or connected edges do not overlap.
		if (s0 == s1 || s0 == t1 || t0 == s1 || t0 == t1)
			continue;
		if (overlapSegSeg2d(&pts[s0*3],&pts[t0*3], &pts[s1*3],&pts[t1*3]))
			return true;
	}
	return false;
}

static void completeFacet(rcContext* ctx, const rcReal* pts, int npts, int* edges, int& nedges, const int maxEdges, int& nfaces, int e)
{
	static const rcReal EPS = 1e-5f;

	int* edge = &edges[e*4];
	
	// Cache s and t.
	int s,t;
	if (edge[2] == UNDEF)
	{
		s = edge[0];
		t = edge[1];
	}
	else if (edge[3] == UNDEF)
	{
		s = edge[1];
		t = edge[0];
	}
	else
	{
	    // Edge already completed. 
	    return;
	}
    
	// Find best point on left of edge. 
	int pt = npts;
	rcReal c[3] = {0,0,0};
	rcReal r = -1;
	for (int u = 0; u < npts; ++u)
	{
		if (u == s || u == t) continue;
		if (vcross2(&pts[s*3], &pts[t*3], &pts[u*3]) > EPS)
		{
			if (r < 0)
			{
				// The circle is not updated yet, do it now.
				pt = u;
				circumCircle(&pts[s*3], &pts[t*3], &pts[u*3], c, r);
				continue;
			}
			const rcReal d = vdist2(c, &pts[u*3]);
			const rcReal tol = 0.001f;
			if (d > r*(1+tol))
			{
				// Outside current circumcircle, skip.
				continue;
			}
			else if (d < r*(1-tol))
			{
				// Inside safe circumcircle, update circle.
				pt = u;
				circumCircle(&pts[s*3], &pts[t*3], &pts[u*3], c, r);
			}
			else
			{
				// Inside epsilon circum circle, do extra tests to make sure the edge is valid.
				// s-u and t-u cannot overlap with s-pt nor t-pt if they exists.
				if (overlapEdges(pts, edges, nedges, s,u))
					continue;
				if (overlapEdges(pts, edges, nedges, t,u))
					continue;
				// Edge is valid.
				pt = u;
				circumCircle(&pts[s*3], &pts[t*3], &pts[u*3], c, r);
			}
		}
	}
	
	// Add new triangle or update edge info if s-t is on hull. 
	if (pt < npts)
	{
		// Update face information of edge being completed. 
		updateLeftFace(&edges[e*4], s, t, nfaces);
		
		// Add new edge or update face info of old edge. 
		e = findEdge(edges, nedges, pt, s);
		if (e == UNDEF)
		    addEdge(ctx, edges, nedges, maxEdges, pt, s, nfaces, UNDEF);
		else
		    updateLeftFace(&edges[e*4], pt, s, nfaces);
		
		// Add new edge or update face info of old edge. 
		e = findEdge(edges, nedges, t, pt);
		if (e == UNDEF)
		    addEdge(ctx, edges, nedges, maxEdges, t, pt, nfaces, UNDEF);
		else
		    updateLeftFace(&edges[e*4], t, pt, nfaces);
		
		nfaces++;
	}
	else
	{
		updateLeftFace(&edges[e*4], s, t, HULL);
	}
}

static void delaunayHull(rcContext* ctx, const int npts, const rcReal* pts,
						 const int nhull, const int* hull,
						 rcIntArray& tris, rcIntArray& edges)
{
	int nfaces = 0;
	int nedges = 0;
	const int maxEdges = npts*10;
	edges.resize(maxEdges*4);
	
	for (int i = 0, j = nhull-1; i < nhull; j=i++)
		addEdge(ctx, &edges[0], nedges, maxEdges, hull[j],hull[i], HULL, UNDEF);
	
	int currentEdge = 0;
	while (currentEdge < nedges)
	{
		if (edges[currentEdge*4+2] == UNDEF)
			completeFacet(ctx, pts, npts, &edges[0], nedges, maxEdges, nfaces, currentEdge);
		if (edges[currentEdge*4+3] == UNDEF)
			completeFacet(ctx, pts, npts, &edges[0], nedges, maxEdges, nfaces, currentEdge);
		currentEdge++;
	}

	// Create tris
	tris.resize(nfaces*4);
	for (int i = 0; i < nfaces*4; ++i)
		tris[i] = -1;
	
	for (int i = 0; i < nedges; ++i)
	{
		const int* e = &edges[i*4];
		if (e[3] >= 0)
		{
			// Left face
			int* t = &tris[e[3]*4];
			if (t[0] == -1)
			{
				t[0] = e[0];
				t[1] = e[1];
			}
			else if (t[0] == e[1])
				t[2] = e[0];
			else if (t[1] == e[0])
				t[2] = e[1];
		}
		if (e[2] >= 0)
		{
			// Right
			int* t = &tris[e[2]*4];
			if (t[0] == -1)
			{
				t[0] = e[1];
				t[1] = e[0];
			}
			else if (t[0] == e[0])
				t[2] = e[1];
			else if (t[1] == e[1])
				t[2] = e[0];
		}
	}
	
	for (int i = 0; i < tris.size()/4; ++i)
	{
		int* t = &tris[i*4];
		if (t[0] == -1 || t[1] == -1 || t[2] == -1)
		{
			ctx->log(RC_LOG_WARNING, "delaunayHull: Removing dangling face %d [%d,%d,%d].", i, t[0],t[1],t[2]);
			t[0] = tris[tris.size()-4];
			t[1] = tris[tris.size()-3];
			t[2] = tris[tris.size()-2];
			t[3] = tris[tris.size()-1];
			tris.resize(tris.size()-4);
			--i;
		}
	}
}


inline rcReal getJitterX(const int i)
{
	return (((i * 0x8da6b343) & 0xffff) / rcReal(65535.) * 2.0f) - 1.0f;
}

inline rcReal getJitterY(const int i)
{
	return (((i * 0xd8163841) & 0xffff) / rcReal(65535.) * 2.0f) - 1.0f;
}

static bool buildPolyDetail(rcContext* ctx, const rcReal* in, const int nin,
							const rcReal sampleDist, const rcReal sampleMaxError,
							const rcCompactHeightfield& chf, const rcHeightPatch& hp,
							rcReal* verts, int& nverts, rcIntArray& tris,
							rcIntArray& edges, rcIntArray& samples)
{
	static const int MAX_VERTS = 127;
	static const int MAX_TRIS = 255;	// Max tris for delaunay is 2n-2-k (n=num verts, k=num hull verts).
	static const int MAX_VERTS_PER_EDGE = 32;
	rcReal edge[(MAX_VERTS_PER_EDGE+1)*3];
	int hull[MAX_VERTS];
	int nhull = 0;

	nverts = 0;

	for (int i = 0; i < nin; ++i)
		rcVcopy(&verts[i*3], &in[i*3]);
	nverts = nin;
	
	const rcReal cs = chf.cs;
	const rcReal ics = 1.0f/cs;
	
	// Tessellate outlines.
	// This is done in separate pass in order to ensure
	// seamless height values across the ply boundaries.
	if (sampleDist > 0)
	{
		for (int i = 0, j = nin-1; i < nin; j=i++)
		{
			const rcReal* vj = &in[j*3];
			const rcReal* vi = &in[i*3];
			bool swapped = false;
			// Make sure the segments are always handled in same order
			// using lexological sort or else there will be seams.
			if (rcAbs(vj[0]-vi[0]) < 1e-6f)
			{
				if (vj[2] > vi[2])
				{
					rcSwap(vj,vi);
					swapped = true;
				}
			}
			else
			{
				if (vj[0] > vi[0])
				{
					rcSwap(vj,vi);
					swapped = true;
				}
			}
			// Create samples along the edge.
			rcReal dx = vi[0] - vj[0];
			rcReal dy = vi[1] - vj[1];
			rcReal dz = vi[2] - vj[2];
			rcReal d = rcSqrt(dx*dx + dz*dz);
			int nn = 1 + (int)rcFloor(d/sampleDist);
			if (nn >= MAX_VERTS_PER_EDGE) nn = MAX_VERTS_PER_EDGE-1;
			if (nverts+nn >= MAX_VERTS)
				nn = MAX_VERTS-1-nverts;
			
			for (int k = 0; k <= nn; ++k)
			{
				rcReal u = (rcReal)k/(rcReal)nn;
				rcReal* pos = &edge[k*3];
				pos[0] = vj[0] + dx*u;
				pos[1] = vj[1] + dy*u;
				pos[2] = vj[2] + dz*u;
				pos[1] = getHeight(pos[0],pos[1],pos[2], cs, ics, chf.ch, hp)*chf.ch;
			}
			// Simplify samples.
			int idx[MAX_VERTS_PER_EDGE] = {0,nn};
			int nidx = 2;
			for (int k = 0; k < nidx-1; )
			{
				const int a = idx[k];
				const int b = idx[k+1];
				const rcReal* va = &edge[a*3];
				const rcReal* vb = &edge[b*3];
				// Find maximum deviation along the segment.
				rcReal maxd = 0;
				int maxi = -1;
				for (int m = a+1; m < b; ++m)
				{
					rcReal dev = distancePtSeg(&edge[m*3],va,vb);
					if (dev > maxd)
					{
						maxd = dev;
						maxi = m;
					}
				}
				// If the max deviation is larger than accepted error,
				// add new point, else continue to next segment.
				if (maxi != -1 && maxd > rcSqr(sampleMaxError))
				{
					for (int m = nidx; m > k; --m)
						idx[m] = idx[m-1];
					idx[k+1] = maxi;
					nidx++;
				}
				else
				{
					++k;
				}
			}
			
			hull[nhull++] = j;
			// Add new vertices.
			if (swapped)
			{
				for (int k = nidx-2; k > 0; --k)
				{
					rcVcopy(&verts[nverts*3], &edge[idx[k]*3]);
					hull[nhull++] = nverts;
					nverts++;
				}
			}
			else
			{
				for (int k = 1; k < nidx-1; ++k)
				{
					rcVcopy(&verts[nverts*3], &edge[idx[k]*3]);
					hull[nhull++] = nverts;
					nverts++;
				}
			}
		}
	}
	

	// Tessellate the base mesh.
	edges.resize(0);
	tris.resize(0);

	delaunayHull(ctx, nverts, verts, nhull, hull, tris, edges);
	
	if (tris.size() == 0)
	{
		// Could not triangulate the poly, make sure there is some valid data there.
		ctx->log(RC_LOG_WARNING, "buildPolyDetail: Could not triangulate polygon, adding default data.");
		for (int i = 2; i < nverts; ++i)
		{
			tris.push(0);
			tris.push(i-1);
			tris.push(i);
			tris.push(0);
		}
		return true;
	}

	if (sampleDist > 0)
	{
		// Create sample locations in a grid.
		rcReal bmin[3], bmax[3];
		rcVcopy(bmin, in);
		rcVcopy(bmax, in);
		for (int i = 1; i < nin; ++i)
		{
			rcVmin(bmin, &in[i*3]);
			rcVmax(bmax, &in[i*3]);
		}
		int x0 = (int)rcFloor(bmin[0]/sampleDist);
		int x1 = (int)rcCeil(bmax[0]/sampleDist);
		int z0 = (int)rcFloor(bmin[2]/sampleDist);
		int z1 = (int)rcCeil(bmax[2]/sampleDist);
		samples.resize(0);
		for (int z = z0; z < z1; ++z)
		{
			for (int x = x0; x < x1; ++x)
			{
				rcReal pt[3];
				pt[0] = x*sampleDist;
				pt[1] = (bmax[1]+bmin[1])*0.5f;
				pt[2] = z*sampleDist;
				// Make sure the samples are not too close to the edges.
				if (distToPoly(nin,in,pt) > -sampleDist/2) continue;
				samples.push(x);
				samples.push(getHeight(pt[0], pt[1], pt[2], cs, ics, chf.ch, hp));
				samples.push(z);
				samples.push(0); // Not added
			}
		}
				
		// Add the samples starting from the one that has the most
		// error. The procedure stops when all samples are added
		// or when the max error is within treshold.
		const int nsamples = samples.size()/4;
		for (int iter = 0; iter < nsamples; ++iter)
		{
			if (nverts >= MAX_VERTS)
				break;

			// Find sample with most error.
			rcReal bestpt[3] = {0,0,0};
			rcReal bestd = 0;
			int besti = -1;
			for (int i = 0; i < nsamples; ++i)
			{
				const int* s = &samples[i*4];
				if (s[3]) continue; // skip added.
				rcReal pt[3];
				// The sample location is jittered to get rid of some bad triangulations
				// which are cause by symmetrical data from the grid structure.
				pt[0] = s[0]*sampleDist + getJitterX(i)*cs*0.1f;
				pt[1] = s[1]*chf.ch;
				pt[2] = s[2]*sampleDist + getJitterY(i)*cs*0.1f;
				rcReal d = distToTriMesh(pt, verts, nverts, &tris[0], tris.size()/4);
				if (d < 0) continue; // did not hit the mesh.
				if (d > bestd)
				{
					bestd = d;
					besti = i;
					rcVcopy(bestpt,pt);
				}
			}
			// If the max error is within accepted threshold, stop tesselating.
			if (bestd <= sampleMaxError || besti == -1)
				break;
			// Mark sample as added.
			samples[besti*4+3] = 1;
			// Add the new sample point.
			rcVcopy(&verts[nverts*3],bestpt);
			nverts++;
			
			// Create new triangulation.
			// TODO: Incremental add instead of full rebuild.
			edges.resize(0);
			tris.resize(0);
			delaunayHull(ctx, nverts, verts, nhull, hull, tris, edges);
		}		
	}

	const int ntris = tris.size()/4;
	if (ntris > MAX_TRIS)
	{
		tris.resize(MAX_TRIS*4);
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Shrinking triangle count from %d to max %d.", ntris, MAX_TRIS);
	}

	return true;
}

static void getHeightData(const rcCompactHeightfield& chf,
						  const unsigned short* poly, const int npoly,
						  const unsigned short* verts, const rcBorderSize bs,	//@UE
						  rcHeightPatch& hp, rcIntArray& stack)
{
	// Floodfill the heightfield to get 2D height data,
	// starting at vertex locations as seeds.
	
	// Note: Reads to the compact heightfield are offset by border size (bs)
	// since border size offset is already removed from the polymesh vertices.
	
	memset(hp.data, 0, sizeof(unsigned short)*hp.width*hp.height);
	
	stack.resize(0);
	
	static const int offset[9*2] =
	{
		0,0, -1,-1, 0,-1, 1,-1, 1,0, 1,1, 0,1, -1,1, -1,0,
	};
	
	// Use poly vertices as seed points for the flood fill.
	for (int j = 0; j < npoly; ++j)
	{
		int cx = 0, cz = 0, ci =-1;
		int dmin = RC_UNSET_HEIGHT;
		for (int k = 0; k < 9; ++k)
		{
			const int ax = (int)verts[poly[j]*3+0] + offset[k*2+0];
			const int ay = (int)verts[poly[j]*3+1];
			const int az = (int)verts[poly[j]*3+2] + offset[k*2+1];
			if (ax < hp.xmin || ax >= hp.xmin+hp.width ||
				az < hp.ymin || az >= hp.ymin+hp.height)
				continue;
			
			const rcCompactCell& c = chf.cells[(ax+bs.low)+(az+bs.low)*chf.width];	//@UE
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				int d = rcAbs(ay - (int)s.y);
				if (d < dmin)
				{
					cx = ax;
					cz = az;
					ci = i;
					dmin = d;
				}
			}
		}
		if (ci != -1)
		{
			stack.push(cx);
			stack.push(cz);
			stack.push(ci);
		}
	}
	
	// Find center of the polygon using flood fill.
	int pcx = 0, pcz = 0;
	for (int j = 0; j < npoly; ++j)
	{
		pcx += (int)verts[poly[j]*3+0];
		pcz += (int)verts[poly[j]*3+2];
	}
	pcx /= npoly;
	pcz /= npoly;
	
	for (int i = 0; i < stack.size(); i += 3)
	{
		int cx = stack[i+0];
		int cy = stack[i+1];
		int idx = cx-hp.xmin+(cy-hp.ymin)*hp.width;
		hp.data[idx] = 1;
	}
	
	while (stack.size() > 0)
	{
		int ci = stack.pop();
		int cy = stack.pop();
		int cx = stack.pop();
		
		// Check if close to center of the polygon.
		if (rcAbs(cx-pcx) <= 1 && rcAbs(cy-pcz) <= 1)
		{
			stack.resize(0);
			stack.push(cx);
			stack.push(cy);
			stack.push(ci);
			break;
		}
		
		const rcCompactSpan& cs = chf.spans[ci];
		
		for (int dir = 0; dir < 4; ++dir)
		{
			if (rcGetCon(cs, dir) == RC_NOT_CONNECTED) continue;
			
			const int ax = cx + rcGetDirOffsetX(dir);
			const int ay = cy + rcGetDirOffsetY(dir);
			
			if (ax < hp.xmin || ax >= (hp.xmin+hp.width) ||
				ay < hp.ymin || ay >= (hp.ymin+hp.height))
				continue;
			
			if (hp.data[ax-hp.xmin+(ay-hp.ymin)*hp.width] != 0)
				continue;
			
			const int ai = (int)chf.cells[(ax+bs.low)+(ay+bs.low)*chf.width].index + rcGetCon(cs, dir);	//@UE

			int idx = ax-hp.xmin+(ay-hp.ymin)*hp.width;
			hp.data[idx] = 1;
			
			stack.push(ax);
			stack.push(ay);
			stack.push(ai);
		}
	}

	memset(hp.data, 0xff, sizeof(unsigned short)*hp.width*hp.height);

	// Mark start locations.
	for (int i = 0; i < stack.size(); i += 3)
	{
		int cx = stack[i+0];
		int cy = stack[i+1];
		int ci = stack[i+2];
		int idx = cx-hp.xmin+(cy-hp.ymin)*hp.width;
		const rcCompactSpan& cs = chf.spans[ci];
		hp.data[idx] = cs.y;
	}
	
	static const int RETRACT_SIZE = 256;
	int head = 0;
	
	while (head*3 < stack.size())
	{
		int cx = stack[head*3+0];
		int cy = stack[head*3+1];
		int ci = stack[head*3+2];
		head++;
		if (head >= RETRACT_SIZE)
		{
			head = 0;
			if (stack.size() > RETRACT_SIZE*3)
				memmove(&stack[0], &stack[RETRACT_SIZE*3], sizeof(int)*(stack.size()-RETRACT_SIZE*3));
			stack.resize(stack.size()-RETRACT_SIZE*3);
		}

		const rcCompactSpan& cs = chf.spans[ci];
		for (int dir = 0; dir < 4; ++dir)
		{
			if (rcGetCon(cs, dir) == RC_NOT_CONNECTED) continue;
			
			const int ax = cx + rcGetDirOffsetX(dir);
			const int ay = cy + rcGetDirOffsetY(dir);
			
			if (ax < hp.xmin || ax >= (hp.xmin+hp.width) ||
				ay < hp.ymin || ay >= (hp.ymin+hp.height))
				continue;
			
			if (hp.data[ax-hp.xmin+(ay-hp.ymin)*hp.width] != RC_UNSET_HEIGHT)
				continue;
			
			const int ai = (int)chf.cells[(ax+bs.low)+(ay+bs.low)*chf.width].index + rcGetCon(cs, dir);	//@UE
			
			const rcCompactSpan& as = chf.spans[ai];
			int idx = ax-hp.xmin+(ay-hp.ymin)*hp.width;
			hp.data[idx] = as.y;

			stack.push(ax);
			stack.push(ay);
			stack.push(ai);
		}
	}
	
}

static unsigned char getEdgeFlags(const rcReal* va, const rcReal* vb,
								  const rcReal* vpoly, const int npoly)
{
	// Return true if edge (va,vb) is part of the polygon.
	static const rcReal thrSqr = rcSqr(0.001f);
	for (int i = 0, j = npoly-1; i < npoly; j=i++)
	{
		if (distancePtSeg2d(va, &vpoly[j*3], &vpoly[i*3]) < thrSqr && 
			distancePtSeg2d(vb, &vpoly[j*3], &vpoly[i*3]) < thrSqr)
			return 1;
	}
	return 0;
}

static unsigned char getTriFlags(const rcReal* va, const rcReal* vb, const rcReal* vc,
								 const rcReal* vpoly, const int npoly)
{
	unsigned char flags = 0;
	flags |= getEdgeFlags(va,vb,vpoly,npoly) << 0;
	flags |= getEdgeFlags(vb,vc,vpoly,npoly) << 2;
	flags |= getEdgeFlags(vc,va,vpoly,npoly) << 4;
	return flags;
}

/// @par
///
/// See the #rcConfig documentation for more information on the configuration parameters.
///
/// @see rcAllocPolyMeshDetail, rcPolyMesh, rcCompactHeightfield, rcPolyMeshDetail, rcConfig
bool rcBuildPolyMeshDetail(rcContext* ctx, const rcPolyMesh& mesh, const rcCompactHeightfield& chf,
						   const rcReal sampleDist, const rcReal sampleMaxError,
						   rcPolyMeshDetail& dmesh)
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_BUILD_POLYMESHDETAIL);

	if (mesh.nverts == 0 || mesh.npolys == 0)
		return true;
	
	const int nvp = mesh.nvp;
	const rcReal cs = mesh.cs;
	const rcReal ch = mesh.ch;
	const rcReal* orig = mesh.bmin;
	const rcBorderSize borderSize = mesh.borderSize;	//@UE
	
	rcIntArray edges(64);
	rcIntArray tris(512);
	rcIntArray stack(512);
	rcIntArray samples(512);
	rcReal verts[256*3];
	rcHeightPatch hp;
	int nPolyVerts = 0;
	int maxhw = 0, maxhh = 0;
	
	rcScopedDelete<int> bounds = (int*)rcAlloc(sizeof(int)*mesh.npolys*4, RC_ALLOC_TEMP);
	if (!bounds)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'bounds' (%d).", mesh.npolys*4);
		return false;
	}
	rcScopedDelete<rcReal> poly = (rcReal*)rcAlloc(sizeof(rcReal)*nvp*3, RC_ALLOC_TEMP);
	if (!poly)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'poly' (%d).", nvp*3);
		return false;
	}
	
	// Find max size for a polygon area.
	for (int i = 0; i < mesh.npolys; ++i)
	{
		const unsigned short* p = &mesh.polys[i*nvp*2];
		int& xmin = bounds[i*4+0];
		int& xmax = bounds[i*4+1];
		int& ymin = bounds[i*4+2];
		int& ymax = bounds[i*4+3];
		xmin = chf.width;
		xmax = 0;
		ymin = chf.height;
		ymax = 0;
		for (int j = 0; j < nvp; ++j)
		{
			if(p[j] == RC_MESH_NULL_IDX) break;
			const unsigned short* v = &mesh.verts[p[j]*3];
			xmin = rcMin(xmin, (int)v[0]);
			xmax = rcMax(xmax, (int)v[0]);
			ymin = rcMin(ymin, (int)v[2]);
			ymax = rcMax(ymax, (int)v[2]);
			nPolyVerts++;
		}
		xmin = rcMax(0,xmin-1);
		xmax = rcMin(chf.width,xmax+1);
		ymin = rcMax(0,ymin-1);
		ymax = rcMin(chf.height,ymax+1);
		if (xmin >= xmax || ymin >= ymax) continue;
		maxhw = rcMax(maxhw, xmax-xmin);
		maxhh = rcMax(maxhh, ymax-ymin);
	}
	
	hp.data = (unsigned short*)rcAlloc(sizeof(unsigned short)*maxhw*maxhh, RC_ALLOC_TEMP);
	if (!hp.data)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'hp.data' (%d).", maxhw*maxhh);
		return false;
	}
	
	dmesh.nmeshes = mesh.npolys;
	dmesh.nverts = 0;
	dmesh.ntris = 0;
	dmesh.meshes = (unsigned int*)rcAlloc(sizeof(unsigned int)*dmesh.nmeshes*4, RC_ALLOC_PERM);
	if (!dmesh.meshes)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'dmesh.meshes' (%d).", dmesh.nmeshes*4);
		return false;
	}

	int vcap = nPolyVerts+nPolyVerts/2;
	int tcap = vcap*2;

	dmesh.nverts = 0;
	dmesh.verts = (rcReal*)rcAlloc(sizeof(rcReal)*vcap*3, RC_ALLOC_PERM);
	if (!dmesh.verts)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'dmesh.verts' (%d).", vcap*3);
		return false;
	}
	dmesh.ntris = 0;
	dmesh.tris = (unsigned char*)rcAlloc(sizeof(unsigned char*)*tcap*4, RC_ALLOC_PERM);
	if (!dmesh.tris)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'dmesh.tris' (%d).", tcap*4);
		return false;
	}
	
	for (int i = 0; i < mesh.npolys; ++i)
	{
		const unsigned short* p = &mesh.polys[i*nvp*2];
		
		// Store polygon vertices for processing.
		int npoly = 0;
		for (int j = 0; j < nvp; ++j)
		{
			if(p[j] == RC_MESH_NULL_IDX) break;
			const unsigned short* v = &mesh.verts[p[j]*3];
			poly[j*3+0] = v[0]*cs;
			poly[j*3+1] = v[1]*ch;
			poly[j*3+2] = v[2]*cs;
			npoly++;
		}
		
		// Get the height data from the area of the polygon.
		hp.xmin = bounds[i*4+0];
		hp.ymin = bounds[i*4+2];
		hp.width = bounds[i*4+1]-bounds[i*4+0];
		hp.height = bounds[i*4+3]-bounds[i*4+2];
		getHeightData(chf, p, npoly, mesh.verts, borderSize, hp, stack);
		
		// Build detail mesh.
		int nverts = 0;
		if (!buildPolyDetail(ctx, poly, npoly,
							 sampleDist, sampleMaxError,
							 chf, hp, verts, nverts, tris,
							 edges, samples))
		{
			return false;
		}

		// Move detail verts to world space.
		for (int j = 0; j < nverts; ++j)
		{
			verts[j*3+0] += orig[0];
			verts[j*3+1] += orig[1] + chf.ch; // Is this offset necessary?
			verts[j*3+2] += orig[2];
		}
		// Offset poly too, will be used to flag checking.
		for (int j = 0; j < npoly; ++j)
		{
			poly[j*3+0] += orig[0];
			poly[j*3+1] += orig[1];
			poly[j*3+2] += orig[2];
		}
	
		// Store detail submesh.
		const int ntris = tris.size()/4;

		dmesh.meshes[i*4+0] = (unsigned int)dmesh.nverts;
		dmesh.meshes[i*4+1] = (unsigned int)nverts;
		dmesh.meshes[i*4+2] = (unsigned int)dmesh.ntris;
		dmesh.meshes[i*4+3] = (unsigned int)ntris;		
		
		// Store vertices, allocate more memory if necessary.
		if (dmesh.nverts+nverts > vcap)
		{
			while (dmesh.nverts+nverts > vcap)
				vcap += 256;
				
			rcReal* newv = (rcReal*)rcAlloc(sizeof(rcReal)*vcap*3, RC_ALLOC_PERM);
			if (!newv)
			{
				ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'newv' (%d).", vcap*3);
				return false;
			}
			if (dmesh.nverts)
				memcpy(newv, dmesh.verts, sizeof(rcReal)*3*dmesh.nverts);
			rcFree(dmesh.verts);
			dmesh.verts = newv;
		}
		for (int j = 0; j < nverts; ++j)
		{
			dmesh.verts[dmesh.nverts*3+0] = verts[j*3+0];
			dmesh.verts[dmesh.nverts*3+1] = verts[j*3+1];
			dmesh.verts[dmesh.nverts*3+2] = verts[j*3+2];
			dmesh.nverts++;
		}
		
		// Store triangles, allocate more memory if necessary.
		if (dmesh.ntris+ntris > tcap)
		{
			while (dmesh.ntris+ntris > tcap)
				tcap += 256;
			unsigned char* newt = (unsigned char*)rcAlloc(sizeof(unsigned char)*tcap*4, RC_ALLOC_PERM);
			if (!newt)
			{
				ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'newt' (%d).", tcap*4);
				return false;
			}
			if (dmesh.ntris)
				memcpy(newt, dmesh.tris, sizeof(unsigned char)*4*dmesh.ntris);
			rcFree(dmesh.tris);
			dmesh.tris = newt;
		}
		for (int j = 0; j < ntris; ++j)
		{
			const int* t = &tris[j*4];
			dmesh.tris[dmesh.ntris*4+0] = (unsigned char)t[0];
			dmesh.tris[dmesh.ntris*4+1] = (unsigned char)t[1];
			dmesh.tris[dmesh.ntris*4+2] = (unsigned char)t[2];
			dmesh.tris[dmesh.ntris*4+3] = getTriFlags(&verts[t[0]*3], &verts[t[1]*3], &verts[t[2]*3], poly, npoly);
			dmesh.ntris++;
		}
	}
		
	ctx->stopTimer(RC_TIMER_BUILD_POLYMESHDETAIL);

	return true;
}

/// @see rcAllocPolyMeshDetail, rcPolyMeshDetail
bool rcMergePolyMeshDetails(rcContext* ctx, rcPolyMeshDetail** meshes, const int nmeshes, rcPolyMeshDetail& mesh)
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_MERGE_POLYMESHDETAIL);

	int maxVerts = 0;
	int maxTris = 0;
	int maxMeshes = 0;

	for (int i = 0; i < nmeshes; ++i)
	{
		if (!meshes[i]) continue;
		maxVerts += meshes[i]->nverts;
		maxTris += meshes[i]->ntris;
		maxMeshes += meshes[i]->nmeshes;
	}

	mesh.nmeshes = 0;
	mesh.meshes = (unsigned int*)rcAlloc(sizeof(unsigned int)*maxMeshes*4, RC_ALLOC_PERM);
	if (!mesh.meshes)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'pmdtl.meshes' (%d).", maxMeshes*4);
		return false;
	}

	mesh.ntris = 0;
	mesh.tris = (unsigned char*)rcAlloc(sizeof(unsigned char)*maxTris*4, RC_ALLOC_PERM);
	if (!mesh.tris)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'dmesh.tris' (%d).", maxTris*4);
		return false;
	}

	mesh.nverts = 0;
	mesh.verts = (rcReal*)rcAlloc(sizeof(rcReal)*maxVerts*3, RC_ALLOC_PERM);
	if (!mesh.verts)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildPolyMeshDetail: Out of memory 'dmesh.verts' (%d).", maxVerts*3);
		return false;
	}
	
	// Merge datas.
	for (int i = 0; i < nmeshes; ++i)
	{
		rcPolyMeshDetail* dm = meshes[i];
		if (!dm) continue;
		for (int j = 0; j < dm->nmeshes; ++j)
		{
			unsigned int* dst = &mesh.meshes[mesh.nmeshes*4];
			unsigned int* src = &dm->meshes[j*4];
			dst[0] = (unsigned int)mesh.nverts+src[0];
			dst[1] = src[1];
			dst[2] = (unsigned int)mesh.ntris+src[2];
			dst[3] = src[3];
			mesh.nmeshes++;
		}
			
		for (int k = 0; k < dm->nverts; ++k)
		{
			rcVcopy(&mesh.verts[mesh.nverts*3], &dm->verts[k*3]);
			mesh.nverts++;
		}
		for (int k = 0; k < dm->ntris; ++k)
		{
			mesh.tris[mesh.ntris*4+0] = dm->tris[k*4+0];
			mesh.tris[mesh.ntris*4+1] = dm->tris[k*4+1];
			mesh.tris[mesh.ntris*4+2] = dm->tris[k*4+2];
			mesh.tris[mesh.ntris*4+3] = dm->tris[k*4+3];
			mesh.ntris++;
		}
	}

	ctx->stopTimer(RC_TIMER_MERGE_POLYMESHDETAIL);
	
	return true;
}

