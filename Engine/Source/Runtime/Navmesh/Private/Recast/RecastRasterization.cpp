// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "CoreMinimal.h"
#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAlloc.h"
#include "Recast/RecastAssert.h"
#include "HAL/ConsoleManager.h"

//@UE BEGIN
namespace UE::Recast::Private
{
	static bool bEnableSpanHeightRasterizationFix = true;
	static FAutoConsoleVariableRef CVarEnableSpanHeightRasterizationFix(TEXT("ai.nav.EnableSpanHeightRasterizationFix"), bEnableSpanHeightRasterizationFix, TEXT("Active by default. Enable rasterization fix for span height."), ECVF_Default);
}
//@UE END

inline bool overlapBounds(const rcReal* amin, const rcReal* amax, const rcReal* bmin, const rcReal* bmax)
{
	bool overlap = true;
	overlap = (amin[0] > bmax[0] || amax[0] < bmin[0]) ? false : overlap;
	overlap = (amin[1] > bmax[1] || amax[1] < bmin[1]) ? false : overlap;
	overlap = (amin[2] > bmax[2] || amax[2] < bmin[2]) ? false : overlap;
	return overlap;
}

inline bool overlapInterval(unsigned short amin, unsigned short amax,
							unsigned short bmin, unsigned short bmax)
{
	if (amax < bmin) return false;
	if (amin > bmax) return false;
	return true;
}


static rcSpan* allocSpan(rcHeightfield& hf)
{
	// If running out of memory, allocate new page and update the freelist.
	if (!hf.freelist || !hf.freelist->next)
	{
		// Create new page.
		// Allocate memory for the new pool.
		rcSpanPool* pool = (rcSpanPool*)rcAlloc(sizeof(rcSpanPool), RC_ALLOC_PERM);
		if (!pool) return 0;
		pool->next = 0;
		// Add the pool into the list of pools.
		pool->next = hf.pools;
		hf.pools = pool;
		// Add new items to the free list.
		rcSpan* freelist = hf.freelist;
		rcSpan* head = &pool->items[0];
		rcSpan* it = &pool->items[RC_SPANS_PER_POOL];
		do
		{
			--it;
			it->next = freelist;
			freelist = it;
		}
		while (it != head);
		hf.freelist = it;
	}
	
	// Pop item from in front of the free list.
	rcSpan* it = hf.freelist;
	hf.freelist = hf.freelist->next;
	return it;
}

static void freeSpan(rcHeightfield& hf, rcSpan* ptr)
{
	if (!ptr) return;
	// Add the node in front of the free list.
	ptr->next = hf.freelist;
	hf.freelist = ptr;
}

// @UE BEGIN
// Merge overlapping spans
static void mergeSpanData(rcSpanData& span, const rcSpanUInt smin, const rcSpanUInt smax, 
					const unsigned char area, const int flagMergeThr)
{
	// For spans whose tops are really close to each other, prefer walkable areas.
	// This is done in order to remove aliasing (similar to z-fighting) on surfaces close to each other.
	if (rcAbs((int)span.smax - (int)smax) <= flagMergeThr)
	{
		span.area = rcMax(span.area, area);
	}
	else
	{
		// Use the new spans area if it will become the top.
		if (smax > span.smax)
			span.area = area;
	}

	// Merge height intervals.
	if (smin < span.smin)
		span.smin = smin;
	if (smax > span.smax)
		span.smax = smax;
}
// @UE END

static void addSpan(rcHeightfield& hf, const int x, const int y,
					const rcSpanUInt smin, const rcSpanUInt smax,
					const unsigned char area, const int flagMergeThr)
{
	
	int idx = x + y*hf.width;
	
	rcSpan* s = allocSpan(hf);
	s->data.smin = smin;
	s->data.smax = smax;
	s->data.area = area;
	s->next = 0;
	
	// Empty cell, add the first span.
	if (!hf.spans[idx])
	{
		hf.spans[idx] = s;
		return;
	}
	rcSpan* prev = 0;
	rcSpan* cur = hf.spans[idx];
	
	// Insert and merge spans.
	while (cur)
	{
		if (cur->data.smin > s->data.smax)
		{
			// Current span is further than the new span, break.
			break;
		}
		else if (cur->data.smax < s->data.smin)
		{
			// Current span is before the new span advance.
			prev = cur;
			cur = cur->next;
		}
		else
		{
			// @UE BEGIN
			mergeSpanData(s->data, cur->data.smin, cur->data.smax, cur->data.area, flagMergeThr);
			// @UE END
			
			// Remove current span.
			rcSpan* next = cur->next;
			freeSpan(hf, cur);
			if (prev)
				prev->next = next;
			else
				hf.spans[idx] = next;
			cur = next;
		}
	}
	
	// Insert new span.
	if (prev)
	{
		s->next = prev->next;
		prev->next = s;
	}
	else
	{
		s->next = hf.spans[idx];
		hf.spans[idx] = s;
	}
}

/// @par
///
/// The span addition can be set to favor flags. If the span is merged to
/// another span and the new @p smax is within @p flagMergeThr units
/// from the existing span, the span flags are merged.
///
/// @see rcHeightfield, rcSpan.
void rcAddSpan(rcContext* /*ctx*/, rcHeightfield& hf, const int x, const int y,
			   const rcSpanUInt smin, const rcSpanUInt smax,
			   const unsigned char area, const int flagMergeThr)
{
//	rcAssert(ctx);
	addSpan(hf, x,y, smin, smax, area, flagMergeThr);
}

void rcAddSpans(rcContext* /*ctx*/, rcHeightfield& hf, const int flagMergeThr,
				const rcSpanCache* cachedSpans, const int nspans)
{
	const rcSpanCache* cachedInfo = cachedSpans;
	for (int i = 0; i < nspans; i++, cachedInfo++)
	{
		addSpan(hf, cachedInfo->x, cachedInfo->y, cachedInfo->data.smin, cachedInfo->data.smax, cachedInfo->data.area, flagMergeThr);
	}
}

int rcCountSpans(rcContext* /*ctx*/, rcHeightfield& hf)
{
	if (hf.width > 0xffff || hf.height > 0xffff)
	{
		return 0;
	}

	int numSpans = 0;
	for (rcSpanPool* pool = hf.pools; pool; pool = pool->next)
	{
		numSpans += RC_SPANS_PER_POOL;
	}

	for (rcSpan* s = hf.freelist; s; s = s->next)
	{
		numSpans--;
	}

	return numSpans;
}

void rcCacheSpans(rcContext* /*ctx*/, rcHeightfield& hf, rcSpanCache* cachedSpans)
{
	rcSpanCache* cachedInfo = cachedSpans;
	for (int iz = 0; iz < hf.height; iz++)
	{
		for (int ix = 0; ix < hf.width; ix++)
		{
			const int idx = ix + (iz * hf.width);
			for (rcSpan* s = hf.spans[idx]; s; s = s->next)
			{
				cachedInfo->x = (unsigned short)ix;
				cachedInfo->y = (unsigned short)iz;
				cachedInfo->data = s->data;
				cachedInfo++;
			}
		}
	}
}

static int clipPoly(const rcReal* in, int n, rcReal* out, rcReal pnx, rcReal pnz, rcReal pd)
{
	rcReal d[12];
	for (int i = 0; i < n; ++i)
		d[i] = pnx*in[i*3+0] + pnz*in[i*3+2] + pd;
	
	int m = 0;
	for (int i = 0, j = n-1; i < n; j=i, ++i)
	{
		bool ina = d[j] >= 0;
		bool inb = d[i] >= 0;
		if (ina != inb)
		{
			rcReal s = d[j] / (d[j] - d[i]);
			out[m*3+0] = in[j*3+0] + (in[i*3+0] - in[j*3+0])*s;
			out[m*3+1] = in[j*3+1] + (in[i*3+1] - in[j*3+1])*s;
			out[m*3+2] = in[j*3+2] + (in[i*3+2] - in[j*3+2])*s;
			m++;
		}
		if (inb)
		{
			out[m*3+0] = in[i*3+0];
			out[m*3+1] = in[i*3+1];
			out[m*3+2] = in[i*3+2];
			m++;
		}
	}
	return m;
}

#if EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER

#define TEST_NEW_RASTERIZER (0)
#define TEST_COVERAGE (0)

static inline int intMax(int a, int b)
{
	return a < b ? b : a;
}

static inline int intMin(int a, int b)
{
	return a < b ? a : b;
}

#if TEST_COVERAGE
static inline void markSpanSample(rcHeightfield& hf, const int x, const int y)
{
#if TEST_NEW_RASTERIZER
	rcAssert(x >= -1 && x < hf.width + 1 && y >= -1 && y < hf.height + 1);
#endif
	hf.RowExt[y + 1].MinCol = intMin(hf.RowExt[y + 1].MinCol, x);
	hf.RowExt[y + 1].MaxCol = intMax(hf.RowExt[y + 1].MaxCol, x);

}
#endif

static inline void addFlatSpanSample(rcHeightfield& hf, const int x, const int y)
{
#if TEST_NEW_RASTERIZER
	rcAssert(x >= -1 && x < hf.width + 1 && y >= -1 && y < hf.height + 1);
#endif
#if TEST_COVERAGE
	rcAssert(hf.RowExt[y + 1].MinCol == intMin(hf.RowExt[y + 1].MinCol, x));
	rcAssert(hf.RowExt[y + 1].MaxCol == intMax(hf.RowExt[y + 1].MaxCol, x));
#endif
	hf.RowExt[y + 1].MinCol = intMin(hf.RowExt[y + 1].MinCol, x);
	hf.RowExt[y + 1].MaxCol = intMax(hf.RowExt[y + 1].MaxCol, x);

}

static inline int SampleIndex(rcHeightfield const& hf, const int x, const int y)
{
#if TEST_NEW_RASTERIZER
	rcAssert(x >= -1 && x < hf.width + 1 && y >= -1 && y < hf.height + 1);
#endif
	return x + 1 + (y + 1)*(hf.width + 2);
}

static inline void addSpanSample(rcHeightfield& hf, const int x, const int y, int sint)
{
	addFlatSpanSample(hf, x, y);
	int idx = SampleIndex(hf, x, y);
	rcTempSpan& Temp = hf.tempspans[idx];

	Temp.sminmax[0] = Temp.sminmax[0] > sint ? sint : Temp.sminmax[0];
	Temp.sminmax[1] = Temp.sminmax[1] < sint ? sint : Temp.sminmax[1];
}


static inline void intersectX(const rcReal* v0, const rcReal* edge, rcReal cx, rcReal *pnt)
{
	rcReal t = rcClamp((cx - v0[0]) * edge[9 + 0], 0.0f, 1.0f);  // inverses

	pnt[0] = v0[0] + t * edge[0];
	pnt[1] = v0[1] + t * edge[1];
	pnt[2] = v0[2] + t * edge[2];
}

static inline void intersectZ(const rcReal* v0, const rcReal* edge, rcReal cz, rcReal *pnt)
{
	rcReal t = rcClamp((cz - v0[2]) * edge[9 + 2], 0.0f, 1.0f); //inverses

	pnt[0] = v0[0] + t * edge[0];
	pnt[1] = v0[1] + t * edge[1];
	pnt[2] = v0[2] + t * edge[2];
}

#if TEST_NEW_RASTERIZER

static void rasterizeTriTest(const rcReal* v0, const rcReal* v1, const rcReal* v2,
						int xtest, int ytest,
						int& outsmin, int& outsmax, 
						 const unsigned char area, rcHeightfield& hf,
						 const rcReal* bmin, const rcReal* bmax,
						 const rcReal cs, const rcReal ics, const rcReal ich,
						 const int flagMergeThr)
{
	outsmin = RC_SPAN_MAX_HEIGHT;
	outsmax = -1;
	const int w = hf.width;
	const int h = hf.height;
	rcReal tmin[3], tmax[3];
	const rcReal by = bmax[1] - bmin[1];

	// Calculate the bounding box of the triangle.
	rcVcopy(tmin, v0);
	rcVcopy(tmax, v0);
	rcVmin(tmin, v1);
	rcVmin(tmin, v2);
	rcVmax(tmax, v1);
	rcVmax(tmax, v2);

	// If the triangle does not touch the bbox of the heightfield, skip the triagle.
	if (!overlapBounds(bmin, bmax, tmin, tmax))
		return;

	// Calculate the footpring of the triangle on the grid.
	int x0 = (int)((tmin[0] - bmin[0])*ics);
	int y0 = (int)((tmin[2] - bmin[2])*ics);
	int x1 = (int)((tmax[0] - bmin[0])*ics);
	int y1 = (int)((tmax[2] - bmin[2])*ics);
	x0 = rcClamp(x0, 0, w-1);
	y0 = rcClamp(y0, 0, h-1);
	x1 = rcClamp(x1, 0, w-1);
	y1 = rcClamp(y1, 0, h-1);

	// Clip the triangle into all grid cells it touches.
	rcReal in[7*3], out[7*3], inrow[7*3];

	for (int y = y0; y <= y1; ++y)
	{
		if (y != ytest) continue;
		// Clip polygon to row.
		rcVcopy(&in[0], v0);
		rcVcopy(&in[1*3], v1);
		rcVcopy(&in[2*3], v2);
		int nvrow = 3;
		const rcReal cz = bmin[2] + y*cs;
		nvrow = clipPoly(in, nvrow, out, 0, 1, -cz);
		if (nvrow < 3) continue;
		nvrow = clipPoly(out, nvrow, inrow, 0, -1, cz+cs);
		if (nvrow < 3) continue;

		for (int x = x0; x <= x1; ++x)
		{
			if (x != xtest) continue;
			// Clip polygon to column.
			int nv = nvrow;
			const rcReal cx = bmin[0] + x*cs;
			nv = clipPoly(inrow, nv, out, 1, 0, -cx);
			if (nv < 3) continue;
			nv = clipPoly(out, nv, in, -1, 0, cx+cs);
			if (nv < 3) continue;

			// Calculate min and max of the span.
			rcReal smin = in[1], smax = in[1];
			for (int i = 1; i < nv; ++i)
			{
				smin = rcMin(smin, in[i*3+1]);
				smax = rcMax(smax, in[i*3+1]);
			}
			smin -= bmin[1];
			smax -= bmin[1];
			// Skip the span if it is outside the heightfield bbox
			if (smax < 0.0f) continue;
			if (smin > by) continue;
			// Clamp the span to the heightfield bbox.
			if (smin < 0.0f) smin = 0;
			if (smax > by) smax = by;

			// Snap the span to the heightfield height grid.
			rcSpanUInt ismin = (rcSpanUInt)rcClamp((int)rcFloor(smin * ich), 0, RC_SPAN_MAX_HEIGHT);
			rcSpanUInt ismax = (rcSpanUInt)rcClamp((int)rcCeil(smax * ich), (int)ismin+1, RC_SPAN_MAX_HEIGHT);
			outsmin = ismin;
			outsmax = ismax;
		}
	}
}
#endif //TEST_NEW_RASTERIZER

template <typename AddSpanFunc>
static void rasterizeTri(const rcReal* v0, const rcReal* v1, const rcReal* v2,
						 const unsigned char area, rcHeightfield& hf,
						 const rcReal* bmin, const rcReal* bmax,
						 const rcReal cs, const rcReal ics, const rcReal ich, 
						 const int flagMergeThr,
						 const int rasterizationFlags, /*UE*/
	                     const int* rasterizationMasks, /*UE*/
	                     AddSpanFunc& Func /*UE*/)
{
	rcEdgeHit* const hfEdgeHits = hf.EdgeHits; //this prevents a static analysis warning

	const int w = hf.width;
	const int h = hf.height;

	int intverts[3][2];

	intverts[0][0] = (int)rcFloor((v0[0] - bmin[0])*ics);
	intverts[0][1] = (int)rcFloor((v0[2] - bmin[2])*ics);
	intverts[1][0] = (int)rcFloor((v1[0] - bmin[0])*ics);
	intverts[1][1] = (int)rcFloor((v1[2] - bmin[2])*ics);
	intverts[2][0] = (int)rcFloor((v2[0] - bmin[0])*ics);
	intverts[2][1] = (int)rcFloor((v2[2] - bmin[2])*ics);

	int x0 = intMin(intverts[0][0], intMin(intverts[1][0], intverts[2][0]));
	int x1 = intMax(intverts[0][0], intMax(intverts[1][0], intverts[2][0]));
	int y0 = intMin(intverts[0][1], intMin(intverts[1][1], intverts[2][1]));
	int y1 = intMax(intverts[0][1], intMax(intverts[1][1], intverts[2][1]));

	if (x1 < 0 || x0 >= w || y1 < 0 || y0 >= h)
		return;

	const rcReal by = bmax[1] - bmin[1];
	const int projectTriToBottom = rasterizationFlags & RC_PROJECT_TO_BOTTOM; //UE
	
	// Calculate min and max of the triangle

	rcReal triangle_smin = rcMin(rcMin(v0[1], v1[1]), v2[1]);
	rcReal triangle_smax = rcMax(rcMax(v0[1], v1[1]), v2[1]);
	triangle_smin -= bmin[1];
	triangle_smax -= bmin[1];
	// Skip the span if it is outside the heightfield bbox
	if (triangle_smax < 0.0f) return;
	if (triangle_smin > by) return;

	if (x0 == x1 && y0 == y1)
	{
		// Clamp the span to the heightfield bbox.
		if (triangle_smin < 0.0f) triangle_smin = 0.0f;
		if (triangle_smax > by) triangle_smax = by;

		// Snap the span to the heightfield height grid.
		rcSpanUInt triangle_ismin = (rcSpanUInt)rcClamp((int)rcFloor(triangle_smin * ich), 0, RC_SPAN_MAX_HEIGHT);
		rcSpanUInt triangle_ismax = (rcSpanUInt)rcClamp((int)rcCeil(triangle_smax * ich), (int)triangle_ismin+1, RC_SPAN_MAX_HEIGHT);
		const int projectSpanToBottom = rasterizationMasks != nullptr ? (projectTriToBottom & rasterizationMasks[x0+y0*w]) : projectTriToBottom;	//UE
		if (projectSpanToBottom) //UE
		{
			triangle_ismin = 0; //UE
		}

		Func.addSpan(hf, x0, y0, triangle_ismin, triangle_ismax, area, flagMergeThr);
		return;
	}

	constexpr int RANGE = INT_MAX;
	
	const int triangle_ismin = rcClamp((int)rcFloor(triangle_smin * ich), -RANGE, RANGE);
	const int triangle_ismax = rcClamp((int)rcFloor(triangle_smax * ich), -RANGE, RANGE);

	x0 = intMax(x0, 0);
	int x1_edge = intMin(x1, w);
	x1 = intMin(x1, w - 1);
	y0 = intMax(y0, 0);
	int y1_edge = intMin(y1, h);
	y1 = intMin(y1, h - 1);

#if TEST_COVERAGE
	for (int y = y0; y <= y1; y++)
	{
		for (int x = x0; x <= x1; x++)
		{
			int outsmin, outsmax;
			rasterizeTriTest(v0, v1, v2, x, y, 
				outsmin, outsmax, 
				area, hf,
				bmin, bmax,
				cs, ics, ich,
				flagMergeThr);
			if (outsmin != RC_SPAN_MAX_HEIGHT)
			{
				markSpanSample(hf, x, y);
			}
		}
	}
#endif

	rcReal edges[6][3];

	rcReal vertarray[3][3];
	rcVcopy(vertarray[0], v0);
	rcVcopy(vertarray[1], v1);
	rcVcopy(vertarray[2], v2);

	bool doFlat = true;
	if (doFlat && triangle_ismin == triangle_ismax)
	{
		// flat horizontal, much faster
		for (int basevert = 0; basevert < 3; basevert++)
		{
			int othervert = basevert == 2 ? 0 : basevert + 1;
			int edge = basevert == 0 ? 2 : basevert - 1;

			rcVsub(&edges[edge][0], vertarray[othervert], vertarray[basevert]);
			//rcVnormalize(&edges[edge][0]);
			edges[3 + edge][0] = 1.0f / edges[edge][0];
			edges[3 + edge][1] = 1.0f / edges[edge][1];
			edges[3 + edge][2] = 1.0f / edges[edge][2];

			// drop the vert into the temp span area
			if (intverts[basevert][0] >= x0 && intverts[basevert][0] <= x1 && intverts[basevert][1] >= y0 && intverts[basevert][1] <= y1)
			{
				addFlatSpanSample(hf, intverts[basevert][0], intverts[basevert][1]);
			}
			// set up the edge intersections with horizontal planes
			if (intverts[basevert][1] != intverts[othervert][1])
			{
				int edge0 = intMin(intverts[basevert][1], intverts[othervert][1]);
				int edge1 = intMax(intverts[basevert][1], intverts[othervert][1]);
				int loop0 = intMax(edge0 + 1, y0);
				int loop1 = intMin(edge1, y1_edge);

				unsigned char edgeBits = (unsigned char)((edge << 4) | (othervert << 2) | basevert);
				for (int y = loop0; y <= loop1; y++)
				{
					int HitIndex = !!hfEdgeHits[y].Hits[0];
					hfEdgeHits[y].Hits[HitIndex] = edgeBits;
				}
			}
			// do the edge intersections with vertical planes
			if (intverts[basevert][0] != intverts[othervert][0])
			{
				int edge0 = intMin(intverts[basevert][0], intverts[othervert][0]);
				int edge1 = intMax(intverts[basevert][0], intverts[othervert][0]);
				int loop0 = intMax(edge0 + 1, x0);
				int loop1 = intMin(edge1, x1_edge);

				rcReal temppnt[3];
				rcReal cx = bmin[0] + cs * loop0;
				for (int x = loop0; x <= loop1; x++, cx += cs)
				{
					intersectX(vertarray[basevert], &edges[edge][0], cx, temppnt);
					int y = (int)rcFloor((temppnt[2] - bmin[2])*ics);
					if (y >= y0 && y <= y1)
					{
						addFlatSpanSample(hf, x, y);
						addFlatSpanSample(hf, x - 1, y);
					}
				}
			}
		}
		{
			// deal with the horizontal intersections 
			int edge0 = intMin(intverts[0][1], intMin(intverts[1][1],intverts[2][1]));
			int edge1 = intMax(intverts[0][1], intMax(intverts[1][1],intverts[2][1]));
			int loop0 = intMax(edge0 + 1, y0);
			int loop1 = intMin(edge1, y1_edge);

			rcReal Inter[2][3];
			int xInter[2];

			rcReal cz = bmin[2] + cs * loop0;
			for (int y = loop0; y <= loop1; y++, cz += cs)
			{
				rcEdgeHit& Hits = hfEdgeHits[y];
				if (Hits.Hits[0])
				{
					rcAssert(Hits.Hits[1]); // must have two hits

					for (int i = 0; i < 2; i++)
					{
						int edge = Hits.Hits[i] >> 4;
						int othervert = (Hits.Hits[i] >> 2) & 3;
						int basevert = Hits.Hits[i] & 3;

						intersectZ(vertarray[basevert], &edges[edge][0], cz, Inter[i]);
						int x = (int)rcFloor((Inter[i][0] - bmin[0])*ics);
						xInter[i] = x;
						if (x >= x0 && x <= x1)
						{
							addFlatSpanSample(hf, x, y);
							addFlatSpanSample(hf, x, y - 1);
						}
					}
					if (xInter[0] != xInter[1])
					{
						// now fill in the fully contained ones.
						int left = Inter[1][0] < Inter[0][0];  
						int xloop0 = intMax(xInter[left] + 1, x0);
						int xloop1 = intMin(xInter[1 - left], x1);
						if (xloop0 <= xloop1)
						{
							addFlatSpanSample(hf, xloop0, y);
							addFlatSpanSample(hf, xloop1, y);
							addFlatSpanSample(hf, xloop0 - 1, y);
							addFlatSpanSample(hf, xloop1 - 1, y);
							addFlatSpanSample(hf, xloop0, y - 1);
							addFlatSpanSample(hf, xloop1, y - 1);
							addFlatSpanSample(hf, xloop0 - 1, y - 1);
							addFlatSpanSample(hf, xloop1 - 1, y - 1);
						}
					}
					// reset for next triangle
					Hits.Hits[0] = 0;
					Hits.Hits[1] = 0;
				}
			}
		}

		if (rasterizationMasks == nullptr) //UE
		{
			// Snap the span to the heightfield height grid.
			rcSpanUInt triangle_ismin_clamp = (rcSpanUInt)rcClamp((int)triangle_ismin, 0, RC_SPAN_MAX_HEIGHT);
			const rcSpanUInt triangle_ismax_clamp = (rcSpanUInt)rcClamp((int)triangle_ismax, (int)triangle_ismin_clamp+1, RC_SPAN_MAX_HEIGHT);
			if (projectTriToBottom) //UE
			{
				triangle_ismin_clamp = 0; //UE
			}

			for (int y = y0; y <= y1; y++)
			{
				int xloop0 = intMax(hf.RowExt[y + 1].MinCol, x0);
				int xloop1 = intMin(hf.RowExt[y + 1].MaxCol, x1);
				for (int x = xloop0; x <= xloop1; x++)
				{
					Func.addSpan(hf, x, y, triangle_ismin_clamp, triangle_ismax_clamp, area, flagMergeThr);
				}

				// reset for next triangle
				hf.RowExt[y + 1].MinCol = hf.width + 2;
				hf.RowExt[y + 1].MaxCol = -2;
			}
		}
		else
		{
// @UE BEGIN
			for (int y = y0; y <= y1; y++)
			{
				int xloop0 = intMax(hf.RowExt[y + 1].MinCol, x0);
				int xloop1 = intMin(hf.RowExt[y + 1].MaxCol, x1);
				for (int x = xloop0; x <= xloop1; x++)
				{
					// Snap the span to the heightfield height grid.
					rcSpanUInt triangle_ismin_clamp = (rcSpanUInt)rcClamp((int)triangle_ismin, 0, RC_SPAN_MAX_HEIGHT);
					const rcSpanUInt triangle_ismax_clamp = (rcSpanUInt)rcClamp((int)triangle_ismax, (int)triangle_ismin_clamp+1, RC_SPAN_MAX_HEIGHT);
					const int projectSpanToBottom = projectTriToBottom & rasterizationMasks[x+y*w];		//UE
					if (projectSpanToBottom) //UE
					{
						triangle_ismin_clamp = 0; //UE
					}
					Func.addSpan(hf, x, y, triangle_ismin_clamp, triangle_ismax_clamp, area, flagMergeThr);
				}

				// reset for next triangle
				hf.RowExt[y + 1].MinCol = hf.width + 2;
				hf.RowExt[y + 1].MaxCol = -2;
			}
// @UE END
		}
	}
	else
	{
		//non-flat case
		for (int basevert = 0; basevert < 3; basevert++)
		{
			int othervert = basevert == 2 ? 0 : basevert + 1;
			int edge = basevert == 0 ? 2 : basevert - 1;

			rcVsub(&edges[edge][0], vertarray[othervert], vertarray[basevert]);
			//rcVnormalize(&edges[edge][0]);
			edges[3 + edge][0] = 1.0f / edges[edge][0];
			edges[3 + edge][1] = 1.0f / edges[edge][1];
			edges[3 + edge][2] = 1.0f / edges[edge][2];

			// drop the vert into the temp span area
			if (intverts[basevert][0] >= x0 && intverts[basevert][0] <= x1 && intverts[basevert][1] >= y0 && intverts[basevert][1] <= y1)
			{
				rcReal sfloat = vertarray[basevert][1] - bmin[1];
				const int sint = (int)rcClamp((int)rcFloor(sfloat * ich), -RANGE, RANGE);
	#if TEST_NEW_RASTERIZER
				rcAssert(sint >= triangle_ismin - 1 && sint <= triangle_ismax + 1);
	#endif
				addSpanSample(hf, intverts[basevert][0], intverts[basevert][1], sint);
			}
			// set up the edge intersections with horizontal planes
			if (intverts[basevert][1] != intverts[othervert][1])
			{
				int edge0 = intMin(intverts[basevert][1], intverts[othervert][1]);
				int edge1 = intMax(intverts[basevert][1], intverts[othervert][1]);
				int loop0 = intMax(edge0 + 1, y0);
				int loop1 = intMin(edge1, y1_edge);

				unsigned char edgeBits = (unsigned char)((edge << 4) | (othervert << 2) | basevert);
				for (int y = loop0; y <= loop1; y++)
				{
					int HitIndex = !!hfEdgeHits[y].Hits[0];
					hfEdgeHits[y].Hits[HitIndex] = edgeBits;
				}
			}
			// do the edge intersections with vertical planes
			if (intverts[basevert][0] != intverts[othervert][0])
			{
				int edge0 = intMin(intverts[basevert][0], intverts[othervert][0]);
				int edge1 = intMax(intverts[basevert][0], intverts[othervert][0]);
				int loop0 = intMax(edge0 + 1, x0);
				int loop1 = intMin(edge1, x1_edge);

				rcReal temppnt[3];
				rcReal cx = bmin[0] + cs * loop0;
				for (int x = loop0; x <= loop1; x++, cx += cs)
				{
					intersectX(vertarray[basevert], &edges[edge][0], cx, temppnt);
					int y = (int)rcFloor((temppnt[2] - bmin[2])*ics);
					if (y >= y0 && y <= y1)
					{
						rcReal sfloat = temppnt[1] - bmin[1];
						const int sint = (int)rcClamp((int)rcFloor(sfloat * ich), -RANGE, RANGE);
#if TEST_NEW_RASTERIZER
						rcAssert(sint >= triangle_ismin - 1 && sint <= triangle_ismax + 1);
#endif
						addSpanSample(hf, x, y, sint);
						addSpanSample(hf, x - 1, y, sint);
					}
				}
			}
		}
		{
			// deal with the horizontal intersections 
			int edge0 = intMin(intverts[0][1], intMin(intverts[1][1],intverts[2][1]));
			int edge1 = intMax(intverts[0][1], intMax(intverts[1][1],intverts[2][1]));
			int loop0 = intMax(edge0 + 1, y0);
			int loop1 = intMin(edge1, y1_edge);

			rcReal Inter[2][3];
			int xInter[2];

			rcReal cz = bmin[2] + cs * loop0;
			for (int y = loop0; y <= loop1; y++, cz += cs)
			{
				rcEdgeHit& Hits = hfEdgeHits[y];
				if (Hits.Hits[0])
				{
					rcAssert(Hits.Hits[1]); // must have two hits

					for (int i = 0; i < 2; i++)
					{
						int edge = Hits.Hits[i] >> 4;
						int othervert = (Hits.Hits[i] >> 2) & 3;
						int basevert = Hits.Hits[i] & 3;

						CA_SUPPRESS(6385);
						intersectZ(vertarray[basevert], &edges[edge][0], cz, Inter[i]);
						int x = (int)rcFloor((Inter[i][0] - bmin[0])*ics);
						xInter[i] = x;
						if (x >= x0 && x <= x1)
						{
							rcReal sfloat = Inter[i][1] - bmin[1];
							const int sint = (int)rcClamp((int)rcFloor(sfloat * ich), -RANGE, RANGE);
#if TEST_NEW_RASTERIZER
							rcAssert(sint >= triangle_ismin - 1 && sint <= triangle_ismax + 1);
#endif
							addSpanSample(hf, x, y, sint);
							addSpanSample(hf, x, y - 1, sint);
						}
					}
					if (xInter[0] != xInter[1])
					{
						// now fill in the fully contained ones.
						int left = Inter[1][0] < Inter[0][0];  
						int xloop0 = intMax(xInter[left] + 1, x0);
						int xloop1 = intMin(xInter[1 - left], x1_edge);

						rcReal d = 1.0f / (Inter[1-left][0] - Inter[left][0]);
						rcReal dy = Inter[1-left][1] - Inter[left][1];
						//rcReal ds = dy * d;
						rcReal ds = 0.0f;
						rcReal t = rcClamp((rcReal(xloop0)*cs + bmin[0] - Inter[left][0]) * d, 0.0f, 1.0f);
						rcReal sfloat = (Inter[left][1] + t * dy) - bmin[1];
						if (xloop1 - xloop0 > 0)
						{
							rcReal t2 = rcClamp((rcReal(xloop1)*cs + bmin[0] - Inter[left][0]) * d, 0.0f, 1.0f);
							rcReal sfloat2 = (Inter[left][1] + t2 * dy) - bmin[1];
							ds = (sfloat2 - sfloat) / rcReal(xloop1 - xloop0);
						}
						for (int x = xloop0; x <= xloop1; x++, sfloat += ds)
						{
							const int sint = (int)rcClamp((int)rcFloor(sfloat * ich), -RANGE, RANGE);
#if TEST_NEW_RASTERIZER
							rcAssert(sint >= triangle_ismin - 1 && sint <= triangle_ismax + 1);
#endif
							addSpanSample(hf, x, y, sint);
							addSpanSample(hf, x - 1, y, sint);
							addSpanSample(hf, x, y - 1, sint);
							addSpanSample(hf, x - 1, y - 1, sint);
						}
					}
					// reset for next triangle
					Hits.Hits[0] = 0;
					Hits.Hits[1] = 0;
				}
			}
		}
		
		for (int y = y0; y <= y1; y++)
		{
			int xloop0 = intMax(hf.RowExt[y + 1].MinCol, x0);
			int xloop1 = intMin(hf.RowExt[y + 1].MaxCol, x1);
			for (int x = xloop0; x <= xloop1; x++)
			{
				int idx = SampleIndex(hf, x, y);
				rcTempSpan& Temp = hf.tempspans[idx];

				int smin = Temp.sminmax[0];

				// +1 because span sample heights are computed from rcFloor instead of rcCeil (and they need to have a height of at least 1)
				int smax = UE::Recast::Private::bEnableSpanHeightRasterizationFix ? Temp.sminmax[1] + 1 : Temp.sminmax[1]; //UE

				// reset for next triangle
				Temp.sminmax[0] = RANGE;
				Temp.sminmax[1] = -RANGE;

				// Skip the span if it is outside the heightfield bbox
				if (smin >= RC_SPAN_MAX_HEIGHT || smax < 0) continue;

				smin = intMax(smin, 0);
				smax = intMin(intMax(smax,smin+1), RC_SPAN_MAX_HEIGHT);
				const int projectSpanToBottom = rasterizationMasks != nullptr ? (projectTriToBottom & rasterizationMasks[x+y*w]) : projectTriToBottom; //UE
				if (projectSpanToBottom) //UE
				{
					smin = 0; //UE
				}

	#if TEST_NEW_RASTERIZER
				{
					int outsmin, outsmax;
					rasterizeTriTest(v0, v1, v2, x, y, 
						outsmin, outsmax, 
						area, hf,
						bmin, bmax,
						cs, ics, ich,
						flagMergeThr);
					const int tol = 1;
					if (outsmin > smin + tol || outsmin < smin - tol ||
						outsmax > smax + tol || outsmax < smax - tol
						)
					{
						Temp.sminmax[0] = RANGE;
						Temp.sminmax[1] = -RANGE;
						rasterizeTriTest(v0, v1, v2, x, y, 
							outsmin, outsmax, 
							area, hf,
							bmin, bmax,
							cs, ics, ich,
							flagMergeThr);
						if (outsmin != RC_SPAN_MAX_HEIGHT)
						{
							Temp.sminmax[0] = RANGE;
							Temp.sminmax[1] = -RANGE;
						}
					}

				}
	#endif
				Func.addSpan(hf, x, y, smin, smax, area, flagMergeThr);
			}

			// reset for next triangle
			hf.RowExt[y + 1].MinCol = hf.width + 2;
			hf.RowExt[y + 1].MaxCol = -2;
		}
	}
#if TEST_NEW_RASTERIZER
	for (int y = 0; y < h; y++)
	{
		rcAssert(hf.RowExt[y + 1].MinCol == hf.width + 2 && hf.RowExt[y + 1].MaxCol == -2);
		rcEdgeHit& Hits = hfEdgeHits[y];
		rcAssert(!Hits.Hits[0] && !Hits.Hits[1]);
		for (int x = 0; x < w; x++)
		{
			int idx = SampleIndex(hf, x, y);
			rcTempSpan& Temp = hf.tempspans[idx];
			rcAssert(Temp.sminmax[0] == RANGE && Temp.sminmax[1] == -RANGE);
		}
	}
	rcEdgeHit& Hits = hfEdgeHits[h];
	rcAssert(!Hits.Hits[0] && !Hits.Hits[1]);
#endif
}


#else
static void rasterizeTri(const rcReal* v0, const rcReal* v1, const rcReal* v2,
						 const unsigned char area, rcHeightfield& hf,
						 const rcReal* bmin, const rcReal* bmax,
						 const rcReal cs, const rcReal ics, const rcReal ich,
						 const int flagMergeThr,
						 const int rasterizationFlags,  //UE
						 const int* rasterizationMasks) //UE
{
	const int w = hf.width;
	const int h = hf.height;
	rcReal tmin[3], tmax[3];
	const rcReal by = bmax[1] - bmin[1];
	const int projectTriToBottom = rasterizationFlags & RC_PROJECT_TO_BOTTOM; //UE
	
	// Calculate the bounding box of the triangle.
	rcVcopy(tmin, v0);
	rcVcopy(tmax, v0);
	rcVmin(tmin, v1);
	rcVmin(tmin, v2);
	rcVmax(tmax, v1);
	rcVmax(tmax, v2);
	
	// If the triangle does not touch the bbox of the heightfield, skip the triangle.
	if (!overlapBounds(bmin, bmax, tmin, tmax))
		return;
	
	// Calculate the footprint of the triangle on the grid.
	int x0 = (int)((tmin[0] - bmin[0])*ics);
	int y0 = (int)((tmin[2] - bmin[2])*ics);
	int x1 = (int)((tmax[0] - bmin[0])*ics);
	int y1 = (int)((tmax[2] - bmin[2])*ics);
	x0 = rcClamp(x0, 0, w-1);
	y0 = rcClamp(y0, 0, h-1);
	x1 = rcClamp(x1, 0, w-1);
	y1 = rcClamp(y1, 0, h-1);
	
	// Clip the triangle into all grid cells it touches.
	rcReal in[7*3], out[7*3], inrow[7*3];
	
	for (int y = y0; y <= y1; ++y)
	{
		// Clip polygon to row.
		rcVcopy(&in[0], v0);
		rcVcopy(&in[1*3], v1);
		rcVcopy(&in[2*3], v2);
		int nvrow = 3;
		const rcReal cz = bmin[2] + y*cs;
		nvrow = clipPoly(in, nvrow, out, 0, 1, -cz);
		if (nvrow < 3) continue;
		nvrow = clipPoly(out, nvrow, inrow, 0, -1, cz+cs);
		if (nvrow < 3) continue;
		
		for (int x = x0; x <= x1; ++x)
		{
			// Clip polygon to column.
			int nv = nvrow;
			const rcReal cx = bmin[0] + x*cs;
			nv = clipPoly(inrow, nv, out, 1, 0, -cx);
			if (nv < 3) continue;
			nv = clipPoly(out, nv, in, -1, 0, cx+cs);
			if (nv < 3) continue;
			
			// Calculate min and max of the span.
			rcReal smin = in[1], smax = in[1];
			for (int i = 1; i < nv; ++i)
			{
				smin = rcMin(smin, in[i*3+1]);
				smax = rcMax(smax, in[i*3+1]);
			}
			smin -= bmin[1];
			smax -= bmin[1];
			// Skip the span if it is outside the heightfield bbox
			if (smax < 0.0f) continue;
			if (smin > by) continue;
			// Clamp the span to the heightfield bbox.
			if (smin < 0.0f) smin = 0;
			if (smax > by) smax = by;
			
			// Snap the span to the heightfield height grid.
			unsigned short ismin = (unsigned short)rcClamp((int)rcFloor(smin * ich), 0, RC_SPAN_MAX_HEIGHT);
			unsigned short ismax = (unsigned short)rcClamp((int)rcCeil(smax * ich), (int)ismin+1, RC_SPAN_MAX_HEIGHT);
			const int projectSpanToBottom = rasterizationMasks != nullptr ? (projectTriToBottom & rasterizationMasks[x+y*w]) : projectTriToBottom;	//UE
			if (projectSpanToBottom) //UE
			{
				ismin = 0; //UE
			}

			Func.addSpan(hf, x, y, ismin, ismax, area, flagMergeThr);
		}
	}
}
#endif //EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER

struct AddSpanInHeightfield
{
	static void addSpan(rcHeightfield& hf, const int x, const int y,
					const rcSpanUInt smin, const rcSpanUInt smax,
					const unsigned char area, const int flagMergeThr)
	{
		::addSpan(hf, x, y, smin, smax, area, flagMergeThr);
	}
};

struct AddSpanInTempColumns
{
	AddSpanInTempColumns(rcContext* inctx, rcHeightfield& hf, const float ics, const rcReal* vertsBMin, const rcReal* vertsBMax)
	{
#if EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
		ctx = inctx;
		if (vertsBMin == nullptr || vertsBMax == nullptr)
		{
			if (ctx)
			{
				ctx->log(RC_LOG_WARNING, "rcRasterizeTriangles called for triangles to rasterize as filled convex but the verts bounds is not valid.");
			}
			return;
		}

		if (!hf.tempSpanColumns)
		{
			if (ctx)
			{
				ctx->log(RC_LOG_WARNING, "rcRasterizeTriangles called for triangles to rasterize as filled convex but tempSpanColumns was not allocated.");
			}
			return;
		}

		// clamp the verts bounding box to the heightfield
		const rcReal xMin = rcClamp(vertsBMin[0], hf.bmin[0], hf.bmax[0]);
		const rcReal xMax = rcClamp(vertsBMax[0], hf.bmin[0], hf.bmax[0]);
		const rcReal zMin = rcClamp(vertsBMin[2], hf.bmin[2], hf.bmax[2]);
		const rcReal zMax = rcClamp(vertsBMax[2], hf.bmin[2], hf.bmax[2]);
		if (xMax == xMin || zMax == zMin)
		{
			// this can be valid if we receive a geometry that is outside of the height field. No need to log
			return;
		}

		// convert it into heightfield span coords
		const int ixMin = ics * (xMin - hf.bmin[0]);
		const int izMin = ics * (zMin - hf.bmin[2]);
		const int ixMax = ics * (xMax - hf.bmin[0]);
		const int izMax = ics * (zMax - hf.bmin[2]);

		ibminx = ixMin;
		ibminz = izMin;
		// the max is inclusive so we need to add 1 to width and height
		ibwidth = rcClamp(ixMax - ixMin + 1, 0, hf.width);
		ibheight = rcClamp(izMax - izMin + 1, 0, hf.height);
		memset(hf.tempSpanColumns, 0, sizeof(rcSpanData)*ibwidth*ibheight);	// we could keep an array of call idx to avoid doing that too often
#endif // EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
	}

	void addSpan(rcHeightfield& hf, const int x, const int z,
					const rcSpanUInt smin, const rcSpanUInt smax,
					const unsigned char area, const int flagMergeThr)
	{
#if EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
		// convert the heightfield coord to the tempSpanColumns coord
		const int tempColumnx = x - ibminx;
		const int tempColumnz = z - ibminz;
		if (tempColumnx < 0 || tempColumnz < 0 || tempColumnx >= ibwidth || tempColumnz >= ibheight)
		{
			// the span is outside of the verts bounding box, we have to skip it because it would mean we access invalid memory
			if (ctx)
			{
				ctx->log(RC_LOG_WARNING, "rcRasterizeTriangles trying to rasterize a triangle as filled convex but some spans are outside of the provided bounding box. Those spans will be skipped.");
			}
			return;
		}

		const int tempColumnIdx = tempColumnx + (tempColumnz) * ibwidth;
		rcSpanData& tempColumn = hf.tempSpanColumns[tempColumnIdx];
		if (tempColumn.smin == 0 && tempColumn.smax == 0)
		{
			// first span added, store it as is
			tempColumn.smin = smin;
			tempColumn.smax = smax;
			tempColumn.area = area;
		}
		else
		{
			// need to merge with a previously added span
			mergeSpanData(tempColumn, smin, smax, area, flagMergeThr);
		}
#endif // EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
	}

	void TransferColumnsToHeightfield(rcHeightfield& hf, const float flagMergeThr)
	{
#if EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
		// push all the created spans into the heightfield
		for (int iz = 0; iz < ibheight; iz++)
		{
			for (int ix = 0; ix < ibwidth; ix++)
			{
				const int tempSpanColumnsIdx = ix + (iz * ibwidth);
				const rcSpanData& tempSpanColumn = hf.tempSpanColumns[tempSpanColumnsIdx];
				if (tempSpanColumn.smin == 0 && tempSpanColumn.smax == 0)
				{
					continue;
				}

				::addSpan(hf, ibminx + ix, ibminz + iz, tempSpanColumn.smin, tempSpanColumn.smax, tempSpanColumn.area, flagMergeThr);
			}
		}
#endif // EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
	}

	bool IsValid() const { return ibwidth > 0 && ibheight > 0; }

	rcContext* ctx = nullptr;
	int ibminx = 0;
	int ibminz = 0;
	int ibwidth = 0;
	int ibheight = 0;
};

/// @par
///
/// No spans will be added if the triangle does not overlap the heightfield grid.
///
/// @see rcHeightfield
void rcRasterizeTriangle(rcContext* ctx, const rcReal* v0, const rcReal* v1, const rcReal* v2,
						 const unsigned char area, rcHeightfield& solid,
						 const int flagMergeThr, const int rasterizationFlags, const int* rasterizationMasks) //UE
{
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_RASTERIZE_TRIANGLES);

	AddSpanInHeightfield Func;

	const rcReal ics = 1.0f/solid.cs;
	const rcReal ich = 1.0f/solid.ch;
	rasterizeTri(v0, v1, v2, area, solid, solid.bmin, solid.bmax, solid.cs, ics, ich, flagMergeThr, rasterizationFlags, rasterizationMasks, Func); //UE

	ctx->stopTimer(RC_TIMER_RASTERIZE_TRIANGLES);
}

/// @par
///
/// Spans will only be added for triangles that overlap the heightfield grid.
///
/// @see rcHeightfield
void rcRasterizeTriangles(rcContext* ctx, const rcReal* verts, const int /*nv*/,
						  const int* tris, const unsigned char* areas, const int nt,
						  rcHeightfield& solid, const int flagMergeThr, const int rasterizationFlags, const int* rasterizationMasks, //UE
						  const rcReal* vertsbmin, const rcReal* vertsbmax) //UE
{
	if (ctx)
		ctx->startTimer(RC_TIMER_RASTERIZE_TRIANGLES);
	
	const rcReal ics = 1.0f/solid.cs;
	const rcReal ich = 1.0f/solid.ch;

	const bool bAsFilledVerticalConvexVolume = rasterizationFlags & RC_RASTERIZE_AS_FILLED_CONVEX;
	if (!bAsFilledVerticalConvexVolume)
	{
		AddSpanInHeightfield Func;
		// Rasterize triangles.
		for (int i = 0; i < nt; ++i)
		{
			const rcReal* v0 = &verts[tris[i*3+0]*3];
			const rcReal* v1 = &verts[tris[i*3+1]*3];
			const rcReal* v2 = &verts[tris[i*3+2]*3];
			// Rasterize.
			rasterizeTri(v0, v1, v2, areas[i], solid, solid.bmin, solid.bmax, solid.cs, ics, ich, flagMergeThr, rasterizationFlags, rasterizationMasks, Func); //UE
		}
	}
	else
	{
		AddSpanInTempColumns Func(ctx, solid, ics, vertsbmin, vertsbmax);
		if (Func.IsValid())
		{
			for (int i = 0; i < nt; ++i)
			{
				const rcReal* v0 = &verts[tris[i*3+0]*3];
				const rcReal* v1 = &verts[tris[i*3+1]*3];
				const rcReal* v2 = &verts[tris[i*3+2]*3];
				rasterizeTri(v0, v1, v2, areas[i], solid, solid.bmin, solid.bmax, solid.cs, ics, ich, flagMergeThr, rasterizationFlags, rasterizationMasks, Func); //UE
			}

			Func.TransferColumnsToHeightfield(solid, flagMergeThr);
		}
	}

	if (ctx)
		ctx->stopTimer(RC_TIMER_RASTERIZE_TRIANGLES);
}

/// @par
///
/// Spans will only be added for triangles that overlap the heightfield grid.
///
/// @see rcHeightfield
void rcRasterizeTriangles(rcContext* ctx, const rcReal* verts, const int /*nv*/,
						  const unsigned short* tris, const unsigned char* areas, const int nt,
						  rcHeightfield& solid, const int flagMergeThr, const int rasterizationFlags, const int* rasterizationMasks, //UE
						  const rcReal* vertsbmin, const rcReal* vertsbmax) //UE
{
	if (ctx)
		ctx->startTimer(RC_TIMER_RASTERIZE_TRIANGLES);
	
	const rcReal ics = 1.0f/solid.cs;
	const rcReal ich = 1.0f/solid.ch;

	const bool bAsFilledVerticalConvexVolume = rasterizationFlags & RC_RASTERIZE_AS_FILLED_CONVEX;
	if (!bAsFilledVerticalConvexVolume)
	{
		AddSpanInHeightfield Func;
		// Rasterize triangles.
		for (int i = 0; i < nt; ++i)
		{
			const rcReal* v0 = &verts[tris[i*3+0]*3];
			const rcReal* v1 = &verts[tris[i*3+1]*3];
			const rcReal* v2 = &verts[tris[i*3+2]*3];
			// Rasterize.
			rasterizeTri(v0, v1, v2, areas[i], solid, solid.bmin, solid.bmax, solid.cs, ics, ich, flagMergeThr, rasterizationFlags, rasterizationMasks, Func); //UE
		}
	}
	else
	{
		AddSpanInTempColumns Func(ctx, solid, ics, vertsbmin, vertsbmax);
		if (Func.IsValid())
		{
			for (int i = 0; i < nt; ++i)
			{
				const rcReal* v0 = &verts[tris[i*3+0]*3];
				const rcReal* v1 = &verts[tris[i*3+1]*3];
				const rcReal* v2 = &verts[tris[i*3+2]*3];
				rasterizeTri(v0, v1, v2, areas[i], solid, solid.bmin, solid.bmax, solid.cs, ics, ich, flagMergeThr, rasterizationFlags, rasterizationMasks, Func); //UE
			}

			Func.TransferColumnsToHeightfield(solid, flagMergeThr);
		}
	}
	
	if (ctx)
		ctx->stopTimer(RC_TIMER_RASTERIZE_TRIANGLES);
}

/// @par
///
/// Spans will only be added for triangles that overlap the heightfield grid.
///
/// @see rcHeightfield
void rcRasterizeTriangles(rcContext* ctx, const rcReal* verts, const unsigned char* areas, const int nt,
						  rcHeightfield& solid, const int flagMergeThr, const int rasterizationFlags, const int* rasterizationMasks, //UE
						  const rcReal* vertsbmin, const rcReal* vertsbmax) //UE
{
	if (ctx)
		ctx->startTimer(RC_TIMER_RASTERIZE_TRIANGLES);
	
	const rcReal ics = 1.0f/solid.cs;
	const rcReal ich = 1.0f/solid.ch;

	const bool bAsFilledVerticalConvexVolume = rasterizationFlags & RC_RASTERIZE_AS_FILLED_CONVEX;
	if (!bAsFilledVerticalConvexVolume)
	{
		AddSpanInHeightfield Func;
		// Rasterize triangles.
		for (int i = 0; i < nt; ++i)
		{
			const rcReal* v0 = &verts[(i*3+0)*3];
			const rcReal* v1 = &verts[(i*3+1)*3];
			const rcReal* v2 = &verts[(i*3+2)*3];
			// Rasterize.
			rasterizeTri(v0, v1, v2, areas[i], solid, solid.bmin, solid.bmax, solid.cs, ics, ich, flagMergeThr, rasterizationFlags, rasterizationMasks, Func); //UE
		}
	}
	else
	{
		AddSpanInTempColumns Func(ctx, solid, ics, vertsbmin, vertsbmax);
		if (Func.IsValid())
		{
			for (int i = 0; i < nt; ++i)
			{
				const rcReal* v0 = &verts[(i*3+0)*3];
				const rcReal* v1 = &verts[(i*3+1)*3];
				const rcReal* v2 = &verts[(i*3+2)*3];
				rasterizeTri(v0, v1, v2, areas[i], solid, solid.bmin, solid.bmax, solid.cs, ics, ich, flagMergeThr, rasterizationFlags, rasterizationMasks, Func); //UE
			}

			Func.TransferColumnsToHeightfield(solid, flagMergeThr);
		}
	}
	
	if (ctx)
		ctx->stopTimer(RC_TIMER_RASTERIZE_TRIANGLES);
}
