// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#define _USE_MATH_DEFINES
#include "Recast/Recast.h"
#include "Recast/RecastAlloc.h"
#include "Recast/RecastAssert.h"


static void calculateDistanceField(rcCompactHeightfield& chf, unsigned short* src, unsigned short& maxDist)
{
	const int w = chf.width;
	const int h = chf.height;
	
	// Init distance and points.
	for (int i = 0; i < chf.spanCount; ++i)
		src[i] = 0xffff;
	
	// Mark boundary cells.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				const unsigned char area = chf.areas[i];
				
				int nc = 0;
				for (int dir = 0; dir < 4; ++dir)
				{
					if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
					{
						const int ax = x + rcGetDirOffsetX(dir);
						const int ay = y + rcGetDirOffsetY(dir);
						const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, dir);
						if (area == chf.areas[ai])
							nc++;
					}
				}
				if (nc != 4)
					src[i] = 0;
			}
		}
	}
	
			
	// Pass 1
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				
				if (rcGetCon(s, 0) != RC_NOT_CONNECTED)
				{
					// (-1,0)
					const int ax = x + rcGetDirOffsetX(0);
					const int ay = y + rcGetDirOffsetY(0);
					const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 0);
					const rcCompactSpan& as = chf.spans[ai];
					if (src[ai]+2 < src[i])
						src[i] = src[ai]+2;
					
					// (-1,-1)
					if (rcGetCon(as, 3) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(3);
						const int aay = ay + rcGetDirOffsetY(3);
						const int aai = (int)chf.cells[aax+aay*w].index + rcGetCon(as, 3);
						if (src[aai]+3 < src[i])
							src[i] = src[aai]+3;
					}
				}
				if (rcGetCon(s, 3) != RC_NOT_CONNECTED)
				{
					// (0,-1)
					const int ax = x + rcGetDirOffsetX(3);
					const int ay = y + rcGetDirOffsetY(3);
					const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 3);
					const rcCompactSpan& as = chf.spans[ai];
					if (src[ai]+2 < src[i])
						src[i] = src[ai]+2;
					
					// (1,-1)
					if (rcGetCon(as, 2) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(2);
						const int aay = ay + rcGetDirOffsetY(2);
						const int aai = (int)chf.cells[aax+aay*w].index + rcGetCon(as, 2);
						if (src[aai]+3 < src[i])
							src[i] = src[aai]+3;
					}
				}
			}
		}
	}
	
	// Pass 2
	for (int y = h-1; y >= 0; --y)
	{
		for (int x = w-1; x >= 0; --x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				
				if (rcGetCon(s, 2) != RC_NOT_CONNECTED)
				{
					// (1,0)
					const int ax = x + rcGetDirOffsetX(2);
					const int ay = y + rcGetDirOffsetY(2);
					const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 2);
					const rcCompactSpan& as = chf.spans[ai];
					if (src[ai]+2 < src[i])
						src[i] = src[ai]+2;
					
					// (1,1)
					if (rcGetCon(as, 1) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(1);
						const int aay = ay + rcGetDirOffsetY(1);
						const int aai = (int)chf.cells[aax+aay*w].index + rcGetCon(as, 1);
						if (src[aai]+3 < src[i])
							src[i] = src[aai]+3;
					}
				}
				if (rcGetCon(s, 1) != RC_NOT_CONNECTED)
				{
					// (0,1)
					const int ax = x + rcGetDirOffsetX(1);
					const int ay = y + rcGetDirOffsetY(1);
					const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 1);
					const rcCompactSpan& as = chf.spans[ai];
					if (src[ai]+2 < src[i])
						src[i] = src[ai]+2;
					
					// (-1,1)
					if (rcGetCon(as, 0) != RC_NOT_CONNECTED)
					{
						const int aax = ax + rcGetDirOffsetX(0);
						const int aay = ay + rcGetDirOffsetY(0);
						const int aai = (int)chf.cells[aax+aay*w].index + rcGetCon(as, 0);
						if (src[aai]+3 < src[i])
							src[i] = src[aai]+3;
					}
				}
			}
		}
	}	
	
	maxDist = 0;
	for (int i = 0; i < chf.spanCount; ++i)
		maxDist = rcMax(src[i], maxDist);
	
}

static unsigned short* boxBlur(rcCompactHeightfield& chf, int thr,
							   unsigned short* src, unsigned short* dst)
{
	const int w = chf.width;
	const int h = chf.height;
	
	thr *= 2;
	
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				const unsigned short cd = src[i];
				if (cd <= thr)
				{
					dst[i] = cd;
					continue;
				}

				int d = (int)cd;
				for (int dir = 0; dir < 4; ++dir)
				{
					if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
					{
						const int ax = x + rcGetDirOffsetX(dir);
						const int ay = y + rcGetDirOffsetY(dir);
						const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, dir);
						d += (int)src[ai];
						
						const rcCompactSpan& as = chf.spans[ai];
						const int dir2 = (dir+1) & 0x3;
						if (rcGetCon(as, dir2) != RC_NOT_CONNECTED)
						{
							const int ax2 = ax + rcGetDirOffsetX(dir2);
							const int ay2 = ay + rcGetDirOffsetY(dir2);
							const int ai2 = (int)chf.cells[ax2+ay2*w].index + rcGetCon(as, dir2);
							d += (int)src[ai2];
						}
						else
						{
							d += cd;
						}
					}
					else
					{
						d += cd*2;
					}
				}
				dst[i] = (unsigned short)((d+5)/9);
			}
		}
	}
	return dst;
}


static bool floodRegion(int x, int y, int i,
						unsigned short level, unsigned short r,
						rcCompactHeightfield& chf,
						unsigned short* srcReg, unsigned short* srcDist,
						rcIntArray& stack)
{
	const int w = chf.width;
	
	const unsigned char area = chf.areas[i];
	
	// Flood fill mark region.
	stack.resize(0);
	stack.push((int)x);
	stack.push((int)y);
	stack.push((int)i);
	srcReg[i] = r;
	srcDist[i] = 0;
	
	unsigned short lev = level >= 2 ? level-2 : 0;
	int count = 0;
	
	while (stack.size() > 0)
	{
		int ci = stack.pop();
		int cy = stack.pop();
		int cx = stack.pop();
		
		const rcCompactSpan& cs = chf.spans[ci];
		
		// Check if any of the neighbours already have a valid region set.
		unsigned short ar = 0;
		for (int dir = 0; dir < 4; ++dir)
		{
			// 8 connected
			if (rcGetCon(cs, dir) != RC_NOT_CONNECTED)
			{
				const int ax = cx + rcGetDirOffsetX(dir);
				const int ay = cy + rcGetDirOffsetY(dir);
				const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(cs, dir);
				if (chf.areas[ai] != area)
					continue;
				unsigned short nr = srcReg[ai];
				if (nr & RC_BORDER_REG) // Do not take borders into account.
					continue;
				if (nr != 0 && nr != r)
					ar = nr;
				
				const rcCompactSpan& as = chf.spans[ai];
				
				const int dir2 = (dir+1) & 0x3;
				if (rcGetCon(as, dir2) != RC_NOT_CONNECTED)
				{
					const int ax2 = ax + rcGetDirOffsetX(dir2);
					const int ay2 = ay + rcGetDirOffsetY(dir2);
					const int ai2 = (int)chf.cells[ax2+ay2*w].index + rcGetCon(as, dir2);
					if (chf.areas[ai2] != area)
						continue;
					unsigned short nr2 = srcReg[ai2];
					if (nr2 != 0 && nr2 != r)
						ar = nr2;
				}				
			}
		}
		if (ar != 0)
		{
			srcReg[ci] = 0;
			continue;
		}
		count++;
		
		// Expand neighbours.
		for (int dir = 0; dir < 4; ++dir)
		{
			if (rcGetCon(cs, dir) != RC_NOT_CONNECTED)
			{
				const int ax = cx + rcGetDirOffsetX(dir);
				const int ay = cy + rcGetDirOffsetY(dir);
				const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(cs, dir);
				if (chf.areas[ai] != area)
					continue;
				if (chf.dist[ai] >= lev && srcReg[ai] == 0)
				{
					srcReg[ai] = r;
					srcDist[ai] = 0;
					stack.push(ax);
					stack.push(ay);
					stack.push(ai);
				}
			}
		}
	}
	
	return count > 0;
}

static unsigned short* expandRegions(int maxIter, unsigned short level,
									 rcCompactHeightfield& chf,
									 unsigned short* srcReg, unsigned short* srcDist,
									 unsigned short* dstReg, unsigned short* dstDist, 
									 rcIntArray& stack)
{
	const int w = chf.width;
	const int h = chf.height;

	// Find cells revealed by the raised level.
	stack.resize(0);
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				if (chf.dist[i] >= level && srcReg[i] == 0 && chf.areas[i] != RC_NULL_AREA)
				{
					stack.push(x);
					stack.push(y);
					stack.push(i);
				}
			}
		}
	}
	
	int iter = 0;
	while (stack.size() > 0)
	{
		int failed = 0;
		
		memcpy(dstReg, srcReg, sizeof(unsigned short)*chf.spanCount);
		memcpy(dstDist, srcDist, sizeof(unsigned short)*chf.spanCount);
		
		for (int j = 0; j < stack.size(); j += 3)
		{
			int x = stack[j+0];
			int y = stack[j+1];
			int i = stack[j+2];
			if (i < 0)
			{
				failed++;
				continue;
			}
			
			unsigned short r = srcReg[i];
			unsigned short d2 = 0xffff;
			const unsigned char area = chf.areas[i];
			const rcCompactSpan& s = chf.spans[i];
			for (int dir = 0; dir < 4; ++dir)
			{
				if (rcGetCon(s, dir) == RC_NOT_CONNECTED) continue;
				const int ax = x + rcGetDirOffsetX(dir);
				const int ay = y + rcGetDirOffsetY(dir);
				const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, dir);
				if (chf.areas[ai] != area) continue;
				if (srcReg[ai] > 0 && (srcReg[ai] & RC_BORDER_REG) == 0)
				{
					if ((int)srcDist[ai]+2 < (int)d2)
					{
						r = srcReg[ai];
						d2 = srcDist[ai]+2;
					}
				}
			}
			if (r)
			{
				stack[j+2] = -1; // mark as used
				dstReg[i] = r;
				dstDist[i] = d2;
			}
			else
			{
				failed++;
			}
		}
		
		// rcSwap source and dest.
		rcSwap(srcReg, dstReg);
		rcSwap(srcDist, dstDist);
		
		if (failed*3 == stack.size())
			break;
		
		if (level > 0)
		{
			++iter;
			if (iter >= maxIter)
				break;
		}
	}
	
	return srcReg;
}


struct rcRegion
{
	inline rcRegion(unsigned short i) :
		chunkId(0),
		spanCount(0),
		id(i),
		areaType(0),
		remap(false),
		visited(false)
	{}
	
	int chunkId;					// ID of chunk containing this region
	int spanCount;					// Number of spans belonging to this region
	unsigned short id;				// ID of the region
	unsigned char areaType;			// Are type.
	unsigned char remap : 1;
	unsigned char visited : 1;
	rcIntArray connections;
	rcIntArray floors;
};

static void removeAdjacentNeighbours(rcRegion& reg)
{
	// Remove adjacent duplicates.
	for (int i = 0; i < reg.connections.size() && reg.connections.size() > 1; )
	{
		int ni = (i+1) % reg.connections.size();
		if (reg.connections[i] == reg.connections[ni])
		{
			// Remove duplicate
			for (int j = i; j < reg.connections.size()-1; ++j)
				reg.connections[j] = reg.connections[j+1];
			reg.connections.pop();
		}
		else
			++i;
	}
}

static void replaceNeighbour(rcRegion& reg, unsigned short oldId, unsigned short newId)
{
	bool neiChanged = false;
	for (int i = 0; i < reg.connections.size(); ++i)
	{
		if (reg.connections[i] == oldId)
		{
			reg.connections[i] = newId;
			neiChanged = true;
		}
	}
	for (int i = 0; i < reg.floors.size(); ++i)
	{
		if (reg.floors[i] == oldId)
			reg.floors[i] = newId;
	}
	if (neiChanged)
		removeAdjacentNeighbours(reg);
}

static bool canMergeWithRegion(const rcRegion& rega, const rcRegion& regb)
{
	if (rega.areaType != regb.areaType || rega.chunkId != regb.chunkId)
		return false;

	int n = 0;
	for (int i = 0; i < rega.connections.size(); ++i)
	{
		if (rega.connections[i] == regb.id)
			n++;
	}
	if (n > 1)
		return false;
	for (int i = 0; i < rega.floors.size(); ++i)
	{
		if (rega.floors[i] == regb.id)
			return false;
	}
	return true;
}

static void addUniqueFloorRegion(rcRegion& reg, int n)
{
	for (int i = 0; i < reg.floors.size(); ++i)
		if (reg.floors[i] == n)
			return;
	reg.floors.push(n);
}

static bool mergeRegions(rcRegion& rega, rcRegion& regb)
{
	unsigned short aid = rega.id;
	unsigned short bid = regb.id;
	
	// Duplicate current neighbourhood.
	rcIntArray acon;
	acon.resize(rega.connections.size());
	for (int i = 0; i < rega.connections.size(); ++i)
		acon[i] = rega.connections[i];
	rcIntArray& bcon = regb.connections;
	
	// Find insertion point on A.
	int insa = -1;
	for (int i = 0; i < acon.size(); ++i)
	{
		if (acon[i] == bid)
		{
			insa = i;
			break;
		}
	}
	if (insa == -1)
		return false;
	
	// Find insertion point on B.
	int insb = -1;
	for (int i = 0; i < bcon.size(); ++i)
	{
		if (bcon[i] == aid)
		{
			insb = i;
			break;
		}
	}
	if (insb == -1)
		return false;
	
	// Merge neighbours.
	rega.connections.resize(0);
	for (int i = 0, ni = acon.size(); i < ni-1; ++i)
		rega.connections.push(acon[(insa+1+i) % ni]);
		
	for (int i = 0, ni = bcon.size(); i < ni-1; ++i)
		rega.connections.push(bcon[(insb+1+i) % ni]);
	
	removeAdjacentNeighbours(rega);
	
	for (int j = 0; j < regb.floors.size(); ++j)
		addUniqueFloorRegion(rega, regb.floors[j]);
	rega.spanCount += regb.spanCount;
	regb.spanCount = 0;
	regb.connections.resize(0);

	return true;
}

static bool isRegionConnectedToBorder(const rcRegion& reg)
{
	// Region is connected to border if
	// one of the neighbours is null id.
	for (int i = 0; i < reg.connections.size(); ++i)
	{
		if (reg.connections[i] == 0)
			return true;
	}
	return false;
}

namespace RecastRegionFunc
{

static bool isSolidEdge(rcCompactHeightfield& chf, unsigned short* srcReg,
						int x, int y, int i, int dir)
{
	const rcCompactSpan& s = chf.spans[i];
	unsigned short r = 0;
	if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
	{
		const int ax = x + rcGetDirOffsetX(dir);
		const int ay = y + rcGetDirOffsetY(dir);
		const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(s, dir);
		r = srcReg[ai];
	}
	if (r == srcReg[i])
		return false;
	return true;
}

static void walkContour(int x, int y, int i, int dir,
						rcCompactHeightfield& chf,
						unsigned short* srcReg,
						rcIntArray& cont)
{
	int startDir = dir;
	int starti = i;

	const rcCompactSpan& ss = chf.spans[i];
	unsigned short curReg = 0;
	if (rcGetCon(ss, dir) != RC_NOT_CONNECTED)
	{
		const int ax = x + rcGetDirOffsetX(dir);
		const int ay = y + rcGetDirOffsetY(dir);
		const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(ss, dir);
		curReg = srcReg[ai];
	}
	cont.push(curReg);
			
	int iter = 0;
	while (++iter < 40000)
	{
		const rcCompactSpan& s = chf.spans[i];
		
		if (RecastRegionFunc::isSolidEdge(chf, srcReg, x, y, i, dir))
		{
			// Choose the edge corner
			unsigned short r = 0;
			if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
			{
				const int ax = x + rcGetDirOffsetX(dir);
				const int ay = y + rcGetDirOffsetY(dir);
				const int ai = (int)chf.cells[ax+ay*chf.width].index + rcGetCon(s, dir);
				r = srcReg[ai];
			}
			if (r != curReg)
			{
				curReg = r;
				cont.push(curReg);
			}
			
			dir = (dir+1) & 0x3;  // Rotate CW
		}
		else
		{
			int ni = -1;
			const int nx = x + rcGetDirOffsetX(dir);
			const int ny = y + rcGetDirOffsetY(dir);
			if (rcGetCon(s, dir) != RC_NOT_CONNECTED)
			{
				const rcCompactCell& nc = chf.cells[nx+ny*chf.width];
				ni = (int)nc.index + rcGetCon(s, dir);
			}
			if (ni == -1)
			{
				// Should not happen.
				return;
			}
			x = nx;
			y = ny;
			i = ni;
			dir = (dir+3) & 0x3;	// Rotate CCW
		}
		
		if (starti == i && startDir == dir)
		{
			break;
		}
	}

	// Remove adjacent duplicates.
	if (cont.size() > 1)
	{
		for (int j = 0; j < cont.size(); )
		{
			int nj = (j+1) % cont.size();
			if (cont[j] == cont[nj])
			{
				for (int k = j; k < cont.size()-1; ++k)
					cont[k] = cont[k+1];
				cont.pop();
			}
			else
				++j;
		}
	}
}

} // RecastRegionFunc

static bool filterSmallRegions(rcContext* ctx, int minRegionArea, int mergeRegionSize, int chunkSize,
							   unsigned short& maxRegionId,
							   rcCompactHeightfield& chf,
							   unsigned short* srcReg)
{
	const int w = chf.width;
	const int h = chf.height;
	
	const int nreg = maxRegionId+1;
	rcRegion* regions = (rcRegion*)rcAlloc(sizeof(rcRegion)*nreg, RC_ALLOC_TEMP);
	if (!regions)
	{
		ctx->log(RC_LOG_ERROR, "filterSmallRegions: Out of memory 'regions' (%d).", nreg);
		return false;
	}

	// Construct regions
	for (int i = 0; i < nreg; ++i)
		new(&regions[i]) rcRegion((unsigned short)i);
	
	// Find edge of a region and find connections around the contour.
	for (int y = 0; y < h; ++y)
	{
		for (int x = 0; x < w; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				unsigned short r = srcReg[i];
				if (r == 0 || r >= nreg)
					continue;
				
				rcRegion& reg = regions[r];
				reg.spanCount++;
				
				// Update floors.
				for (int j = (int)c.index; j < ni; ++j)
				{
					if (i == j) continue;
					unsigned short floorId = srcReg[j];
					if (floorId == 0 || floorId >= nreg)
						continue;
					addUniqueFloorRegion(reg, floorId);
				}
				
				// Have found contour
				if (reg.connections.size() > 0)
					continue;
				
				reg.areaType = chf.areas[i];
				reg.chunkId = (x / chunkSize) + (y / chunkSize) * chunkSize;
				
				// Check if this cell is next to a border.
				int ndir = -1;
				for (int dir = 0; dir < 4; ++dir)
				{
					if (RecastRegionFunc::isSolidEdge(chf, srcReg, x, y, i, dir))
					{
						ndir = dir;
						break;
					}
				}
				
				if (ndir != -1)
				{
					// The cell is at border.
					// Walk around the contour to find all the neighbours.
					RecastRegionFunc::walkContour(x, y, i, ndir, chf, srcReg, reg.connections);
				}
			}
		}
	}

	// Remove too small regions.
	rcIntArray stack(32);
	rcIntArray trace(32);
	for (int i = 0; i < nreg; ++i)
	{
		rcRegion& reg = regions[i];
		if (reg.id == 0 || (reg.id & RC_BORDER_REG))
			continue;                       
		if (reg.spanCount == 0)
			continue;
		if (reg.visited)
			continue;
		
		// Count the total size of all the connected regions.
		// Also keep track of the regions connects to a tile border.
		bool connectsToBorder = false;
		int spanCount = 0;
		stack.resize(0);
		trace.resize(0);

		reg.visited = true;
		stack.push(i);
		
		while (stack.size())
		{
			// Pop
			int ri = stack.pop();
			
			rcRegion& creg = regions[ri];

			spanCount += creg.spanCount;
			trace.push(ri);

			for (int j = 0; j < creg.connections.size(); ++j)
			{
				if (creg.connections[j] & RC_BORDER_REG)
				{
					connectsToBorder = true;
					continue;
				}
				rcRegion& neireg = regions[creg.connections[j]];
				if (neireg.visited)
					continue;
				if (neireg.id == 0 || (neireg.id & RC_BORDER_REG))
					continue;
				// Visit
				stack.push(neireg.id);
				neireg.visited = true;
			}
		}
		
		// If the accumulated regions size is too small, remove it.
		// Do not remove areas which connect to tile borders
		// as their size cannot be estimated correctly and removing them
		// can potentially remove necessary areas.
		if (spanCount < minRegionArea && !connectsToBorder)
		{
			// Kill all visited regions.
			for (int j = 0; j < trace.size(); ++j)
			{
				regions[trace[j]].spanCount = 0;
				regions[trace[j]].id = 0;
			}
		}
	}
		
	// Merge too small regions to neighbour regions.
	int mergeCount = 0 ;
	do
	{
		mergeCount = 0;
		for (int i = 0; i < nreg; ++i)
		{
			rcRegion& reg = regions[i];
			if (reg.id == 0 || (reg.id & RC_BORDER_REG))
				continue;                       
			if (reg.spanCount == 0)
				continue;
			
			// Check to see if the region should be merged.
			if (reg.spanCount > mergeRegionSize && isRegionConnectedToBorder(reg))
				continue;
			
			// Small region with more than 1 connection.
			// Or region which is not connected to a border at all.
			// Find smallest neighbour region that connects to this one.
			int smallest = 0xfffffff;
			unsigned short mergeId = reg.id;
			for (int j = 0; j < reg.connections.size(); ++j)
			{
				if (reg.connections[j] & RC_BORDER_REG) continue;
				rcRegion& mreg = regions[reg.connections[j]];
				if (mreg.id == 0 || (mreg.id & RC_BORDER_REG)) continue;
				if (mreg.spanCount < smallest &&
					canMergeWithRegion(reg, mreg) &&
					canMergeWithRegion(mreg, reg))
				{
					smallest = mreg.spanCount;
					mergeId = mreg.id;
				}
			}
			// Found new id.
			if (mergeId != reg.id)
			{
				unsigned short oldId = reg.id;
				rcRegion& target = regions[mergeId];
				
				// Merge neighbours.
				if (mergeRegions(target, reg))
				{
					// Fixup regions pointing to current region.
					for (int j = 0; j < nreg; ++j)
					{
						if (regions[j].id == 0 || (regions[j].id & RC_BORDER_REG)) continue;
						// If another region was already merged into current region
						// change the nid of the previous region too.
						if (regions[j].id == oldId)
							regions[j].id = mergeId;
						// Replace the current region with the new one if the
						// current regions is neighbour.
						replaceNeighbour(regions[j], oldId, mergeId);
					}
					mergeCount++;
				}
			}
		}
	}
	while (mergeCount > 0);
	
	// Compress region Ids.
	for (int i = 0; i < nreg; ++i)
	{
		regions[i].remap = false;
		if (regions[i].id == 0) continue;       // Skip nil regions.
		if (regions[i].id & RC_BORDER_REG) continue;    // Skip external regions.
		regions[i].remap = true;
	}
	
	unsigned short regIdGen = 0;
	for (int i = 0; i < nreg; ++i)
	{
		if (!regions[i].remap)
			continue;
		unsigned short oldId = regions[i].id;
		unsigned short newId = ++regIdGen;
		for (int j = i; j < nreg; ++j)
		{
			if (regions[j].id == oldId)
			{
				regions[j].id = newId;
				regions[j].remap = false;
			}
		}
	}
	maxRegionId = regIdGen;
	
	// Remap regions.
	for (int i = 0; i < chf.spanCount; ++i)
	{
		if ((srcReg[i] & RC_BORDER_REG) == 0)
			srcReg[i] = regions[srcReg[i]].id;
	}
	
	for (int i = 0; i < nreg; ++i)
		regions[i].~rcRegion();
	rcFree(regions);
	
	return true;
}

/// @par
/// 
/// This is usually the second to the last step in creating a fully built
/// compact heightfield.  This step is required before regions are built
/// using #rcBuildRegions or #rcBuildRegionsMonotone.
/// 
/// After this step, the distance data is available via the rcCompactHeightfield::maxDistance
/// and rcCompactHeightfield::dist fields.
///
/// @see rcCompactHeightfield, rcBuildRegions, rcBuildRegionsMonotone
bool rcBuildDistanceField(rcContext* ctx, rcCompactHeightfield& chf)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_BuildDistanceField);
	
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_BUILD_DISTANCEFIELD);
	
	if (chf.dist)
	{
		rcFree(chf.dist);
		chf.dist = 0;
	}
	
	unsigned short* src = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount, RC_ALLOC_TEMP);
	if (!src)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildDistanceField: Out of memory 'src' (%d).", chf.spanCount);
		return false;
	}
	unsigned short* dst = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount, RC_ALLOC_TEMP);
	if (!dst)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildDistanceField: Out of memory 'dst' (%d).", chf.spanCount);
		rcFree(src);
		return false;
	}
	
	unsigned short maxDist = 0;

	ctx->startTimer(RC_TIMER_BUILD_DISTANCEFIELD_DIST);
	
	calculateDistanceField(chf, src, maxDist);
	chf.maxDistance = maxDist;
	
	ctx->stopTimer(RC_TIMER_BUILD_DISTANCEFIELD_DIST);
	
	ctx->startTimer(RC_TIMER_BUILD_DISTANCEFIELD_BLUR);
	
	// Blur
	if (boxBlur(chf, 1, src, dst) != src)
		rcSwap(src, dst);
	
	// Store distance.
	chf.dist = src;
	
	ctx->stopTimer(RC_TIMER_BUILD_DISTANCEFIELD_BLUR);

	ctx->stopTimer(RC_TIMER_BUILD_DISTANCEFIELD);
	
	rcFree(dst);
	
	return true;
}

static void paintRectRegion(int minx, int maxx, int miny, int maxy, unsigned short regId,
							const rcCompactHeightfield& chf, unsigned short* srcReg)
{
	const int w = chf.width;	
	for (int y = miny; y < maxy; ++y)
	{
		for (int x = minx; x < maxx; ++x)
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				if (chf.areas[i] != RC_NULL_AREA)
					srcReg[i] = regId;
			}
		}
	}
}

// @UE BEGIN
namespace UE::Recast::Private
{
	void rcMarkBorderRegions(const rcBorderSize borderSize, const int w, const int h,
		rcCompactHeightfield& chf, unsigned short* srcReg, unsigned short& id)
	{
		if (borderSize.low > 0 || borderSize.high > 0)
		{
			const int lowW = rcMin(w, borderSize.low);
			const int lowH = rcMin(h, borderSize.low);
			const int highW = rcMin(w, borderSize.high);
			const int highH = rcMin(h, borderSize.high);
			
			// Paint regions
			paintRectRegion(0, lowW, 0, h, id|RC_BORDER_REG, chf, srcReg); id++;
			paintRectRegion(w-highW, w, 0, h, id|RC_BORDER_REG, chf, srcReg); id++;
			paintRectRegion(0, w, 0, lowH, id|RC_BORDER_REG, chf, srcReg); id++;
			paintRectRegion(0, w, h-highH, h, id|RC_BORDER_REG, chf, srcReg); id++;

			chf.borderSize = borderSize;
		}
	}
} //namespace UE::Recast::Private
// @UE END

static const unsigned short RC_NULL_NEI = 0xffff;

struct rcSweepSpan
{
	unsigned short rid;	// row id
	unsigned short id;	// region id
	unsigned short ns;	// number samples
	unsigned short nei;	// neighbour id
};

/// @par
/// 
/// Non-null regions will consist of connected, non-overlapping walkable spans that form a single contour.
/// Contours will form simple polygons.
/// 
/// If multiple regions form an area that is smaller than @p minRegionArea, then all spans will be
/// re-assigned to the zero (null) region.
/// 
/// Partitioning can result in smaller than necessary regions. @p mergeRegionArea helps 
/// reduce unecessarily small regions.
/// 
/// See the #rcConfig documentation for more information on the configuration parameters.
/// 
/// The region data will be available via the rcCompactHeightfield::maxRegions
/// and rcCompactSpan::reg fields.
/// 
/// @warning The distance field must be created using #rcBuildDistanceField before attempting to build regions.
/// 
/// @see rcCompactHeightfield, rcCompactSpan, rcBuildDistanceField, rcBuildRegionsMonotone, rcConfig
bool rcBuildRegionsMonotone(rcContext* ctx, rcCompactHeightfield& chf,
							const rcBorderSize borderSize, const int minRegionArea, const int mergeRegionArea)	//@UE
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_BUILD_REGIONS);
	
	const int w = chf.width;
	const int h = chf.height;
	unsigned short id = 1;
	
	rcScopedDelete<unsigned short> srcReg = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount, RC_ALLOC_TEMP);
	if (!srcReg)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildRegionsMonotone: Out of memory 'src' (%d).", chf.spanCount);
		return false;
	}
	memset(srcReg,0,sizeof(unsigned short)*chf.spanCount);

	const int nsweeps = rcMax(chf.width,chf.height);
	rcScopedDelete<rcSweepSpan> sweeps = (rcSweepSpan*)rcAlloc(sizeof(rcSweepSpan)*nsweeps, RC_ALLOC_TEMP);
	if (!sweeps)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildRegionsMonotone: Out of memory 'sweeps' (%d).", nsweeps);
		return false;
	}

	UE::Recast::Private::rcMarkBorderRegions(borderSize, w, h, chf, srcReg, id);		//@UE
	
	rcIntArray prev(256);

	// Sweep one line at a time.
	for (int y = borderSize.low; y < h-borderSize.high; ++y)		//@UE
	{
		// Collect spans from this row.
		prev.resize(id+1);
		memset(&prev[0],0,sizeof(int)*id);
		unsigned short rid = 1;
		
		for (int x = borderSize.low; x < w-borderSize.high; ++x)	//@UE
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				const rcCompactSpan& s = chf.spans[i];
				if (chf.areas[i] == RC_NULL_AREA) continue;
				
				// -x
				unsigned short previd = 0;
				if (rcGetCon(s, 0) != RC_NOT_CONNECTED)
				{
					const int ax = x + rcGetDirOffsetX(0);
					const int ay = y + rcGetDirOffsetY(0);
					const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 0);
					if ((srcReg[ai] & RC_BORDER_REG) == 0 && chf.areas[i] == chf.areas[ai])
						previd = srcReg[ai];
				}
				
				if (!previd)
				{
					previd = rid++;
					sweeps[previd].rid = previd;
					sweeps[previd].ns = 0;
					sweeps[previd].nei = 0;
				}

				// -y
				if (rcGetCon(s,3) != RC_NOT_CONNECTED)
				{
					const int ax = x + rcGetDirOffsetX(3);
					const int ay = y + rcGetDirOffsetY(3);
					const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 3);
					if (srcReg[ai] && (srcReg[ai] & RC_BORDER_REG) == 0 && chf.areas[i] == chf.areas[ai])
					{
						unsigned short nr = srcReg[ai];
						if (!sweeps[previd].nei || sweeps[previd].nei == nr)
						{
							sweeps[previd].nei = nr;
							sweeps[previd].ns++;
							prev[nr]++;
						}
						else
						{
							sweeps[previd].nei = RC_NULL_NEI;
						}
					}
				}

				srcReg[i] = previd;
			}
		}
		
		// Create unique ID.
		for (int i = 1; i < rid; ++i)
		{
			if (sweeps[i].nei != RC_NULL_NEI && sweeps[i].nei != 0 &&
				prev[sweeps[i].nei] == (int)sweeps[i].ns)
			{
				sweeps[i].id = sweeps[i].nei;
			}
			else
			{
				sweeps[i].id = id++;
			}
		}
		
		// Remap IDs
		for (int x = borderSize.low; x < w-borderSize.high; ++x)	//@UE
		{
			const rcCompactCell& c = chf.cells[x+y*w];
			
			for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
			{
				if (srcReg[i] > 0 && srcReg[i] < rid)
					srcReg[i] = sweeps[srcReg[i]].id;
			}
		}
	}

	ctx->startTimer(RC_TIMER_BUILD_REGIONS_FILTER);

	// Filter out small regions.
	chf.maxRegions = id;
	if (!filterSmallRegions(ctx, minRegionArea, mergeRegionArea, nsweeps, chf.maxRegions, chf, srcReg))
		return false;

	ctx->stopTimer(RC_TIMER_BUILD_REGIONS_FILTER);
	
	// Store the result out.
	for (int i = 0; i < chf.spanCount; ++i)
		chf.spans[i].reg = srcReg[i];
	
	ctx->stopTimer(RC_TIMER_BUILD_REGIONS);

	return true;
}

bool rcBuildRegionsChunky(rcContext* ctx, rcCompactHeightfield& chf,
						  const rcBorderSize borderSize, const int minRegionArea, const int mergeRegionArea,	//@UE
						  const int regionChunkSize)
{
	rcAssert(ctx);

	ctx->startTimer(RC_TIMER_BUILD_REGIONS);

	const int w = chf.width;
	const int h = chf.height;
	unsigned short id = 1;

	rcScopedDelete<unsigned short> srcReg = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount, RC_ALLOC_TEMP);
	if (!srcReg)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildRegionsChunky: Out of memory 'src' (%d).", chf.spanCount);
		return false;
	}
	memset(srcReg,0,sizeof(unsigned short)*chf.spanCount);

	const int nsweeps = regionChunkSize;
	rcScopedDelete<rcSweepSpan> sweeps = (rcSweepSpan*)rcAlloc(sizeof(rcSweepSpan)*nsweeps, RC_ALLOC_TEMP);
	if (!sweeps)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildRegionsChunky: Out of memory 'sweeps' (%d).", nsweeps);
		return false;
	}

	UE::Recast::Private::rcMarkBorderRegions(borderSize, w, h, chf, srcReg, id);		//@UE

	rcIntArray prev(256);
	for (int chunkx = borderSize.low; chunkx < w-borderSize.high; chunkx += regionChunkSize)		//@UE
	{
		for (int chunky = borderSize.low; chunky < h-borderSize.high; chunky += regionChunkSize)		//@UE
		{
			const int maxx = rcMin(chunkx + regionChunkSize, w-borderSize.high);		//@UE
			const int maxy = rcMin(chunky + regionChunkSize, h-borderSize.high);		//@UE

			// Sweep one line at a time.
			for (int y = chunky; y < maxy; ++y)
			{
				// Collect spans from this row.
				prev.resize(id+1);
				memset(&prev[0],0,sizeof(int)*id);
				unsigned short rid = 1;

				for (int x = chunkx; x < maxx; ++x)
				{
					const rcCompactCell& c = chf.cells[x+y*w];

					for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
					{
						const rcCompactSpan& s = chf.spans[i];
						if (chf.areas[i] == RC_NULL_AREA) continue;

						// -x
						unsigned short previd = 0;
						if (rcGetCon(s, 0) != RC_NOT_CONNECTED && x > chunkx)
						{
							const int ax = x + rcGetDirOffsetX(0);
							const int ay = y + rcGetDirOffsetY(0);
							const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 0);
							if ((srcReg[ai] & RC_BORDER_REG) == 0 && chf.areas[i] == chf.areas[ai])
								previd = srcReg[ai];
						}

						if (!previd)
						{
							previd = rid++;
							sweeps[previd].rid = previd;
							sweeps[previd].ns = 0;
							sweeps[previd].nei = 0;
						}

						// -y
						if (rcGetCon(s,3) != RC_NOT_CONNECTED && y > chunky)
						{
							const int ax = x + rcGetDirOffsetX(3);
							const int ay = y + rcGetDirOffsetY(3);
							const int ai = (int)chf.cells[ax+ay*w].index + rcGetCon(s, 3);
							if (srcReg[ai] && (srcReg[ai] & RC_BORDER_REG) == 0 && chf.areas[i] == chf.areas[ai])
							{
								unsigned short nr = srcReg[ai];
								if (!sweeps[previd].nei || sweeps[previd].nei == nr)
								{
									sweeps[previd].nei = nr;
									sweeps[previd].ns++;
									prev[nr]++;
								}
								else
								{
									sweeps[previd].nei = RC_NULL_NEI;
								}
							}
						}

						srcReg[i] = previd;
					}
				}

				// Create unique ID.
				for (int i = 1; i < rid; ++i)
				{
					if (sweeps[i].nei != RC_NULL_NEI && sweeps[i].nei != 0 &&
						prev[sweeps[i].nei] == (int)sweeps[i].ns)
					{
						sweeps[i].id = sweeps[i].nei;
					}
					else
					{
						sweeps[i].id = id++;
					}
				}

				// Remap IDs
				for (int x = chunkx; x < maxx; ++x)
				{
					const rcCompactCell& c = chf.cells[x+y*w];

					for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
					{
						if (srcReg[i] > 0 && srcReg[i] < rid)
							srcReg[i] = sweeps[srcReg[i]].id;
					}
				}
			}
		}
	}

	ctx->startTimer(RC_TIMER_BUILD_REGIONS_FILTER);

	// Filter out small regions.
	chf.maxRegions = id;
	if (!filterSmallRegions(ctx, minRegionArea, mergeRegionArea, nsweeps, chf.maxRegions, chf, srcReg))
		return false;

	ctx->stopTimer(RC_TIMER_BUILD_REGIONS_FILTER);

	// Store the result out.
	for (int i = 0; i < chf.spanCount; ++i)
		chf.spans[i].reg = srcReg[i];

	ctx->stopTimer(RC_TIMER_BUILD_REGIONS);

	return true;
}

/// @UE: rcBuildRegions is now split into two functions: gathering regions and merging them
///       it allows reusing existing code for building layer set (flood fill based partitioning)
///		  
///		  spanBuf4 is temporary buffer, allocated and freed by caller (size = chf.spanCount * 4)
/// 
///		  OutRegPtr points to where the caller should look for the results, which will be at either spanBuf4 or spanBuf4+chf.spanCount*2 //@UE
///
bool rcGatherRegionsNoFilter(rcContext* ctx, rcCompactHeightfield& chf, const rcBorderSize borderSize, unsigned short* spanBuf4, unsigned short*& resultBufPtr)		//@UE
{
	const int w = chf.width;
	const int h = chf.height;

	rcIntArray stack(1024);
	rcIntArray visited(1024);

	unsigned short* srcReg = spanBuf4;
	unsigned short* srcDist = spanBuf4+chf.spanCount;
	unsigned short* dstReg = spanBuf4+chf.spanCount*2;
	unsigned short* dstDist = spanBuf4+chf.spanCount*3;

	memset(srcReg, 0, sizeof(unsigned short)*chf.spanCount);
	memset(srcDist, 0, sizeof(unsigned short)*chf.spanCount);

	unsigned short regionId = 1;
	unsigned short level = (chf.maxDistance+1) & ~1;

	// TODO: Figure better formula, expandIters defines how much the 
	// watershed "overflows" and simplifies the regions. Tying it to
	// agent radius was usually good indication how greedy it could be.
	//	const int expandIters = 4 + walkableRadius * 2;
	const int expandIters = 8;

	UE::Recast::Private::rcMarkBorderRegions(borderSize, w, h, chf, srcReg, regionId);		//@UE

	while (level > 0)
	{
		level = level >= 2 ? level-2 : 0;

		ctx->startTimer(RC_TIMER_BUILD_REGIONS_EXPAND);

		// Expand current regions until no empty connected cells found.
		if (expandRegions(expandIters, level, chf, srcReg, srcDist, dstReg, dstDist, stack) != srcReg)
		{
			rcSwap(srcReg, dstReg);
			rcSwap(srcDist, dstDist);
		}

		ctx->stopTimer(RC_TIMER_BUILD_REGIONS_EXPAND);

		ctx->startTimer(RC_TIMER_BUILD_REGIONS_FLOOD);

		// Mark new regions with IDs.
		for (int y = 0; y < h; ++y)
		{
			for (int x = 0; x < w; ++x)
			{
				const rcCompactCell& c = chf.cells[x+y*w];
				for (int i = (int)c.index, ni = (int)(c.index+c.count); i < ni; ++i)
				{
					if (chf.dist[i] < level || srcReg[i] != 0 || chf.areas[i] == RC_NULL_AREA)
						continue;
					if (floodRegion(x, y, i, level, regionId, chf, srcReg, srcDist, stack))
						regionId++;
				}
			}
		}

		ctx->stopTimer(RC_TIMER_BUILD_REGIONS_FLOOD);
	}

	// Expand current regions until no empty connected cells found.
	if (expandRegions(expandIters*8, 0, chf, srcReg, srcDist, dstReg, dstDist, stack) != srcReg)
	{
		rcSwap(srcReg, dstReg);
		rcSwap(srcDist, dstDist);
	}

	// Because srcReg may not still be pointing at spanBuf4.
	resultBufPtr = srcReg;			//@UE

	chf.maxRegions = regionId;
	return true;
}

/// @par
/// 
/// Non-null regions will consist of connected, non-overlapping walkable spans that form a single contour.
/// Contours will form simple polygons.
/// 
/// If multiple regions form an area that is smaller than @p minRegionArea, then all spans will be
/// re-assigned to the zero (null) region.
/// 
/// Watershed partitioning can result in smaller than necessary regions, especially in diagonal corridors. 
/// @p mergeRegionArea helps reduce unnecessarily small regions.
/// 
/// See the #rcConfig documentation for more information on the configuration parameters.
/// 
/// The region data will be available via the rcCompactHeightfield::maxRegions
/// and rcCompactSpan::reg fields.
/// 
/// @warning The distance field must be created using #rcBuildDistanceField before attempting to build regions.
/// 
/// @see rcCompactHeightfield, rcCompactSpan, rcBuildDistanceField, rcBuildRegionsMonotone, rcConfig
bool rcBuildRegions(rcContext* ctx, rcCompactHeightfield& chf,
					const rcBorderSize borderSize, const int minRegionArea, const int mergeRegionArea)	//@UE
{
	rcAssert(ctx);
	
	ctx->startTimer(RC_TIMER_BUILD_REGIONS);
	
	rcScopedDelete<unsigned short> spanBuf4 = (unsigned short*)rcAlloc(sizeof(unsigned short)*chf.spanCount*4, RC_ALLOC_TEMP);
	if (!spanBuf4)
	{
		ctx->log(RC_LOG_ERROR, "rcBuildRegions: Out of memory 'spanBuf4' (%d).", chf.spanCount*4);
		return false;
	}

	ctx->startTimer(RC_TIMER_BUILD_REGIONS_WATERSHED);

	unsigned short* srcReg = spanBuf4;
	// srcReg might be updated and point at a different location.
	if (!rcGatherRegionsNoFilter(ctx, chf, borderSize, spanBuf4, srcReg))	//@UE
		return false;

	ctx->stopTimer(RC_TIMER_BUILD_REGIONS_WATERSHED);	
	ctx->startTimer(RC_TIMER_BUILD_REGIONS_FILTER);
	
	// Filter out small regions.
	const int chunkSize = rcMax(chf.width, chf.height);
	if (!filterSmallRegions(ctx, minRegionArea, mergeRegionArea, chunkSize, chf.maxRegions, chf, srcReg))
		return false;
	
	ctx->stopTimer(RC_TIMER_BUILD_REGIONS_FILTER);
		
	// Write the result out.
	for (int i = 0; i < chf.spanCount; ++i)
		chf.spans[i].reg = srcReg[i];
	
	ctx->stopTimer(RC_TIMER_BUILD_REGIONS);
	
	return true;
}


