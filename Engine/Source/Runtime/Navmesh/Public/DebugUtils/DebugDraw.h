// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DEBUGDRAW_H
#define DEBUGDRAW_H

#include "CoreMinimal.h"
#include "DebugDrawLargeWorldCoordinates.h"

// Some math headers don't have PI defined.
static const duReal DU_PI = 3.14159265358979323846;

inline float duSin(float x)
{
	return sinf(x);
}

inline double duSin(double x)
{
	return sin(x);
}

inline float duCos(float x)
{
	return cosf(x);
}

inline double duCos(double x)
{
	return cos(x);
}

inline float duAbs(float x)
{
	return fabsf(x);
}

inline double duAbs(double x)
{
	return fabs(x);
}

inline float duSqrt(float x)
{
	return sqrtf(x);
}

inline double duSqrt(double x)
{
	return sqrt(x);
}

enum duDebugDrawPrimitives
{
	DU_DRAW_POINTS,
	DU_DRAW_LINES,
	DU_DRAW_TRIS,
	DU_DRAW_QUADS,	
};

/// Abstract debug draw interface.
struct duDebugDraw
{
	NAVMESH_API virtual ~duDebugDraw() = 0;
	
	virtual void depthMask(bool state) = 0;

	virtual void texture(bool state) = 0;

	/// Begin drawing primitives.
	///  @param prim [in] primitive type to draw, one of rcDebugDrawPrimitives.
	///  @param size [in] size of a primitive, applies to point size and line width only.
	virtual void begin(duDebugDrawPrimitives prim, float size = 1.0f) = 0;

	/// Submit a vertex
	///  @param pos [in] position of the verts.
	///  @param color [in] color of the verts.
	virtual void vertex(const duReal* pos, unsigned int color) = 0;
//@UE BEGIN Adding support for LWCoords.
#if !DU_LARGE_WORLD_COORDINATES_DISABLED
	// float version of this function is now deprecated
	virtual void vertex(const float* pos, unsigned int color) final {}
#endif
//@UE END

	/// Submit a vertex
	///  @param x,y,z [in] position of the verts.
	///  @param color [in] color of the verts.
	virtual void vertex(const duReal x, const duReal y, const duReal z, unsigned int color) = 0;
//@UE BEGIN Adding support for LWCoords.
#if !DU_LARGE_WORLD_COORDINATES_DISABLED
	// float version of this function is now deprecated
	virtual void vertex(const float x, const float y, const float z, unsigned int color) final {}
#endif
//@UE END

	/// Submit a vertex
	///  @param pos [in] position of the verts.
	///  @param color [in] color of the verts.
	virtual void vertex(const duReal* pos, unsigned int color, const duReal* uv) = 0;
//@UE BEGIN Adding support for LWCoords.
#if !DU_LARGE_WORLD_COORDINATES_DISABLED
	// float version of this function is now deprecated
	virtual void vertex(const float* pos, unsigned int color, const float* uv) final {}
#endif
//@UE END

	/// Submit a vertex
	///  @param x,y,z [in] position of the verts.
	///  @param color [in] color of the verts.
	virtual void vertex(const duReal x, const duReal y, const duReal z, unsigned int color, const duReal u, const duReal v) = 0;
//@UE BEGIN Adding support for LWCoords.
#if !DU_LARGE_WORLD_COORDINATES_DISABLED
	// float version of this function is now deprecated
	virtual void vertex(const float x, const float y, const float z, unsigned int color, const float u, const float v) final {}
#endif
//@UE END

	/// Submit a text label
	///  @param x,y,z [in] 3D position of the text.
	///  @param text  [in] text to display.
	virtual void text(const duReal x, const duReal y, const duReal z, const char* text) = 0;
//@UE BEGIN Adding support for LWCoords.
#if !DU_LARGE_WORLD_COORDINATES_DISABLED
	// float version of this function is now deprecated
	virtual void text(const float x, const float y, const float z, const char* text) final {}
#endif
//@UE END
	
	/// End drawing primitives.
	virtual void end() = 0;
};

// Reorder to handle the 0xRRGGBBAA format
inline unsigned int duRGBA(unsigned int rgba)
{
	const unsigned int a = (rgba & 0x000000ff);
	const unsigned int b = (rgba & 0x0000ff00);
	const unsigned int g = (rgba & 0x00ff0000);
	const unsigned int r = (rgba & 0xff000000);
	return (r >> 24) | (g >> 8) | (b << 8) | a << 24;
}

inline unsigned int duRGBA(int r, int g, int b, int a)
{
	return ((unsigned int)r) | ((unsigned int)g << 8) | ((unsigned int)b << 16) | ((unsigned int)a << 24);
}

inline unsigned int duRGBAf(float fr, float fg, float fb, float fa)
{
	unsigned char r = (unsigned char)(fr*255.0f);
	unsigned char g = (unsigned char)(fg*255.0f);
	unsigned char b = (unsigned char)(fb*255.0f);
	unsigned char a = (unsigned char)(fa*255.0f);
	return duRGBA(r,g,b,a);
}

unsigned int duIntToCol(int i, int a);
void duIntToCol(int i, float* col);

inline unsigned int duMultCol(const unsigned int col, const unsigned int d)
{
	const unsigned int r = col & 0xff;
	const unsigned int g = (col >> 8) & 0xff;
	const unsigned int b = (col >> 16) & 0xff;
	const unsigned int a = (col >> 24) & 0xff;
	return duRGBA((r*d) >> 8, (g*d) >> 8, (b*d) >> 8, a);
}

inline unsigned int duDarkenCol(unsigned int col)
{
	return ((col >> 1) & 0x007f7f7f) | (col & 0xff000000);
}

inline unsigned int duLerpCol(unsigned int ca, unsigned int cb, unsigned int u)
{
	const unsigned int ra = ca & 0xff;
	const unsigned int ga = (ca >> 8) & 0xff;
	const unsigned int ba = (ca >> 16) & 0xff;
	const unsigned int aa = (ca >> 24) & 0xff;
	const unsigned int rb = cb & 0xff;
	const unsigned int gb = (cb >> 8) & 0xff;
	const unsigned int bb = (cb >> 16) & 0xff;
	const unsigned int ab = (cb >> 24) & 0xff;
	
	unsigned int r = (ra*(255-u) + rb*u)/255;
	unsigned int g = (ga*(255-u) + gb*u)/255;
	unsigned int b = (ba*(255-u) + bb*u)/255;
	unsigned int a = (aa*(255-u) + ab*u)/255;
	return duRGBA(r,g,b,a);
}

inline unsigned int duTransCol(unsigned int c, unsigned int a)
{
	return (a<<24) | (c & 0x00ffffff);
}

//@UE BEGIN
struct duColor
{
	// Color palette
	static unsigned int red;
	static unsigned int pink;
	static unsigned int purple;
	static unsigned int deepPurple;
	static unsigned int indigo;
	static unsigned int blue;
	static unsigned int lightBlue;
	static unsigned int cyan;
	static unsigned int teal;		
	static unsigned int green;
	static unsigned int lightGreen;
	static unsigned int lime;
	static unsigned int yellow;
	static unsigned int amber;
	static unsigned int orange;
	static unsigned int orangeRed;
	static unsigned int brown;
	
	static unsigned int white;
	static unsigned int lightGrey;
	static unsigned int grey;
	static unsigned int darkGrey;
	static unsigned int black;
};
//@UE END

void duCalcBoxColors(unsigned int* colors, unsigned int colTop, unsigned int colSide);

void duDebugDrawCylinderWire(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
							 duReal maxx, duReal maxy, duReal maxz, unsigned int col, const float lineWidth);

void duDebugDrawBoxWire(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
						duReal maxx, duReal maxy, duReal maxz, unsigned int col, const float lineWidth);

void duDebugDrawArc(struct duDebugDraw* dd, const duReal x0, const duReal y0, const duReal z0,
					const duReal x1, const duReal y1, const duReal z1, const duReal h,
					const duReal as0, const duReal as1, unsigned int col, const float lineWidth);

void duDebugDrawArrow(struct duDebugDraw* dd, const duReal x0, const duReal y0, const duReal z0,
					  const duReal x1, const duReal y1, const duReal z1,
					  const duReal as0, const duReal as1, unsigned int col, const float lineWidth);

void duDebugDrawCircle(struct duDebugDraw* dd, const duReal x, const duReal y, const duReal z,
					   const duReal r, unsigned int col, const float lineWidth);

void duDebugDrawCross(struct duDebugDraw* dd, const duReal x, const duReal y, const duReal z,
					  const duReal size, unsigned int col, const float lineWidth);

void duDebugDrawBox(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
					duReal maxx, duReal maxy, duReal maxz, const unsigned int* fcol);

void duDebugDrawCylinder(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
						 duReal maxx, duReal maxy, duReal maxz, unsigned int col);

void duDebugDrawGridXZ(struct duDebugDraw* dd, const duReal ox, const duReal oy, const duReal oz,
					   const int w, const int h, const duReal size,
					   const unsigned int col, const float lineWidth);


// Versions without begin/end, can be used to draw multiple primitives.
void duAppendCylinderWire(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
						  duReal maxx, duReal maxy, duReal maxz, unsigned int col);

void duAppendBoxWire(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
					 duReal maxx, duReal maxy, duReal maxz, unsigned int col);

void duAppendBoxPoints(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
					   duReal maxx, duReal maxy, duReal maxz, unsigned int col);

void duAppendArc(struct duDebugDraw* dd, const duReal x0, const duReal y0, const duReal z0,
				 const duReal x1, const duReal y1, const duReal z1, const duReal h,
				 const duReal as0, const duReal as1, unsigned int col);

void duAppendArcSegment(struct duDebugDraw* dd, const duReal xA0, const duReal yA0, const duReal zA0,
						const duReal xA1, const duReal yA1, const duReal zA1,
						const duReal xB0, const duReal yB0, const duReal zB0,
						const duReal xB1, const duReal yB1, const duReal zB1,
						const duReal h, unsigned int col);

void duAppendArrow(struct duDebugDraw* dd, const duReal x0, const duReal y0, const duReal z0,
				   const duReal x1, const duReal y1, const duReal z1,
				   const duReal as0, const duReal as1, unsigned int col);

void duAppendCircle(struct duDebugDraw* dd, const duReal x, const duReal y, const duReal z,
					const duReal r, unsigned int col);

void duAppendCross(struct duDebugDraw* dd, const duReal x, const duReal y, const duReal z,
				   const duReal size, unsigned int col);

void duAppendBox(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
				 duReal maxx, duReal maxy, duReal maxz, const unsigned int* fcol);

void duAppendCylinder(struct duDebugDraw* dd, duReal minx, duReal miny, duReal minz,
					  duReal maxx, duReal maxy, duReal maxz, unsigned int col);


class duDisplayList : public duDebugDraw
{
	duReal* m_pos;
	unsigned int* m_color;
	int m_size;
	int m_cap;

	bool m_depthMask;
	duDebugDrawPrimitives m_prim;
	float m_primSize;
	
	void resize(int cap);
	
public:
	duDisplayList(int cap = 512);
	virtual ~duDisplayList() override;
	virtual void depthMask(bool state);
	virtual void begin(duDebugDrawPrimitives prim, float size = 1.0f);
	virtual void vertex(const duReal x, const duReal y, const duReal z, unsigned int color);
	virtual void vertex(const duReal* pos, unsigned int color);
	virtual void end();
	void clear();
	void draw(struct duDebugDraw* dd);
};


#endif // DEBUGDRAW_H
