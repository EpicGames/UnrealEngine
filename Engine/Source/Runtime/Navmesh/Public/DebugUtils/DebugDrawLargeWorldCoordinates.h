// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define DU_LARGE_WORLD_COORDINATES_DISABLED 0

#if DU_LARGE_WORLD_COORDINATES_DISABLED

typedef float duReal;

#else // DU_LARGE_WORLD_COORDINATES_DISABLED

typedef double duReal;

#endif // DU_LARGE_WORLD_COORDINATES_DISABLED