// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURDEBUGDRAW_H
#define DETOURDEBUGDRAW_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavMesh.h"

class dtNavMeshQuery;
struct dtTileCacheDistanceField;
struct dtTileCacheLayer;
struct duDebugDraw;

enum DrawNavMeshFlags
{
	DU_DRAWNAVMESH_OFFMESHCONS = 0x01,
	DU_DRAWNAVMESH_CLOSEDLIST = 0x02,
	DU_DRAWNAVMESH_COLOR_TILES = 0x04,
};

NAVMESH_API void duDebugDrawNavMesh(struct duDebugDraw* dd, const dtNavMesh& mesh, unsigned char flags);
NAVMESH_API void duDebugDrawNavMeshWithClosedList(struct duDebugDraw* dd, const dtNavMesh& mesh, const dtNavMeshQuery& query, unsigned char flags);
NAVMESH_API void duDebugDrawNavMeshNodes(struct duDebugDraw* dd, const dtNavMeshQuery& query);
NAVMESH_API void duDebugDrawNavMeshBVTree(struct duDebugDraw* dd, const dtNavMesh& mesh);
NAVMESH_API void duDebugDrawNavMeshPortals(struct duDebugDraw* dd, const dtNavMesh& mesh);
NAVMESH_API void duDebugDrawNavMeshPolysWithFlags(struct duDebugDraw* dd, const dtNavMesh& mesh, const unsigned short polyFlags, const unsigned int col);
NAVMESH_API void duDebugDrawNavMeshPoly(struct duDebugDraw* dd, const dtNavMesh& mesh, dtPolyRef ref, const unsigned int col);

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
NAVMESH_API void duDebugDrawNavMeshClusters(struct duDebugDraw* dd, const dtNavMesh& mesh);
NAVMESH_API void duDebugDrawNavMeshCluster(struct duDebugDraw* dd, const dtNavMesh& mesh, dtClusterRef ref, const unsigned int col);

NAVMESH_API void duDebugDrawTileCacheClusters(duDebugDraw* dd, const struct dtTileCacheClusterSet& lclusters);
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

NAVMESH_API void duDebugDrawTileCacheLayerAreas(struct duDebugDraw* dd, const dtTileCacheLayer& layer, const dtReal cs, const dtReal ch);
NAVMESH_API void duDebugDrawTileCacheLayerDistance(struct duDebugDraw* dd, const dtTileCacheLayer& layer,
									   const dtTileCacheDistanceField& dfield, const dtReal cs, const dtReal ch);
NAVMESH_API void duDebugDrawTileCacheLayerRegions(struct duDebugDraw* dd, const dtTileCacheLayer& layer, const dtReal cs, const dtReal ch);
NAVMESH_API void duDebugDrawTileCacheContours(duDebugDraw* dd, const struct dtTileCacheContourSet& lcset,
								  const int32 layerIdx, const dtReal* orig, const dtReal cs, const dtReal ch);
NAVMESH_API void duDebugDrawTileCachePolyMesh(duDebugDraw* dd, const struct dtTileCachePolyMesh& lmesh,
								  const dtReal* orig, const dtReal cs, const dtReal ch);
NAVMESH_API void duDebugDrawTileCacheDetailMesh(duDebugDraw* dd, const struct dtTileCachePolyMeshDetail& dmesh);

#endif // DETOURDEBUGDRAW_H
