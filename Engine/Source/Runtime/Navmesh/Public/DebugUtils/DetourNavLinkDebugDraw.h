// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Detour/DetourNavLinkBuilder.h"

//@UE BEGIN
struct duDebugDraw;
class dtNavLinkBuilder;

enum duNavLinkBuilderDrawFlags
{
	DRAW_WALKABLE_SURFACE =	    1 << 0,
	DRAW_BORDERS =			    1 << 1,
	DRAW_SELECTED_EDGE =	    1 << 2,
	DRAW_TRAJECTORY =		    1 << 3,
	DRAW_LAND_SAMPLES =		    1 << 4,
	DRAW_COLLISION_SLICES =		1 << 5,
	DRAW_COLLISION_SAMPLES =	1 << 6,
	DRAW_LINKS =			    1 << 7,
	DRAW_FILTERED_LINKS =	    1 << 8,
};

NAVMESH_API void duDebugDrawNavLinkBuilder(duDebugDraw* dd, const dtNavLinkBuilder& linkBuilder, unsigned int drawFlags, const struct dtNavLinkBuilder::EdgeSampler* es);

NAVMESH_API void duDebugDrawTrajectorySamples(duDebugDraw* dd, const dtNavLinkBuilder& linkBuilder, const dtReal* pa, const dtReal* pb,
											  const dtNavLinkBuilder::Trajectory2D* trajectory, const dtReal* trajectoryDir);

NAVMESH_API void duDrawGroundSegment(duDebugDraw* dd, const dtNavLinkBuilder::GroundSegment& segment);
//@UE END
