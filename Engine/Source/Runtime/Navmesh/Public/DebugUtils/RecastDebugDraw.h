// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef RECAST_DEBUGDRAW_H
#define RECAST_DEBUGDRAW_H

#include "CoreMinimal.h"
#include "DebugDrawLargeWorldCoordinates.h"

struct duDebugDraw;
struct rcHeightfield;

NAVMESH_API void duDebugDrawTriMesh(struct duDebugDraw* dd, const duReal* verts, int nverts, const int* tris, const duReal* normals, int ntris, const unsigned char* flags, const duReal texScale);
NAVMESH_API void duDebugDrawTriMeshSlope(struct duDebugDraw* dd, const duReal* verts, int nverts, const int* tris, const duReal* normals, int ntris, const duReal walkableSlopeAngle, const duReal texScale);

NAVMESH_API void duDebugDrawHeightfieldSolid(duDebugDraw* dd, const rcHeightfield& hf);
NAVMESH_API void duDebugDrawHeightfieldWalkable(duDebugDraw* dd, const rcHeightfield& hf);
NAVMESH_API void duDebugDrawHeightfieldBounds(duDebugDraw* dd, const rcHeightfield& hf);

NAVMESH_API void duDebugDrawCompactHeightfieldSolid(struct duDebugDraw* dd, const struct rcCompactHeightfield& chf);
NAVMESH_API void duDebugDrawCompactHeightfieldRegions(struct duDebugDraw* dd, const struct rcCompactHeightfield& chf);
NAVMESH_API void duDebugDrawCompactHeightfieldDistance(struct duDebugDraw* dd, const struct rcCompactHeightfield& chf);

NAVMESH_API void duDrawCylinderArea(duDebugDraw* dd, const duReal* pos, const duReal radius, const duReal height, const unsigned int areaRGBAColor);
NAVMESH_API void duDrawBoxArea(duDebugDraw* dd, const duReal* pos, const duReal* extent, const unsigned int areaRGBAColor);
NAVMESH_API void duDrawConvexArea(duDebugDraw* dd, const duReal* verts, const int nverts, const duReal hmin, const duReal hmax, const unsigned int areaRGBAColor);

NAVMESH_API void duDebugDrawHeightfieldLayer(duDebugDraw* dd, const struct rcHeightfieldLayer& layer, const int idx);
NAVMESH_API void duDebugDrawHeightfieldLayers(duDebugDraw* dd, const struct rcHeightfieldLayerSet& lset);
NAVMESH_API void duDebugDrawHeightfieldLayersRegions(duDebugDraw* dd, const struct rcHeightfieldLayerSet& lset);

NAVMESH_API void duDebugDrawLayerContours(duDebugDraw* dd, const struct rcLayerContourSet& lcset);
NAVMESH_API void duDebugDrawLayerPolyMesh(duDebugDraw* dd, const struct rcLayerPolyMesh& lmesh);

NAVMESH_API void duDebugDrawRegionConnections(struct duDebugDraw* dd, const struct rcContourSet& cset, const float alpha = 1.0f);
NAVMESH_API void duDebugDrawRawContours(struct duDebugDraw* dd, const struct rcContourSet& cset, const float alpha = 1.0f);
NAVMESH_API void duDebugDrawContours(struct duDebugDraw* dd, const struct rcContourSet& cset, const float alpha = 1.0f);
NAVMESH_API void duDebugDrawPolyMesh(struct duDebugDraw* dd, const struct rcPolyMesh& mesh);
NAVMESH_API void duDebugDrawPolyMeshDetail(struct duDebugDraw* dd, const struct rcPolyMeshDetail& dmesh);

#endif // RECAST_DEBUGDRAW_H
