// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURALLOCATOR_H
#define DETOURALLOCATOR_H

#include "CoreMinimal.h"
#include "Detour/DetourAssert.h"

//@UE BEGIN Adding support for memory tracking.
/// Provides hint values on how the memory is expected to be used. Typically used by external memory allocation and tracking systems.
enum dtAllocHint
{
	DT_ALLOC_TEMP,		///< Memory used temporarily within a function.
	DT_ALLOC_PERM_AVOIDANCE,
	DT_ALLOC_PERM_CROWD,
	DT_ALLOC_PERM_LOOKUP,
	DT_ALLOC_PERM_NAVMESH,
	DT_ALLOC_PERM_NAVQUERY,
	DT_ALLOC_PERM_NODE_POOL,
	DT_ALLOC_PERM_PATH_CORRIDOR,
	DT_ALLOC_PERM_PATH_QUEUE,
	DT_ALLOC_PERM_PROXIMITY_GRID,
	DT_ALLOC_PERM_TILE_CACHE_LAYER,
	DT_ALLOC_PERM_TILE_DATA,
	DT_ALLOC_PERM_TILE_DYNLINK,
	DT_ALLOC_PERM_TILE_DYNLINK_CLUSTER,
	DT_ALLOC_PERM_TILES,
	DT_ALLOC_PERM_TILE_LINK_BUILDER,
};
//@UE END Adding support for memory tracking.

/// A memory allocation function.
//  @param[in]		size			The size, in bytes of memory, to allocate.
//  @param[in]		rcAllocHint	A hint to the allocator on how long the memory is expected to be in use.
//  @return A pointer to the beginning of the allocated memory block, or null if the allocation failed.
///  @see dtAllocSetCustom
typedef void* (dtAllocFunc)(int size, dtAllocHint hint);

/// A memory deallocation function.
///  @param[in]		ptr		A pointer to a memory block previously allocated using #dtAllocFunc.
/// @see dtAllocSetCustom
//@UE BEGIN Adding support for memory tracking.
typedef void (dtFreeFunc)(void* ptr, dtAllocHint hint);
//@UE END Adding support for memory tracking.

/// Sets the base custom allocation functions to be used by Detour.
///  @param[in]		allocFunc	The memory allocation function to be used by #dtAlloc
///  @param[in]		freeFunc	The memory de-allocation function to be used by #dtFree
NAVMESH_API void dtAllocSetCustom(dtAllocFunc *allocFunc, dtFreeFunc *freeFunc);

/// Allocates a memory block.
///  @param[in]		size	The size, in bytes of memory, to allocate.
///  @param[in]		hint	A hint to the allocator on how long the memory is expected to be in use.
///  @return A pointer to the beginning of the allocated memory block, or null if the allocation failed.
/// @see dtFree
NAVMESH_API void* dtAlloc(int size, dtAllocHint hint);

/// Deallocates a memory block.
///  @param[in]		ptr		A pointer to a memory block previously allocated using #dtAlloc.
/// @see dtAlloc
//@UE BEGIN Adding support for memory tracking.
NAVMESH_API void dtFree(void* ptr, dtAllocHint hint);
//@UE END Adding support for memory tracking.

NAVMESH_API void dtMemCpy(void* dst, void* src, int size);

/// A simple helper class used to delete an array when it goes out of scope.
/// @note This class is rarely if ever used by the end user.
template<class T> class dtScopedDelete
{
	T* ptr;
	inline T* operator=(T* p);
public:

	/// Constructs an instance with a null pointer.
	inline dtScopedDelete() : ptr(0) {}
	inline dtScopedDelete(int n) { ptr = n ? (T*)dtAlloc(sizeof(T)*n, DT_ALLOC_TEMP) : 0; }

	/// Constructs an instance with the specified pointer.
	///  @param[in]		p	An pointer to an allocated array.
	inline dtScopedDelete(T* p) : ptr(p) {}
	inline ~dtScopedDelete() { dtFree(ptr, DT_ALLOC_TEMP); }

	/// The root array pointer.
	///  @return The root array pointer.
	inline operator T*() { return ptr; }
	inline T* get() { return ptr; }
};

/// A simple dynamic array of integers.
class dtIntArray
{
	int* m_data;
	int m_size, m_cap;
	inline dtIntArray(const dtIntArray&);
	inline dtIntArray& operator=(const dtIntArray&);
public:

	/// Constructs an instance with an initial array size of zero.
	inline dtIntArray() : m_data(0), m_size(0), m_cap(0) {}

	/// Constructs an instance initialized to the specified size.
	///  @param[in]		n	The initial size of the integer array.
	inline dtIntArray(int n) : m_data(0), m_size(0), m_cap(0) { resize(n); }
	inline ~dtIntArray() { dtFree(m_data, DT_ALLOC_TEMP); }

	/// Specifies the new size of the integer array.
	///  @param[in]		n	The new size of the integer array.
	void resize(int n);

	/// Push the specified integer onto the end of the array and increases the size by one.
	///  @param[in]		item	The new value.
	inline void push(int item) { resize(m_size+1); m_data[m_size-1] = item; }

	/// Returns the value at the end of the array and reduces the size by one.
	///  @return The value at the end of the array.
	inline int pop() { if (m_size > 0) m_size--; return m_data[m_size]; }

	/// The value at the specified array index.
	/// @warning Does not provide overflow protection.
	///  @param[in]		i	The index of the value.
	inline const int& operator[](int i) const { return m_data[i]; }

	/// The value at the specified array index.
	/// @warning Does not provide overflow protection.
	///  @param[in]		i	The index of the value.
	inline int& operator[](int i) { return m_data[i]; }

	/// The current size of the integer array.
	inline int size() const { return m_size; }

	inline int* getData() const { return m_data; }

	void copy(const dtIntArray& src);
	
	bool contains(int v) const 
	{
		for (int i = 0; i < m_size; i++)
			if (m_data[i] == v)
				return true;

		return false;
	}
};

/// A simple dynamic array of integers.
template<class T, dtAllocHint TAllocHint = DT_ALLOC_TEMP> //UE Adding support for memory tracking.
class dtChunkArray
{
	T* m_data;
	int m_size, m_cap;

	inline dtChunkArray(const dtChunkArray&);
	inline dtChunkArray& operator=(const dtChunkArray&);
public:

	/// Constructs an instance with an initial array size of zero.
	inline dtChunkArray() : m_data(0), m_size(0), m_cap(0){}

	/// Constructs an instance initialized to the specified size.
	///  @param[in]		n	The initial size of the integer array.
	inline dtChunkArray(int n) : m_data(0), m_size(0), m_cap(0){ resize(n); }
	inline ~dtChunkArray() { dtFree(m_data, TAllocHint); }

	/// Specifies the new size of the integer array.
	///  @param[in]		n	The new size of the integer array.
	void resize(int n);

	/// Push the specified integer onto the end of the array and increases the size by one.
	///  @param[in]		item	The new value.
	inline void push(T item) { resize(m_size+1); m_data[m_size-1] = item; }

	/// Returns the value at the end of the array and reduces the size by one.
	///  @return The value at the end of the array.
	inline T pop() { if (m_size > 0) m_size--; return m_data[m_size]; }

	/// The value at the specified array index.
	/// @warning Does not provide overflow protection.
	///  @param[in]		i	The index of the value.
	inline const T& operator[](int i) const { return m_data[i]; }

	/// The value at the specified array index.
	/// @warning Does not provide overflow protection.
	///  @param[in]		i	The index of the value.
	inline T& operator[](int i) { return m_data[i]; }

	/// The current size of the integer array.
	inline int size() const { return m_size; }
};

template<class T, dtAllocHint TAllocHint>
void dtChunkArray<T, TAllocHint>::resize(int n)
{
	if (n > m_cap)
	{
		if (!m_cap) m_cap = n;
		while (m_cap < n) m_cap += 32;
		T* newData = (T*)dtAlloc(m_cap*sizeof(T), TAllocHint);
		if (m_size && newData) dtMemCpy(newData, m_data, m_size*sizeof(T));
		dtFree(m_data, TAllocHint);
		m_data = newData;
	}
	m_size = n;
}

#endif
