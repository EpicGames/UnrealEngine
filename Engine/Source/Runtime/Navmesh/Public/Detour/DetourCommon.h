// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURCOMMON_H
#define DETOURCOMMON_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "HAL/PlatformMath.h"

// DT_STATS is set from the UE Stats settings
#define DT_STATS STATS

/**
@defgroup detour Detour

Members in this module are used to create, manipulate, and query navigation 
meshes.

@note This is a summary list of members.  Use the index or search 
feature to find minor members.
*/

/// @name General helper functions
/// @{

/// Swaps the values of the two parameters.
///  @param[in,out]	a	Value A
///  @param[in,out]	b	Value B
template<class T> inline void dtSwap(T& a, T& b) { T t = a; a = b; b = t; }

/// Returns the minimum of two values.
///  @param[in]		a	Value A
///  @param[in]		b	Value B
///  @return The minimum of the two values.
template<class T> inline T dtMin(T a, T b) { return a < b ? a : b; }
//@UE BEGIN Adding support for LWCoords. Overloading allows this to be called where one of the parameters is a double and the other a float.
inline dtReal dtMin(dtReal a, dtReal b) { return dtMin<dtReal>(a, b); }
//@UE END Adding support for LWCoords.

/// Returns the maximum of two values.
///  @param[in]		a	Value A
///  @param[in]		b	Value B
///  @return The maximum of the two values.
template<class T> inline T dtMax(T a, T b) { return a > b ? a : b; }
//@UE BEGIN Adding support for LWCoords. Overloading allows this to be called where one of the parameters is a double and the other a float.
inline dtReal dtMax(dtReal a, dtReal b) { return dtMax<dtReal>(a, b); }
//@UE END Adding support for LWCoords.

/// Returns the absolute value.
///  @param[in]		a	The value.
///  @return The absolute value of the specified value.
template<class T> inline T dtAbs(T a) { return a < 0 ? -a : a; }

/// Returns the square of the value.
///  @param[in]		a	The value.
///  @return The square of the value.
template<class T> inline T dtSqr(T a) { return a*a; }

/// Clamps the value to the specified range.
///  @param[in]		v	The value to clamp.
///  @param[in]		mn	The minimum permitted return value.
///  @param[in]		mx	The maximum permitted return value.
///  @return The value, clamped to the specified range.
template<class T> inline T dtClamp(T v, T mn, T mx) { return v < mn ? mn : (v > mx ? mx : v); }
//@UE BEGIN Adding support for LWCoords. Overloading allows this to be called where one of the parameters is a double and the other a float.
inline dtReal dtClamp(dtReal v, dtReal mn, dtReal mx) { return dtClamp<dtReal>(v, mn, mx); }
//@UE END Adding support for LWCoords.

inline float dtFloor(float x)
{
	return floorf(x);
}

inline double dtFloor(double x)
{
	return floor(x);
}

inline float dtCeil(float x)
{
	return ceilf(x);
}

inline double dtCeil(double x)
{
	return ceil(x);
}

inline float dtSin(float x)
{
	return sinf(x);
}

inline double dtSin(double x)
{
	return sin(x);
}

inline float dtCos(float x)
{
	return cosf(x);
}

inline double dtCos(double x)
{
	return cos(x);
}

inline float dtAtan2(float x, float y)
{
	return atan2f(x, y);
}

inline double dtAtan2(double x, double y)
{
	return atan2(x, y);
}

inline float dtSqrt(float x)
{
	return sqrtf(x);
}

inline double dtSqrt(double x)
{
	return sqrt(x);
}

inline float dtfMod(float x, float y)
{
	return fmodf(x, y);
}

inline double dtfMod(double x, double y)
{
	return fmod(x, y);
}

inline float dtLerp(float a, float b, float t)
{
	return a + (b-a)*t;
}

/// @}
/// @name Vector helper functions.
/// @{

/// Derives the cross product of two vectors. (@p v1 x @p v2)
///  @param[out]	dest	The cross product. [(x, y, z)]
///  @param[in]		v1		A Vector [(x, y, z)]
///  @param[in]		v2		A vector [(x, y, z)]
inline void dtVcross(dtReal* dest, const dtReal* v1, const dtReal* v2)
{
	dest[0] = v1[1]*v2[2] - v1[2]*v2[1];
	dest[1] = v1[2]*v2[0] - v1[0]*v2[2];
	dest[2] = v1[0]*v2[1] - v1[1]*v2[0]; 
}

/// Derives the dot product of two vectors. (@p v1 . @p v2)
///  @param[in]		v1	A Vector [(x, y, z)]
///  @param[in]		v2	A vector [(x, y, z)]
/// @return The dot product.
inline dtReal dtVdot(const dtReal* v1, const dtReal* v2)
{
	return v1[0]*v2[0] + v1[1]*v2[1] + v1[2]*v2[2];
}

/// Performs a scaled vector addition. (@p v1 + (@p v2 * @p s))
///  @param[out]	dest	The result vector. [(x, y, z)]
///  @param[in]		v1		The base vector. [(x, y, z)]
///  @param[in]		v2		The vector to scale and add to @p v1. [(x, y, z)]
///  @param[in]		s		The amount to scale @p v2 by before adding to @p v1.
inline void dtVmad(dtReal* dest, const dtReal* v1, const dtReal* v2, const dtReal s)
{
	dest[0] = v1[0]+v2[0]*s;
	dest[1] = v1[1]+v2[1]*s;
	dest[2] = v1[2]+v2[2]*s;
}

/// Performs a linear interpolation between two vectors. (@p v1 toward @p v2)
///  @param[out]	dest	The result vector. [(x, y, x)]
///  @param[in]		v1		The starting vector.
///  @param[in]		v2		The destination vector.
///	 @param[in]		t		The interpolation factor. [Limits: 0 <= value <= 1.0]
inline void dtVlerp(dtReal* dest, const dtReal* v1, const dtReal* v2, const dtReal t)
{
	dest[0] = v1[0]+(v2[0]-v1[0])*t;
	dest[1] = v1[1]+(v2[1]-v1[1])*t;
	dest[2] = v1[2]+(v2[2]-v1[2])*t;
}

/// Performs a vector addition. (@p v1 + @p v2)
///  @param[out]	dest	The result vector. [(x, y, z)]
///  @param[in]		v1		The base vector. [(x, y, z)]
///  @param[in]		v2		The vector to add to @p v1. [(x, y, z)]
inline void dtVadd(dtReal* dest, const dtReal* v1, const dtReal* v2)
{
	dest[0] = v1[0]+v2[0];
	dest[1] = v1[1]+v2[1];
	dest[2] = v1[2]+v2[2];
}

/// Performs a vector subtraction. (@p v1 - @p v2)
///  @param[out]	dest	The result vector. [(x, y, z)]
///  @param[in]		v1		The base vector. [(x, y, z)]
///  @param[in]		v2		The vector to subtract from @p v1. [(x, y, z)]
inline void dtVsub(dtReal* dest, const dtReal* v1, const dtReal* v2)
{
	dest[0] = v1[0]-v2[0];
	dest[1] = v1[1]-v2[1];
	dest[2] = v1[2]-v2[2];
}

/// Scales the vector by the specified value. (@p v * @p t)
///  @param[out]	dest	The result vector. [(x, y, z)]
///  @param[in]		v		The vector to scale. [(x, y, z)]
///  @param[in]		t		The scaling factor.
inline void dtVscale(dtReal* dest, const dtReal* v, const dtReal t)
{
	dest[0] = v[0]*t;
	dest[1] = v[1]*t;
	dest[2] = v[2]*t;
}

/// Selects the minimum value of each element from the specified vectors.
///  @param[in,out]	mn	A vector.  (Will be updated with the result.) [(x, y, z)]
///  @param[in]	v	A vector. [(x, y, z)]
inline void dtVmin(dtReal* mn, const dtReal* v)
{
	mn[0] = dtMin(mn[0], v[0]);
	mn[1] = dtMin(mn[1], v[1]);
	mn[2] = dtMin(mn[2], v[2]);
}

/// Selects the maximum value of each element from the specified vectors.
///  @param[in,out]	mx	A vector.  (Will be updated with the result.) [(x, y, z)]
///  @param[in]		v	A vector. [(x, y, z)]
inline void dtVmax(dtReal* mx, const dtReal* v)
{
	mx[0] = dtMax(mx[0], v[0]);
	mx[1] = dtMax(mx[1], v[1]);
	mx[2] = dtMax(mx[2], v[2]);
}

/// Sets the vector elements to the specified values.
///  @param[out]	dest	The result vector. [(x, y, z)]
///  @param[in]		x		The x-value of the vector.
///  @param[in]		y		The y-value of the vector.
///  @param[in]		z		The z-value of the vector.
inline void dtVset(dtReal* dest, const dtReal x, const dtReal y, const dtReal z)
{
	dest[0] = x; dest[1] = y; dest[2] = z;
}

/// Performs a vector copy.
///  @param[out]	dest	The result. [(x, y, z)]
///  @param[in]		a		The vector to copy. [(x, y, z)]
inline void dtVcopy(dtReal* dest, const dtReal* a)
{
	dest[0] = a[0];
	dest[1] = a[1];
	dest[2] = a[2];
}

/// Derives the scalar length of the vector.
///  @param[in]		v The vector. [(x, y, z)]
/// @return The scalar length of the vector.
inline dtReal dtVlen(const dtReal* v)
{
	return dtSqrt(v[0]*v[0] + v[1]*v[1] + v[2]*v[2]);
}

/// Derives the square of the scalar length of the vector. (len * len)
///  @param[in]		v The vector. [(x, y, z)]
/// @return The square of the scalar length of the vector.
inline dtReal dtVlenSqr(const dtReal* v)
{
	return v[0]*v[0] + v[1]*v[1] + v[2]*v[2];
}

/// Returns the distance between two points.
///  @param[in]		v1	A point. [(x, y, z)]
///  @param[in]		v2	A point. [(x, y, z)]
/// @return The distance between the two points.
inline dtReal dtVdist(const dtReal* v1, const dtReal* v2)
{
	const dtReal dx = v2[0] - v1[0];
	const dtReal dy = v2[1] - v1[1];
	const dtReal dz = v2[2] - v1[2];
	return dtSqrt(dx*dx + dy*dy + dz*dz);
}

/// Returns the square of the distance between two points.
///  @param[in]		v1	A point. [(x, y, z)]
///  @param[in]		v2	A point. [(x, y, z)]
/// @return The square of the distance between the two points.
inline dtReal dtVdistSqr(const dtReal* v1, const dtReal* v2)
{
	const dtReal dx = v2[0] - v1[0];
	const dtReal dy = v2[1] - v1[1];
	const dtReal dz = v2[2] - v1[2];
	return dx*dx + dy*dy + dz*dz;
}

/// Derives the distance between the specified points on the xz-plane.
///  @param[in]		v1	A point. [(x, y, z)]
///  @param[in]		v2	A point. [(x, y, z)]
/// @return The distance between the point on the xz-plane.
///
/// The vectors are projected onto the xz-plane, so the y-values are ignored.
inline dtReal dtVdist2D(const dtReal* v1, const dtReal* v2)
{
	const dtReal dx = v2[0] - v1[0];
	const dtReal dz = v2[2] - v1[2];
	return dtSqrt(dx*dx + dz*dz);
}

/// Derives the square of the distance between the specified points on the xz-plane.
///  @param[in]		v1	A point. [(x, y, z)]
///  @param[in]		v2	A point. [(x, y, z)]
/// @return The square of the distance between the point on the xz-plane.
inline dtReal dtVdist2DSqr(const dtReal* v1, const dtReal* v2)
{
	const dtReal dx = v2[0] - v1[0];
	const dtReal dz = v2[2] - v1[2];
	return dx*dx + dz*dz;
}

/// Normalizes the vector.
///  @param[in,out]	v	The vector to normalize. [(x, y, z)]
inline void dtVnormalize(dtReal* v)
{
	dtReal d = 1.0f / dtSqrt(dtSqr(v[0]) + dtSqr(v[1]) + dtSqr(v[2]));
	v[0] *= d;
	v[1] *= d;
	v[2] *= d;
}

/// Returns true if the vectors are the same.
///  @param[in]		a	A vector. [(x, y, z)]
///  @param[in]		b	A vector. [(x, y, z)]
inline bool dtVisEqual(const unsigned short* a, const unsigned short* b)
{
	return a[0] == b[0] && a[1] == b[1] && a[2] == b[2];
}

/// Performs a 'sloppy' colocation check of the specified points.
///  @param[in]		p0	A point. [(x, y, z)]
///  @param[in]		p1	A point. [(x, y, z)]
/// @return True if the points are considered to be at the same location.
///
/// Basically, this function will return true if the specified points are 
/// close enough to eachother to be considered colocated.
inline bool dtVequal(const dtReal* p0, const dtReal* p1)
{
	static const dtReal thr = dtSqr(dtReal(1.)/16384.0f);
	const dtReal d = dtVdistSqr(p0, p1);
	return d < thr;
}

/// Derives the dot product of two vectors on the xz-plane. (@p u . @p v)
///  @param[in]		u		A vector [(x, y, z)]
///  @param[in]		v		A vector [(x, y, z)]
/// @return The dot product on the xz-plane.
///
/// The vectors are projected onto the xz-plane, so the y-values are ignored.
inline dtReal dtVdot2D(const dtReal* u, const dtReal* v)
{
	return u[0]*v[0] + u[2]*v[2];
}

/// Derives the xz-plane 2D perp product of the two vectors. (uz*vx - ux*vz)
///  @param[in]		u		The LHV vector [(x, y, z)]
///  @param[in]		v		The RHV vector [(x, y, z)]
/// @return The dot product on the xz-plane.
///
/// The vectors are projected onto the xz-plane, so the y-values are ignored.
inline dtReal dtVperp2D(const dtReal* u, const dtReal* v)
{
	return u[2]*v[0] - u[0]*v[2];
}

/// @}
/// @name Computational geometry helper functions.
/// @{

/// Derives the signed xz-plane area of the triangle ABC, or the relationship of line AB to point C.
///  @param[in]		a		Vertex A. [(x, y, z)]
///  @param[in]		b		Vertex B. [(x, y, z)]
///  @param[in]		c		Vertex C. [(x, y, z)]
/// @return The signed xz-plane area of the triangle.
inline dtReal dtTriArea2D(const dtReal* a, const dtReal* b, const dtReal* c)
{
	const dtReal abx = b[0] - a[0];
	const dtReal abz = b[2] - a[2];
	const dtReal acx = c[0] - a[0];
	const dtReal acz = c[2] - a[2];
	return acx*abz - abx*acz;
}

/// Determines if two axis-aligned bounding boxes overlap.
///  @param[in]		amin	Minimum bounds of box A. [(x, y, z)]
///  @param[in]		amax	Maximum bounds of box A. [(x, y, z)]
///  @param[in]		bmin	Minimum bounds of box B. [(x, y, z)]
///  @param[in]		bmax	Maximum bounds of box B. [(x, y, z)]
/// @return True if the two AABB's overlap.
/// @see dtOverlapBounds
inline bool dtOverlapQuantBounds(const unsigned short amin[3], const unsigned short amax[3],
								 const unsigned short bmin[3], const unsigned short bmax[3])
{
	bool overlap = true;
	overlap = (amin[0] > bmax[0] || amax[0] < bmin[0]) ? false : overlap;
	overlap = (amin[1] > bmax[1] || amax[1] < bmin[1]) ? false : overlap;
	overlap = (amin[2] > bmax[2] || amax[2] < bmin[2]) ? false : overlap;
	return overlap;
}

/// Determines if two axis-aligned bounding boxes overlap.
///  @param[in]		amin	Minimum bounds of box A. [(x, y, z)]
///  @param[in]		amax	Maximum bounds of box A. [(x, y, z)]
///  @param[in]		bmin	Minimum bounds of box B. [(x, y, z)]
///  @param[in]		bmax	Maximum bounds of box B. [(x, y, z)]
/// @return True if the two AABB's overlap.
/// @see dtOverlapQuantBounds
inline bool dtOverlapBounds(const dtReal* amin, const dtReal* amax,
							const dtReal* bmin, const dtReal* bmax)
{
	bool overlap = true;
	overlap = (amin[0] > bmax[0] || amax[0] < bmin[0]) ? false : overlap;
	overlap = (amin[1] > bmax[1] || amax[1] < bmin[1]) ? false : overlap;
	overlap = (amin[2] > bmax[2] || amax[2] < bmin[2]) ? false : overlap;
	return overlap;
}

/// Derives the closest point on a triangle from the specified reference point.
///  @param[out]	closest	The closest point on the triangle.	
///  @param[in]		p		The reference point from which to test. [(x, y, z)]
///  @param[in]		a		Vertex A of triangle ABC. [(x, y, z)]
///  @param[in]		b		Vertex B of triangle ABC. [(x, y, z)]
///  @param[in]		c		Vertex C of triangle ABC. [(x, y, z)]
void dtClosestPtPointTriangle(dtReal* closest, const dtReal* p,
							  const dtReal* a, const dtReal* b, const dtReal* c);

/// Derives the y-axis height of the closest point on the triangle from the specified reference point.
///  @param[in]		p		The reference point from which to test. [(x, y, z)]
///  @param[in]		a		Vertex A of triangle ABC. [(x, y, z)]
///  @param[in]		b		Vertex B of triangle ABC. [(x, y, z)]
///  @param[in]		c		Vertex C of triangle ABC. [(x, y, z)]
///  @param[out]	h		The resulting height.
bool dtClosestHeightPointTriangle(const dtReal* p, const dtReal* a, const dtReal* b, const dtReal* c, dtReal& h);

bool dtIntersectSegmentPoly2D(const dtReal* p0, const dtReal* p1,
							  const dtReal* verts, int nverts,
							  dtReal& tmin, dtReal& tmax,
							  int& segMin, int& segMax);

bool dtIntersectSegSeg2D(const dtReal* ap, const dtReal* aq,
						 const dtReal* bp, const dtReal* bq,
						 dtReal& s, dtReal& t);

/// Determines if the specified point is inside the convex polygon on the xz-plane.
///  @param[in]		pt		The point to check. [(x, y, z)]
///  @param[in]		verts	The polygon vertices. [(x, y, z) * @p nverts]
///  @param[in]		nverts	The number of vertices. [Limit: >= 3]
/// @return True if the point is inside the polygon.
bool dtPointInPolygon(const dtReal* pt, const dtReal* verts, const int nverts);

bool dtDistancePtPolyEdgesSqr(const dtReal* pt, const dtReal* verts, const int nverts,
							dtReal* ed, dtReal* et);

dtReal dtDistancePtSegSqr2D(const dtReal* pt, const dtReal* p, const dtReal* q, dtReal& t);
dtReal dtDistancePtSegSqr(const dtReal* pt, const dtReal* p, const dtReal* q);

/// Derives the centroid of a convex polygon.
///  @param[out]	tc		The centroid of the polgyon. [(x, y, z)]
///  @param[in]		idx		The polygon indices. [(vertIndex) * @p nidx]
///  @param[in]		nidx	The number of indices in the polygon. [Limit: >= 3]
///  @param[in]		verts	The polygon vertices. [(x, y, z) * vertCount]
void dtCalcPolyCenter(dtReal* tc, const unsigned short* idx, int nidx, const dtReal* verts);

/// Determines if the two convex polygons overlap on the xz-plane.
///  @param[in]		polya		Polygon A vertices.	[(x, y, z) * @p npolya]
///  @param[in]		npolya		The number of vertices in polygon A.
///  @param[in]		polyb		Polygon B vertices.	[(x, y, z) * @p npolyb]
///  @param[in]		npolyb		The number of vertices in polygon B.
/// @return True if the two polygons overlap.
bool dtOverlapPolyPoly2D(const dtReal* polya, const int npolya,
						 const dtReal* polyb, const int npolyb);

/// @}
/// @name Miscellanious functions.
/// @{

inline unsigned int dtNextPow2(unsigned int v)
{
	v--;
	v |= v >> 1;
	v |= v >> 2;
	v |= v >> 4;
	v |= v >> 8;
	v |= v >> 16;
	v++;
	return v;
}

inline unsigned int dtIlog2(unsigned int v)
{
	unsigned int r;
	unsigned int shift;
	r = (v > 0xffff) << 4; v >>= r;
	shift = (v > 0xff) << 3; v >>= shift; r |= shift;
	shift = (v > 0xf) << 2; v >>= shift; r |= shift;
	shift = (v > 0x3) << 1; v >>= shift; r |= shift;
	r |= (v >> 1);
	return r;
}

//@UE BEGIN Align to 8 byte boundaries when using double precision
inline int dtAlign(int x) 
{
#if DT_LARGE_WORLD_COORDINATES_DISABLED
	return (x + 3) & ~3; // Align to 4 byte boundary
#else
	return (x + 7) & ~7; // Align to 8 byte boundary
#endif
}
//@UE END

inline int dtOppositeTile(int side) { return (side+4) & 0x7; }

inline void dtSwapByte(unsigned char* a, unsigned char* b)
{
	unsigned char tmp = *a;
	*a = *b;
	*b = tmp;
}

inline void dtSwapEndian(unsigned short* v)
{
	unsigned char* x = (unsigned char*)v;
	dtSwapByte(x+0, x+1);
}

inline void dtSwapEndian(short* v)
{
	unsigned char* x = (unsigned char*)v;
	dtSwapByte(x+0, x+1);
}

inline void dtSwapEndian(unsigned int* v)
{
	unsigned char* x = (unsigned char*)v;
	dtSwapByte(x+0, x+3); dtSwapByte(x+1, x+2);
}

inline void dtSwapEndian(int* v)
{
	unsigned char* x = (unsigned char*)v;
	dtSwapByte(x+0, x+3); dtSwapByte(x+1, x+2);
}

inline void dtSwapEndian(unsigned long long int* v)
{
	unsigned char* x = (unsigned char*)v;
	dtSwapByte(x + 0, x + 7); 
	dtSwapByte(x + 1, x + 6);
	dtSwapByte(x + 2, x + 5);
	dtSwapByte(x + 3, x + 4);
}

inline void dtSwapEndian(long long int* v)
{
	unsigned char* x = (unsigned char*)v;
	dtSwapByte(x + 0, x + 7);
	dtSwapByte(x + 1, x + 6);
	dtSwapByte(x + 2, x + 5);
	dtSwapByte(x + 3, x + 4);
}

inline void dtSwapEndian(float* v)
{
	unsigned char* x = (unsigned char*)v;
	dtSwapByte(x+0, x+3); dtSwapByte(x+1, x+2);
}

// @UE BEGIN Adding support for LWCoords.
inline void dtSwapEndian(double* v)
{
	unsigned char* x = (unsigned char*)v;
	dtSwapByte(x + 0, x + 7); dtSwapByte(x + 1, x + 6); dtSwapByte(x + 2, x + 5); dtSwapByte(x + 3, x + 4);
}
//@UE END Adding support for LWCoords.

void dtRandomPointInConvexPoly(const dtReal* pts, const int npts, dtReal* areas,
							   const dtReal s, const dtReal t, dtReal* out);

// @UE BEGIN
enum dtRotation
{
	DT_ROTATE_0, 
	DT_ROTATE_90, 
	DT_ROTATE_180, 
	DT_ROTATE_270
};

/// Select a 90 degree increment value from an input angle in degree.
///  @param[in]		rotationDeg The desired rotation in degree.
///  @return The rotation enum value.
dtRotation dtSelectRotation(dtReal rotationDeg);

/// Rotate by 90 degree increments.
///  @param[out]	dest	The result position. [(x, y, z)]
///  @param[in]		v		The vector to rotate. [(x, y, z)]
///  @param[in]		rot		The rotation enum value.
void dtVRot90(dtReal* dest, const dtReal* v, const dtRotation rot);

/// Rotate by 90 degree increments.
///  @param[out]	dest	The result position. [(x, y, z)]
///  @param[in]		v		The vector to rotate. [(x, y, z)]
///  @param[in]		rot		The rotation enum value.
void dtVRot90(unsigned short* dest, const unsigned short* v, const dtRotation rot);

/// Rotate vector around center position by increments of 90 degrees.
///  @param[out]	dest	The result position. [(x, y, z)]
///  @param[in]		v		The vector to rotate. [(x, y, z)]
///  @param[in]		center	The center point. [(x, y, z)]
///  @param[in]		rot		The rotation enum value.
void dtRotate90(dtReal* dest, const dtReal* v, const dtReal* center, const dtRotation rot);

/// Rotate vector around center position by increments of 90 degrees.
///  @param[out]	dest	The result position. [(x, y, z)]
///  @param[in]		v		The vector to rotate. [(x, y, z)]
///  @param[in]		center	The center point. [(x, y, z)]
///  @param[in]		rot		The rotation enum value.
void dtRotate90(unsigned short* dest, const unsigned short* v, const unsigned short* center, const dtRotation rot);
// @UE END

/// @}

#endif // DETOURCOMMON_H

///////////////////////////////////////////////////////////////////////////

// This section contains detailed documentation for members that don't have
// a source file. It reduces clutter in the main section of the header.

/**

@fn dtReal dtTriArea2D(const dtReal* a, const dtReal* b, const dtReal* c)
@par

The vertices are projected onto the xz-plane, so the y-values are ignored.

This is a low cost function than can be used for various purposes.  Its main purpose
is for point/line relationship testing.

In all cases: A value of zero indicates that all vertices are collinear or represent the same point.
(On the xz-plane.)

When used for point/line relationship tests, AB usually represents a line against which
the C point is to be tested.  In this case:

A positive value indicates that point C is to the left of line AB, looking from A toward B.<br/>
A negative value indicates that point C is to the right of lineAB, looking from A toward B.

When used for evaluating a triangle:

The absolute value of the return value is two times the area of the triangle when it is
projected onto the xz-plane.

A positive return value indicates:

<ul>
<li>The vertices are wrapped in the normal Detour wrap direction.</li>
<li>The triangle's 3D face normal is in the general up direction.</li>
</ul>

A negative return value indicates:

<ul>
<li>The vertices are reverse wrapped. (Wrapped opposite the normal Detour wrap direction.)</li>
<li>The triangle's 3D face normal is in the general down direction.</li>
</ul>

*/
