// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <limits>

#define DT_LARGE_WORLD_COORDINATES_DISABLED 0

#if DT_LARGE_WORLD_COORDINATES_DISABLED

typedef float dtReal;

#else // DT_LARGE_WORLD_COORDINATES_DISABLED

typedef double dtReal;

#endif // DT_LARGE_WORLD_COORDINATES_DISABLED

inline constexpr dtReal DT_REAL_MAX = std::numeric_limits<dtReal>::max();

