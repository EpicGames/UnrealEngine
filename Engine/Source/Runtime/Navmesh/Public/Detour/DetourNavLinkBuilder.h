// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "DetourAlloc.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavLinkBuilderConfig.h"

//@UE BEGIN

struct dtLinkBuilderConfig
{
	TArray<dtNavLinkBuilderJumpConfig> jumpConfigs;
	
	dtReal agentRadius = 0;
	dtReal agentHeight = 0;
	dtReal agentClimb = 0;
	dtReal cellSize = 0;
	dtReal cellHeight = 0;
};

struct rcHeightfield;
struct rcCompactHeightfield;

struct dtLinkBuilderData
{
	bool generatingLinks = false;
};

class rcContext;
struct rcConfig;

class dtNavLinkBuilder
{
	dtLinkBuilderConfig m_linkBuilderConfig;

	dtReal m_cs = 0;
	dtReal m_csSquared = 0;
	dtReal m_ch = 0;
	dtReal m_invCs = 0;
	const rcHeightfield* m_solid = nullptr;
	const rcCompactHeightfield* m_chf = nullptr;

	struct Edge
	{
		dtReal sp[3], sq[3];
	};
	TArray<Edge, TInlineAllocator<32>> m_edges;
	
public:	
	static constexpr int MAX_SPINE = 8;
	int getEdgeCount() const { return m_edges.Num(); }
	
	struct TrajectorySample
	{
		// Min and max heights relative to the height on the segment between the start and the end point (at the sampling locations). 
		float ymin = 0.f;
		float ymax = 0.f;
		
		bool floorStart = false;
		bool floorEnd = false;
	};
	
	struct Trajectory2D
	{
		float spine[2*MAX_SPINE];		// [x,y] relative points representing the desired trajectory (one spine with MAX_SPINE points)
		float radiusOverflow = 0.f;		// extra distance at the start and end of the spine to lookup for samples

		TArray<TrajectorySample, TInlineAllocator<8>> samples;	// samples along trajectory slices to check for collision
		
		unsigned char nspine = 0;		// @todo: remove (use direclty MAX_SPINE) or make relative to trajectory type and config
	};

private:	
	enum GroundSampleFlag : unsigned char
	{
		UNSET = 0,
		HAS_GROUND = 1,
		UNRESTRICTED = 4,
	};
	
	struct GroundSample
	{
		dtReal height;
		GroundSampleFlag flags;
	};

	struct PotentialSeg
	{
		unsigned char mark;
		int idx;
		float umin, umax;
		float dmin, dmax;
		float sp[3], sq[3];
	};
	
	struct GroundSegment
	{
		GroundSegment() : ngsamples(0) {}
		NAVMESH_API ~GroundSegment();

		dtReal p[3], q[3];
		TArray<GroundSample, TInlineAllocator<16>> gsamples;
		unsigned short ngsamples;
		unsigned short npass;
	};
	
public:
	static constexpr uint8 INVALID_CONFIG_INDEX = 0xFF;
	
	struct EdgeSampler
	{
		Trajectory2D trajectory;
	
		GroundSegment start;
		GroundSegment end;

		dtReal rigp[3], rigq[3];		// edge
		dtReal ax[3], ay[3], az[3];		// axis along edge

		float groundRange;
		uint8 configIndex = INVALID_CONFIG_INDEX;
	};

	enum JumpLinkFlag : unsigned char
	{
		FILTERED = 0,
		VALID = 1,
	};
	
	struct JumpLink
	{
		dtReal spine0[MAX_SPINE*3];
		dtReal spine1[MAX_SPINE*3];
		int nspine = 0;
		JumpLinkFlag flags = VALID;
		uint8 configIndex = INVALID_CONFIG_INDEX;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)		
		short debugSourceEdge = -1;
#endif
	};
	TArray<JumpLink, TInlineAllocator<16>> m_links;

private:
	int m_debugSelectedEdge = -1;

	friend NAVMESH_API void duDebugDrawNavLinkBuilder(struct duDebugDraw* dd, const dtNavLinkBuilder& linkBuilder, unsigned int drawFlags, const EdgeSampler* es);
	
	friend NAVMESH_API void duDebugDrawTrajectorySamples(struct duDebugDraw* dd, const dtNavLinkBuilder& linkBuilder, const dtReal* pa, const dtReal* pb,
														 const dtNavLinkBuilder::Trajectory2D* trajectory, const dtReal* trajectoryDir);

	friend NAVMESH_API void duDrawGroundSegment(struct duDebugDraw* dd, const dtNavLinkBuilder::GroundSegment& segment);
	
public:
	// Loops through contours to store edge points in world coordinates.
	NAVMESH_API bool findEdges(rcContext& ctx, const rcConfig& cfg, const dtLinkBuilderConfig& builderConfig,
							   const struct dtTileCacheContourSet& lcset, const dtReal* orig,
							   const rcHeightfield* solidHF, const rcCompactHeightfield* compactHF);

	// For all edges, sample edges (sampleEdge) and add links to m_links
	NAVMESH_API void buildForAllEdges(rcContext& ctx, const dtLinkBuilderConfig& acfg, const dtNavLinkBuilderJumpConfig& jumpConfig);

	NAVMESH_API void debugBuildEdge(const dtLinkBuilderConfig& acfg, int edgeIndex, EdgeSampler& sampler, const dtNavLinkBuilderJumpConfig& jumpConfig);
	
private:
	void initTrajectorySamples(const float groundRange, Trajectory2D* trajectory) const;
	bool isTrajectoryClear(const dtReal* pa, const dtReal* pb, const Trajectory2D* trajectory, const dtReal* trajectoryDir) const;
	
	void initJumpRig(EdgeSampler* es, const dtReal* sp, const dtReal* sq,
						 const dtNavLinkBuilderJumpConfig& config) const;

	bool getCompactHeightfieldHeight(const dtReal* pt, const dtReal hrange, dtReal* height) const;
	bool checkHeightfieldCollision(const dtReal x, const dtReal ymin, const dtReal ymax, const dtReal z) const;

	void sampleGroundSegment(GroundSegment* seg, const int nsamples, const float groundRange) const;

	// Update the min height of the trajectory samples from the height of the ground at the start and the end of the trajectories.
	// This method must be called after the segments have been sampled for ground. 
	void updateTrajectorySamples(EdgeSampler* es) const;
	
	void sampleAction(EdgeSampler* es) const;
	
	void filterOverlappingLinks(const float edgeDistanceThreshold);

	bool sampleEdge(const dtNavLinkBuilderJumpConfig& jumpConfig, const dtReal* sp, const dtReal* sq, dtNavLinkBuilder::EdgeSampler* sampler) const;
	void addEdgeLinks(const dtLinkBuilderConfig& builderConfig, const EdgeSampler* es, const int edgeIndex);
};
//@UE END
