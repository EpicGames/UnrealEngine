// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

enum dtNavLinkBuilderFlags : unsigned short
{
	DT_NAVLINK_CREATE_CENTER_POINT_LINK = 1 << 0,
	DT_NAVLINK_CREATE_EXTREMITY_LINKS	= 1 << 1,
};

/** Configuration for generated jump links. */
struct dtNavLinkBuilderJumpConfig
{
	/// Should this config be used to generate links.
	bool enabled = true;

	/// Horizontal length of the jump. How far from the starting point we will look for ground. [Limit: > 0] [Units: wu]
	float jumpLength = 150.f;

	/// How far from the edge is the jump started. [Limit: > 0] [Units: wu]
	float jumpDistanceFromEdge = 10.f;

	/// How far below the starting height we want to look for landing ground. [Limit: > 0] [Units: wu]
	float jumpMaxDepth = 150.f;

	/// Peak height relative to the height of the starting point. [Limit: >= 0] [Units: wu]
	float jumpHeight = 50.f;

	/// Tolerance at both ends of the jump to find ground. [Limit: > 0] [Units: wu]
	float jumpEndsHeightTolerance = 50.f;

	/// Value multiplied by CellSize to find the distance between sampling trajectories. Default is 1.
	/// Larger values improve generation speed but might introduce sampling errors. [Limit: >= 1]
	float samplingSeparationFactor = 1.f;

	/// When filtering similar links, distance used to compare between segment endpoints to match similar links.
	/// Use greater distance for more filtering (0 to deactivate filtering). [Limit: > 0] [Units: wu]
	float filterDistanceThreshold = 80.f;

	/// Flags used to indicate how links will be added.
	unsigned short linkBuilderFlags = DT_NAVLINK_CREATE_CENTER_POINT_LINK;

	/// Cached parabola constant fitting the configuration parameters. 
	float cachedParabolaConstant = 0;

	/// Cached value used when computing jump trajectory.
	float cachedDownRatio = 0;
	
	/// User id used to handle links made from this configuration.
	unsigned long long linkUserId = 0;

	/// User defined flags assigned to downwards traversal of the off-mesh connections
	unsigned short downDirPolyFlag = 0;

	/// User defined flags assigned to upwards traversal of the off-mesh connections
	unsigned short upDirPolyFlag = 0;

	/// User defined area id assigned to downards traversal of the off-mesh connections
	unsigned char downDirArea = 0;

	/// User defined area id assigned to upwards traversal of the off-mesh connections
	unsigned char upDirArea = 0;

	/// Unique index in TArray<dtNavLinkBuilderJumpConfig> identifying this config
	static constexpr uint8 INVALID_CONFIG_INDEX = 0xFF;
	uint8 ConfigIndex = INVALID_CONFIG_INDEX;

	/// Initialize the configuration by computing cached values.
	NAVMESH_API void init();
};


/** Configuration for generated jump down links. */
struct UE_DEPRECATED(5.7, "Use dtNavLinkBuilderJumpConfig instead.") dtNavLinkBuilderJumpDownConfig
{
	/// Should this config be used to generate links.
	bool enabled = true;

	/// Horizontal length of the jump. How far from the starting point we will look for ground. [Limit: > 0] [Units: wu]
	float jumpLength = 150.f;

	/// How far from the edge is the jump started. [Limit: > 0] [Units: wu]
	float jumpDistanceFromEdge = 10.f;

	/// How far below the starting height we want to look for landing ground. [Limit: > 0] [Units: wu]
	float jumpMaxDepth = 150.f;

	/// Peak height relative to the height of the starting point. [Limit: >= 0] [Units: wu]
	float jumpHeight = 50.f;

	/// Tolerance at both ends of the jump to find ground. [Limit: > 0] [Units: wu]
	float jumpEndsHeightTolerance = 50.f;

	/// Value multiplied by CellSize to find the distance between sampling trajectories. Default is 1.
	/// Larger values improve generation speed but might introduce sampling errors. [Limit: >= 1]
	float samplingSeparationFactor = 1.f;

	/// When filtering similar links, distance used to compare between segment endpoints to match similar links.
	/// Use greater distance for more filtering (0 to deactivate filtering). [Limit: > 0] [Units: wu]
	float filterDistanceThreshold = 80.f;

	/// Flags used to indicate how links will be added.
	unsigned short linkBuilderFlags = DT_NAVLINK_CREATE_CENTER_POINT_LINK;

	/// Cached parabola constant fitting the configuration parameters. 
	float cachedParabolaConstant = 0;

	/// Cached value used when computing jump trajectory.
	float cachedDownRatio = 0;
	
	/// User id used to handle links made from this configuration.
	unsigned long long linkUserId = 0;

	/// User defined flags assigned to downwards traversal of the off-mesh connections
	unsigned short downDirPolyFlag = 0;

	/// User defined flags assigned to upwards traversal of the off-mesh connections
	unsigned short upDirPolyFlag = 0;

	/// User defined area id assigned to downards traversal of the off-mesh connections
	unsigned char downDirArea = 0;

	/// User defined area id assigned to upwards traversal of the off-mesh connections
	unsigned char upDirArea = 0;

	/// Initialize the configuration by computing cached values.
	NAVMESH_API void init();
};