// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURNAVMESH_H
#define DETOURNAVMESH_H

#include "CoreMinimal.h"
#include "Detour/DetourAlloc.h"
#include "Detour/DetourStatus.h"
#include "DetourLargeWorldCoordinates.h"
#include "Detour/DetourAssert.h"
#include "Logging/LogMacros.h"

NAVMESH_API DECLARE_LOG_CATEGORY_EXTERN(LogDetour, Log, All);

#if WITH_NAVMESH_SEGMENT_LINKS
NAVMESH_API DECLARE_LOG_CATEGORY_EXTERN(LogSegmentLink, Warning, All);
#endif

// Note: If you want to use 64-bit refs, change the types of both dtPolyRef & dtTileRef.
// It is also recommended that you change dtHashRef() to a proper 64-bit hash.

#ifndef USE_64BIT_ADDRESS
#define USE_64BIT_ADDRESS 1
#endif

#if USE_64BIT_ADDRESS

#if defined(__LP64__)
// LP64 (Linux/OS X): UE will define its uint64 type as "unsigned long long" so we need to match this
typedef unsigned long long UEType_uint64;
#else
#include <stdint.h>
typedef uint64_t UEType_uint64;
#endif

/// A handle to a polygon within a navigation mesh tile.
/// @ingroup detour
typedef UEType_uint64 dtPolyRef;

/// A handle to a tile within a navigation mesh.
/// @ingroup detour
typedef UEType_uint64 dtTileRef;

/// A handle to a cluster within a navigation mesh tile.
typedef UEType_uint64 dtClusterRef;

#else

/// A handle to a polygon within a navigation mesh tile.
/// @ingroup detour
typedef unsigned int dtPolyRef;

/// A handle to a tile within a navigation mesh.
/// @ingroup detour
typedef unsigned int dtTileRef;

/// A handle to a cluster within a navigation mesh tile.
typedef unsigned int dtClusterRef;

#endif // USE_64BIT_ADDRESS

/// The maximum number of vertices per navigation polygon.
/// @ingroup detour
static const int DT_VERTS_PER_POLYGON = 6;

/// @{
/// @name Tile Serialization Constants
/// These constants are used to detect whether a navigation tile's data
/// and state format is compatible with the current build.
///

/// A magic number used to detect compatibility of navigation tile data.
/// UE magic removed to save memory
//static const int DT_NAVMESH_MAGIC = 'D'<<24 | 'N'<<16 | 'A'<<8 | 'V';

/// A version number used to detect compatibility of navigation tile data.
static const int DT_NAVMESH_VERSION = 8;

/// A magic number used to detect the compatibility of navigation tile states.
static const int DT_NAVMESH_STATE_MAGIC = 'D'<<24 | 'N'<<16 | 'M'<<8 | 'S';

/// A version number used to detect compatibility of navigation tile states.
static const int DT_NAVMESH_STATE_VERSION = 1;

/// @}

/// A flag that indicates that an entity links to an external entity.
/// (E.g. A polygon edge is a portal that links to another polygon.)
static const unsigned short DT_EXT_LINK = 0x8000;

/// A value that indicates the entity does not link to anything.
inline const unsigned int DT_NULL_LINK = 0xffffffff;

/// A flag that indicates that an off-mesh connection can be traversed in both directions. (Is bidirectional.)
static const unsigned char DT_OFFMESH_CON_BIDIR		= 0x01;
static const unsigned char DT_OFFMESH_CON_POINT		= 0x02;		//UE
static const unsigned char DT_OFFMESH_CON_SEGMENT	= 0x04;		//UE
static const unsigned char DT_OFFMESH_CON_CHEAPAREA = 0x08;		//UE
static const unsigned char DT_OFFMESH_CON_GENERATED = 0x10;		//UE
/// A flag that indicates whether the direction of travel along the one-way off-mesh connection is actually reversed.
static const unsigned char DT_OFFMESH_CON_REVERSED  = 0x20;		//UE

/// The maximum number of user defined area ids.
/// @ingroup detour
static const int DT_MAX_AREAS = 64;

//@UE BEGIN
/// Navmesh tiles' salt will have at least this much bits
static const int DT_MIN_SALT_BITS = 5;
static const int DT_SALT_BASE = 1;

static const int DT_RESOLUTION_COUNT = 3; 

#if WITH_NAVMESH_SEGMENT_LINKS
/// Max segment parts for segment-to-segment off mesh connection
static const int DT_MAX_OFFMESH_SEGMENT_PARTS = 4;
static const int DT_INVALID_SEGMENT_PART = 0xffff;
#endif // WITH_NAVMESH_SEGMENT_LINKS

/// flags use to annotate dtLink::side with addotional data
static const unsigned char DT_CONNECTION_INTERNAL = (1 << 7);
static const unsigned char DT_LINK_FLAG_OFFMESH_CON = (1 << 6);
static const unsigned char DT_LINK_FLAG_OFFMESH_CON_BIDIR = (1 << 5);
static const unsigned char DT_LINK_FLAG_OFFMESH_CON_BACKTRACKER = (1 << 4);
static const unsigned char DT_LINK_FLAG_OFFMESH_CON_ENABLED = (1 << 3);
static const unsigned char DT_LINK_FLAG_SIDE_MASK = 7;

#if WITH_NAVMESH_CLUSTER_LINKS
/// flags used to annotate dtClusterLink::flags with additional data
static const unsigned char DT_CLINK_VALID_FWD = 0x01;
static const unsigned char DT_CLINK_VALID_BCK = 0x02;

/// Index of first cluster link within tile
static const unsigned int DT_CLINK_FIRST = 0x80000000;
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

//@UE BEGIN Adding support for memory tracking.
struct dtMeshTile;

typedef void (dtStatsPostAddTileFunc)(const dtMeshTile& TileAdd);

typedef void (dtStatsPreRemoveTileFunc)(const dtMeshTile& tileRemove);

NAVMESH_API void dtStatsSetCustom(dtStatsPostAddTileFunc* addFunc, dtStatsPreRemoveTileFunc* removeFunc);
//@UE END Adding support for memory tracking.

/// Tile flags used for various functions and fields.
/// For an example, see dtNavMesh::addTile().
enum dtTileFlags
{	
	DT_TILE_FREE_DATA = 0x01,					///< The navigation mesh owns the tile memory and is responsible for freeing it.
};

/// Vertex flags returned by dtNavMeshQuery::findStraightPath.
enum dtStraightPathFlags
{
	DT_STRAIGHTPATH_START = 0x01,				///< The vertex is the start position in the path.
	DT_STRAIGHTPATH_END = 0x02,					///< The vertex is the end position in the path.
	DT_STRAIGHTPATH_OFFMESH_CONNECTION = 0x04,	///< The vertex is the start of an off-mesh connection.
};

/// Options for dtNavMeshQuery::findStraightPath.
enum dtStraightPathOptions
{
	DT_STRAIGHTPATH_AREA_CROSSINGS = 0x01,	///< Add a vertex at every polygon edge crossing where area changes.
	DT_STRAIGHTPATH_ALL_CROSSINGS = 0x02,	///< Add a vertex at every polygon edge crossing.
};

/// Flags representing the type of a navigation mesh polygon.
enum dtPolyTypes
{
	/// The polygon is a standard convex polygon that is part of the surface of the mesh.
	DT_POLYTYPE_GROUND = 0,
	/// The polygon is an off-mesh connection consisting of two vertices.
	DT_POLYTYPE_OFFMESH_POINT = 1,

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	/// The polygon is an off-mesh connection consisting of four vertices.
	DT_POLYTYPE_OFFMESH_SEGMENT = 2,
#endif // WITH_NAVMESH_SEGMENT_LINKS
	//@UE END
};

//@UE BEGIN
/// Tile-local vertex: stores a vertex as offsets from the tile's bmin.
/// Use toWorld() to reconstruct the world-space position, and setFromWorld() to write back.
struct dtTileVert
{
	float x, y, z;

	/// Encode from world position
	inline void setFromWorld(const dtReal* bmin, const dtReal* worldPos)
	{
		const dtReal dx = worldPos[0] - bmin[0];
		const dtReal dy = worldPos[1] - bmin[1];
		const dtReal dz = worldPos[2] - bmin[2];

		// Verify that tile-local offsets are reasonable for float storage.

		// Maximum allowed tile-local vertex offset before precision becomes problematic.
		// Typical tiles are <10,000 units. This threshold (100x that) catches bugs like:
		// - Vertex assigned to wrong tile
		// - Wrong bmin passed to setFromWorld()
		// - Coordinate system errors
		// Values this large indicate real bugs, not just edge cases.
		constexpr dtReal DT_TILE_VERT_MAX_OFFSET = 1000000.0;
		dtAssert((dx < 0 ? -dx : dx) < DT_TILE_VERT_MAX_OFFSET && (dy < 0 ? -dy : dy) < DT_TILE_VERT_MAX_OFFSET && (dz < 0 ? -dz : dz) < DT_TILE_VERT_MAX_OFFSET);

		x = (float)dx;
		y = (float)dy;
		z = (float)dz;
	}
	
	/// Decode to world position
	inline void toWorld(const dtReal* bmin, dtReal* out) const
	{
		out[0] = (dtReal)x + bmin[0];
		out[1] = (dtReal)y + bmin[1];
		out[2] = (dtReal)z + bmin[2];
	}
};
//@UE END

/// Defines a polygon within a dtMeshTile object.
/// @ingroup detour
struct dtPoly
{
	/// Index to first link in linked list. (Or #DT_NULL_LINK if there is no link.)
	unsigned int firstLink;

	/// The indices of the polygon's vertices.
	/// The actual vertices are located in dtMeshTile::verts.
	unsigned short verts[DT_VERTS_PER_POLYGON];

	/// Packed data representing neighbor polygons references and flags for each edge.
	unsigned short neis[DT_VERTS_PER_POLYGON];

	/// The user defined polygon flags.
	unsigned short flags;

	/// The number of vertices in the polygon.
	unsigned char vertCount;

	/// The bit packed area id and polygon type.
	/// @note Use the structure's set and get methods to acess this value.
	unsigned char areaAndtype;

	/// Sets the user defined area id. [Limit: < #DT_MAX_AREAS]
	inline void setArea(unsigned char a) { areaAndtype = static_cast<unsigned char>((areaAndtype & 0xc0) | (a & 0x3f)); }

	/// Sets the polygon type. (See: #dtPolyTypes.)
	inline void setType(unsigned char t) { areaAndtype = static_cast<unsigned char>((areaAndtype & 0x3f) | (t << 6)); }

	/// Gets the user defined area id.
	inline unsigned char getArea() const { return areaAndtype & 0x3f; }

	/// Gets the polygon type. (See: #dtPolyTypes)
	inline unsigned char getType() const { return areaAndtype >> 6; }
};

/// Defines the location of detail sub-mesh data within a dtMeshTile.
struct dtPolyDetail
{
	unsigned short vertBase;		///< The offset of the vertices in the dtMeshTile::detailVerts array.
	unsigned short triBase;			///< The offset of the triangles in the dtMeshTile::detailTris array.
	unsigned char vertCount;		///< The number of vertices in the sub-mesh.
	unsigned char triCount;			///< The number of triangles in the sub-mesh.
};

/// Defines a link between polygons.
/// @note This structure is rarely if ever used by the end user.
/// @see dtMeshTile
struct dtLink
{
	dtPolyRef ref;					///< Neighbour reference. (The neighbor that is linked to.)
	unsigned int next;				///< Index of the next link.
	unsigned char edge;				///< Index of the polygon edge that owns this link.
	unsigned char side;				///< If a boundary link, defines on which side the link is.
	unsigned char bmin;				///< If a boundary link, defines the minimum sub-edge area.
	unsigned char bmax;				///< If a boundary link, defines the maximum sub-edge area.
};

/// Bounding volume node.
/// @note This structure is rarely if ever used by the end user.
/// @see dtMeshTile
struct dtBVNode
{
	unsigned short bmin[3];			///< Minimum bounds of the node's AABB. [(x, y, z)]
	unsigned short bmax[3];			///< Maximum bounds of the node's AABB. [(x, y, z)]
	int i;							///< The node's index. (Negative for escape sequence.)
};

//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
/// This is an experimental feature and has not been finished to production quality.
struct dtOffMeshSegmentConnection
{
	dtReal startA[3];				///< Start point of segment A
	dtReal endA[3];					///< End point of segment A
	dtReal startB[3];				///< Start point of segment B
	dtReal endB[3];					///< End point of segment B

	/// The radius of the endpoints. [Limit: >= 0]
	dtReal rad;

	/// The id of the offmesh connection. (User assigned when the navigation mesh is built.)
	unsigned int userId;	
	
	/// First poly in segment pool (+ header->offMeshSegPolyBase)
	unsigned short firstPoly;

	/// Number of created polys
	unsigned char npolys;

	/// Link flags. 
	unsigned char flags;

	/// Sets link flags
	inline void setFlags(unsigned char conFlags)
	{
		flags = ((conFlags & DT_OFFMESH_CON_BIDIR) ? 0x80 : 0) ;
	}

	/// Gets the link direction
	inline bool getBiDirectional() const { return (flags & 0x80) != 0; }
};
#endif // WITH_NAVMESH_SEGMENT_LINKS
//@UE END

/// Defines an navigation mesh off-mesh connection within a dtMeshTile object.
/// An off-mesh connection is a user defined traversable connection made up to two vertices.
struct dtOffMeshConnection
{
	/// The endpoints of the connection. [(ax, ay, az, bx, by, bz)]
	dtReal pos[6];

	/// The radius of the endpoints. [Limit: >= 0]
	dtReal rad;

	//@UE BEGIN
	/// The snap height of endpoints (less than 0 = use step height)
	dtReal height;
	//@UE END

	/// The id of the offmesh connection. (User assigned when the navigation mesh is built.)
	unsigned long long int userId;

	/// The polygon reference of the connection within the tile.
	unsigned short poly;

	/// End point side.
	unsigned char side;

	/// Link flags. 
	unsigned char flags;

	//@UE BEGIN
	/// Sets link flags
	inline void setFlags(unsigned char conTypeFlags)
	{
		flags = ((conTypeFlags & DT_OFFMESH_CON_BIDIR) ? 0x80 : 0) |
				((conTypeFlags & DT_OFFMESH_CON_CHEAPAREA) ? 0x40 : 0) |
				((conTypeFlags & DT_OFFMESH_CON_GENERATED) ? 0x20 : 0) | 
				((conTypeFlags & DT_OFFMESH_CON_REVERSED) ? 0x10 : 0);
	}

	/// Gets the link direction
	inline bool getBiDirectional() const { return (flags & 0x80) != 0; }

	/// Gets the link snap mode
	inline bool getSnapToCheapestArea() const { return (flags & 0x40) != 0; }

	/// Indicates if the link was automatically generated
	inline bool getIsGenerated() const { return (flags & 0x20) != 0; }
	
	/// Indicates if the link is reversed
	inline bool getIsReversed() const { return (flags & 0x10) != 0; }

	//@UE END
};

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
/// Cluster of polys
struct dtCluster
{
	dtReal center[3];				///< Center pos of cluster
	unsigned int firstLink;			///< Link in dtMeshTile.links array
	unsigned int numLinks;			///< Number of cluster links
};

/// Links between clusters
struct dtClusterLink
{
	dtClusterRef ref;				///< Destination tile and cluster
	unsigned int next;				///< Next link in dtMeshTile.links array
	unsigned char flags;			///< Link traversing data
};
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

/// Provides high level information related to a dtMeshTile object.
/// @ingroup detour
struct dtMeshHeader
{
	unsigned short version;	///< Tile data format version number.
	unsigned short layer;			///< The layer of the tile within the dtNavMesh tile grid. (x, y, layer)

	unsigned short polyCount;		///< The number of polygons in the tile.
	unsigned short vertCount;		///< The number of vertices in the tile.

	int x;					///< The x-position of the tile within the dtNavMesh tile grid. (x, y, layer)
	int y;					///< The y-position of the tile within the dtNavMesh tile grid. (x, y, layer)
			
	unsigned short maxLinkCount;		///< The number of allocated links.
	unsigned short detailMeshCount;		///< The number of sub-meshes in the detail mesh.
	
	/// The number of unique vertices in the detail mesh. (In addition to the polygon vertices.)
	unsigned short detailVertCount;
	
	unsigned short detailTriCount;		///< The number of triangles in the detail mesh.
	unsigned short bvNodeCount;			///< The number of bounding volume nodes. (Zero if bounding volumes are disabled.)
	unsigned short offMeshConCount;		///< The number of point type off-mesh connections.
	unsigned short offMeshBase;			///< The index of the first polygon which is a point type off-mesh connection.

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	unsigned short offMeshSegConCount;	///< The number of segment type off-mesh connections.
	unsigned short offMeshSegPolyBase;	///< The index of the first polygon which is an segment type off-mesh connection
	unsigned short offMeshSegVertBase;	///< The index of the first vertex used by segment type off-mesh connection
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	unsigned short clusterCount;			///< Number of clusters
#endif // WITH_NAVMESH_CLUSTER_LINKS

	unsigned char resolution;			///< The resolution index used for the tile.
	//@UE END

	// These should be at the bottom, as they are less often used than the rest of the data. The rest will fit in one cache line.
	dtReal bmin[3];				///< The minimum bounds of the tile's AABB. [(x, y, z)]
	dtReal bmax[3];				///< The maximum bounds of the tile's AABB. [(x, y, z)]
};

/// Defines a navigation mesh tile.
/// @ingroup detour
struct dtMeshTile
{
	unsigned int salt;					///< Counter describing modifications to the tile.

	unsigned int linksFreeList;			///< Index to the next free link.
	dtMeshHeader* header;				///< The tile header.
	dtPoly* polys;						///< The tile polygons. [Size: dtMeshHeader::polyCount]
	dtTileVert* verts;					///< The tile vertices. Local offsets from dtMeshHeader::bmin. [Size: dtMeshHeader::vertCount] //@UE
	dtLink* links;						///< The tile links. [Size: dtMeshHeader::maxLinkCount]
	dtPolyDetail* detailMeshes;			///< The tile's detail sub-meshes. [Size: dtMeshHeader::detailMeshCount]

	/// The detail mesh's unique vertices. [Size: dtMeshHeader::detailVertCount] Local offsets from dtMeshHeader::bmin. //@UE
	dtTileVert* detailVerts; //@UE

	/// The detail mesh's triangles. [(vertA, vertB, vertC) * dtMeshHeader::detailTriCount]
	unsigned char* detailTris;	

	/// The tile bounding volume nodes. [Size: dtMeshHeader::bvNodeCount]
	/// (Will be null if bounding volumes are disabled.)
	dtBVNode* bvTree;

	dtOffMeshConnection* offMeshCons;		///< The tile off-mesh connections. [Size: dtMeshHeader::offMeshConCount]

	unsigned char* data;					///< The tile data. (Not directly accessed under normal situations.)
	int dataSize;							///< Size of the tile data.
	int flags;								///< Tile flags. (See: #dtTileFlags)
	dtMeshTile* next;						///< The next free tile, or the next tile in the spatial grid.

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	dtOffMeshSegmentConnection* offMeshSeg;		///< The tile off-mesh connections. [Size: dtMeshHeader::offMeshSegConCount]
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	dtCluster* clusters;						///< Cluster data
	unsigned short* polyClusters;				///< Cluster Id for each ground type polygon [Size: dtMeshHeader::polyCount]

	dtChunkArray<dtClusterLink, DT_ALLOC_PERM_TILE_DYNLINK_CLUSTER> dynamicLinksC; ///< Dynamic links array (indices starting from DT_CLINK_FIRST)
#endif // WITH_NAVMESH_CLUSTER_LINKS

	dtChunkArray<dtLink, DT_ALLOC_PERM_TILE_DYNLINK> dynamicLinks; ///< Dynamic links array (indices starting from dtMeshHeader::maxLinkCount)	//@UE
	unsigned int dynamicFreeList;				///< Index of the next free dynamic link //@UE

#if WITH_NAVMESH_CLUSTER_LINKS
	unsigned int dynamicFreeListC;				///< Index of the next free dynamic link
#endif // WITH_NAVMESH_CLUSTER_LINKS

	//@UE END
};

//@UE BEGIN
/// Configuration parameters depending on navmesh resolution.
struct dtNavMeshResParams
{
	dtReal bvQuantFactor;			///< The bounding volume quantization factor.
};
//@UE END

/// Configuration parameters used to define multi-tile navigation meshes.
/// The values are used to allocate space during the initialization of a navigation mesh.
/// @see dtNavMesh::init()
/// @ingroup detour
struct dtNavMeshParams
{
	//@UE BEGIN
	dtReal walkableHeight;			///< The height of the agents using the tile.
	dtReal walkableRadius;			///< The radius of the agents using the tile.
	dtReal walkableClimb;			///< The maximum climb height of the agents using the tile.	

	dtNavMeshResParams resolutionParams[DT_RESOLUTION_COUNT]; ///< Parameters depending on resolutions.
	//@UE END
	
	dtReal orig[3];					///< The world space origin of the navigation mesh's tile space. [(x, y, z)]
	dtReal tileWidth;				///< The width of each tile. (Along the x-axis.)
	dtReal tileHeight;				///< The height of each tile. (Along the z-axis.)
	int maxTiles;					///< The maximum number of tiles the navigation mesh can contain.
	int maxPolys;					///< The maximum number of polygons each tile can contain.
};

/// A navigation mesh based on tiles of convex polygons.
/// @ingroup detour
class dtNavMesh
{
public:
	NAVMESH_API dtNavMesh();
	NAVMESH_API ~dtNavMesh();

	/// @{
	/// @name Initialization and Tile Management

	/// Initializes the navigation mesh for tiled use.
	///  @param[in]	params		Initialization parameters.
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus init(const dtNavMeshParams* params);

	/// Initializes the navigation mesh for single tile use.
	///  @param[in]	data		Data of the new tile. (See: #dtCreateNavMeshData)
	///  @param[in]	dataSize	The data size of the new tile.
	///  @param[in]	flags		The tile flags. (See: #dtTileFlags)
	/// @return The status flags for the operation.
	///  @see dtCreateNavMeshData
	NAVMESH_API dtStatus init(unsigned char* data, const int dataSize, const int flags);
	
	/// The navigation mesh initialization params.
	NAVMESH_API const dtNavMeshParams* getParams() const;

	/// Adds a tile to the navigation mesh.
	///  @param[in]		data		Data for the new tile mesh. (See: #dtCreateNavMeshData)
	///  @param[in]		dataSize	Data size of the new tile mesh.
	///  @param[in]		flags		Tile flags. (See: #dtTileFlags)
	///  @param[in]		lastRef		The desired reference for the tile. (When reloading a tile.) [opt] [Default: 0]
	///  @param[out]	result		The tile reference. (If the tile was succesfully added.) [opt]
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus addTile(unsigned char* data, int dataSize, int flags, dtTileRef lastRef, dtTileRef* result);

#if WITH_NAVMESH_SEGMENT_LINKS
	/// Create segment link connections within a tile.
	///  @param[in]		tileRef						The tileRef of the tile owning the segment link connections.
	///  @param[in]		maxSkippedNeighborTiles		The max number of neighbor indices that can be held in skippedNeighborTiles.
	///  @param[out]	skippedNeighborTiles		Array containing the indices of the neighbor tiles processed.
	///  @param[out]	numSkippedNeighborTiles		The number of neighbor tiles skipped during link connection.
	NAVMESH_API void processSegmentLinksForTile(dtTileRef tileRef, unsigned int maxSkippedNeighborTiles, dtTileRef* skippedNeighborTiles, unsigned int& numSkippedNeighborTiles);
#endif // WITH_NAVMESH_SEGMENT_LINKS
	
	/// Removes the specified tile from the navigation mesh.
	///  @param[in]		ref			The reference of the tile to remove.
	///  @param[out]	data		Data associated with deleted tile.
	///  @param[out]	dataSize	Size of the data associated with deleted tile.
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus removeTile(dtTileRef ref, unsigned char** data, int* dataSize);

	/// @}

	/// @{
	/// @name Query Functions

	/// Calculates the tile grid location for the specified world position.
	///  @param[in]	pos  The world position for the query. [(x, y, z)]
	///  @param[out]	tx		The tile's x-location. (x, y)
	///  @param[out]	ty		The tile's y-location. (x, y)
	NAVMESH_API void calcTileLoc(const dtReal* pos, int* tx, int* ty) const;


	/// Calculates whether the tile grid location for the specified world position
	/// can fit in the tile indices type (currently an int)
	///  @param[in]	pos			The world position for the query. [(x, y, z)]
	NAVMESH_API bool isTileLocInValidRange(const dtReal* pos) const;

	/// Gets the tile at the specified grid location.
	///  @param[in]	x		The tile's x-location. (x, y, layer)
	///  @param[in]	y		The tile's y-location. (x, y, layer)
	///  @param[in]	layer	The tile's layer. (x, y, layer)
	/// @return The tile, or null if the tile does not exist.
	NAVMESH_API const dtMeshTile* getTileAt(const int x, const int y, const int layer) const;

// @UE BEGIN
	/// Gets number of tiles at the specified grid location. (All layers.)
	///  @param[in]		x			The tile's x-location. (x, y)
	///  @param[in]		y			The tile's y-location. (x, y)
	/// @return The number of tiles in grid.
	NAVMESH_API int getTileCountAt(const int x, const int y) const;
// @UE END

	/// Gets all tiles at the specified grid location. (All layers.)
	///  @param[in]		x			The tile's x-location. (x, y)
	///  @param[in]		y			The tile's y-location. (x, y)
	///  @param[out]	tiles		A pointer to an array of tiles that will hold the result.
	///  @param[in]		maxTiles	The maximum tiles the tiles parameter can hold.
	/// @return The number of tiles returned in the tiles array.
	NAVMESH_API int getTilesAt(const int x, const int y,
				   dtMeshTile const** tiles, const int maxTiles) const;
	
	/// Gets the tile reference for the tile at specified grid location.
	///  @param[in]	x		The tile's x-location. (x, y, layer)
	///  @param[in]	y		The tile's y-location. (x, y, layer)
	///  @param[in]	layer	The tile's layer. (x, y, layer)
	/// @return The tile reference of the tile, or 0 if there is none.
	NAVMESH_API dtTileRef getTileRefAt(int x, int y, int layer) const;

	/// Gets the tile reference for the specified tile.
	///  @param[in]	tile	The tile.
	/// @return The tile reference of the tile.
	NAVMESH_API dtTileRef getTileRef(const dtMeshTile* tile) const;

	/// Gets the tile for the specified tile reference.
	///  @param[in]	ref		The tile reference of the tile to retrieve.
	/// @return The tile for the specified reference, or null if the 
	///		reference is invalid.
	NAVMESH_API const dtMeshTile* getTileByRef(dtTileRef ref) const;

	/// Gets the tile for the specified tile reference.
	///  @param[in]	ref		The tile reference of the tile to retrieve.
	/// @return The tile for the specified reference, or null if the 
	///		reference is invalid.
	NAVMESH_API dtMeshTile* getMutableTileByRef(dtTileRef ref) const;
	
	/// The maximum number of tiles supported by the navigation mesh.
	/// @return The maximum number of tiles supported by the navigation mesh.
	NAVMESH_API int getMaxTiles() const;
	
	/// Gets the tile at the specified index.
	///  @param[in]	i		The tile index. [Limit: 0 >= index < #getMaxTiles()]
	/// @return The tile at the specified index.
	NAVMESH_API const dtMeshTile* getTile(int i) const;

	/// Gets the tile and polygon for the specified polygon reference.
	///  @param[in]		ref		The reference for the a polygon.
	///  @param[out]	tile	The tile containing the polygon.
	///  @param[out]	poly	The polygon.
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus getTileAndPolyByRef(const dtPolyRef ref, const dtMeshTile** tile, const dtPoly** poly) const;
	
	/// Returns the tile and polygon for the specified polygon reference.
	///  @param[in]		ref		A known valid reference for a polygon.
	///  @param[out]	tile	The tile containing the polygon.
	///  @param[out]	poly	The polygon.
	NAVMESH_API void getTileAndPolyByRefUnsafe(const dtPolyRef ref, const dtMeshTile** tile, const dtPoly** poly) const;

	/// Checks the validity of a polygon reference.
	///  @param[in]	ref		The polygon reference to check.
	/// @return True if polygon reference is valid for the navigation mesh.
	NAVMESH_API bool isValidPolyRef(dtPolyRef ref) const;
	
	/// Gets the polygon reference for the tile's base polygon.
	///  @param[in]	tile		The tile.
	/// @return The polygon reference for the base polygon in the specified tile.
	NAVMESH_API dtPolyRef getPolyRefBase(const dtMeshTile* tile) const;
	
	/// Gets the cluster reference for the tile's base cluster.
	///  @param[in]	tile		The tile.
	/// @return The cluster reference for the base cluster in the specified tile.
	NAVMESH_API dtClusterRef getClusterRefBase(const dtMeshTile* tile) const;

	/// Gets the endpoints for an off-mesh connection, ordered by "direction of travel".
	///  @param[in]		prevRef		The reference of the polygon before the connection.
	///  @param[in]		polyRef		The reference of the off-mesh connection polygon.
	///  @param[in]		currentPos	Position before entering off-mesh connection [(x, y, z)]
	///  @param[out]	startPos	The start position of the off-mesh connection. [(x, y, z)]
	///  @param[out]	endPos		The end position of the off-mesh connection. [(x, y, z)]
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus getOffMeshConnectionPolyEndPoints(dtPolyRef prevRef, dtPolyRef polyRef, const dtReal* currentPos, dtReal* startPos, dtReal* endPos) const;

	/// Gets the specified off-mesh connection: point type.
	///  @param[in]	ref		The polygon reference of the off-mesh connection.
	/// @return The specified off-mesh connection, or null if the polygon reference is not valid.
	NAVMESH_API const dtOffMeshConnection* getOffMeshConnectionByRef(dtPolyRef ref) const;

	/// Updates area and flags for specified off-mesh connection: point type
	///  @param[in] userId	User Id of connection
	///	 @param[in] newArea	Area code to apply
	NAVMESH_API void updateOffMeshConnectionByUserId(unsigned long long int userId, unsigned char newArea, unsigned short newFlags);

	//@UE BEGIN
#if WITH_NAVMESH_SEGMENT_LINKS
	/// Gets the specified off-mesh connection: segment type
	///  @param[in]	ref		The polygon reference of the off-mesh connection.
	/// @return The specified off-mesh connection, or null if the polygon reference is not valid.
	NAVMESH_API const dtOffMeshSegmentConnection* getOffMeshSegmentConnectionByRef(dtPolyRef ref) const;

	/// Updates area and flags for specified off-mesh connection: segment type
	///  @param[in] userId	User Id of connection
	///	 @param[in] newArea	Area code to apply
	NAVMESH_API void updateOffMeshSegmentConnectionByUserId(unsigned int userId, unsigned char newArea, unsigned short newFlags);
#endif // WITH_NAVMESH_SEGMENT_LINKS
	//@UE END

	/// @}

	/// @{
	/// @name State Management
	/// These functions do not effect #dtTileRef or #dtPolyRef's. 

	/// Sets the user defined flags for the specified polygon.
	///  @param[in]	ref		The polygon reference.
	///  @param[in]	flags	The new flags for the polygon.
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus setPolyFlags(dtPolyRef ref, unsigned short flags);

	/// Gets the user defined flags for the specified polygon.
	///  @param[in]		ref				The polygon reference.
	///  @param[out]	resultFlags		The polygon flags.
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus getPolyFlags(dtPolyRef ref, unsigned short* resultFlags) const;

	/// Sets the user defined area for the specified polygon.
	///  @param[in]	ref		The polygon reference.
	///  @param[in]	area	The new area id for the polygon. [Limit: < #DT_MAX_AREAS]
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus setPolyArea(dtPolyRef ref, unsigned char area);

	/// Gets the user defined area for the specified polygon.
	///  @param[in]		ref			The polygon reference.
	///  @param[out]	resultArea	The area id for the polygon.
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus getPolyArea(dtPolyRef ref, unsigned char* resultArea) const;

	/// Gets the size of the buffer required by #storeTileState to store the specified tile's state.
	///  @param[in]	tile	The tile.
	/// @return The size of the buffer required to store the state.
	NAVMESH_API int getTileStateSize(const dtMeshTile* tile) const;
	
	/// Stores the non-structural state of the tile in the specified buffer. (Flags, area ids, etc.)
	///  @param[in]		tile			The tile.
	///  @param[out]	data			The buffer to store the tile's state in.
	///  @param[in]		maxDataSize		The size of the data buffer. [Limit: >= #getTileStateSize]
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus storeTileState(const dtMeshTile* tile, unsigned char* data, const int maxDataSize) const;
	
	/// Restores the state of the tile.
	///  @param[in]	tile			The tile.
	///  @param[in]	data			The new state. (Obtained from #storeTileState.)
	///  @param[in]	maxDataSize		The size of the state within the data buffer.
	/// @return The status flags for the operation.
	NAVMESH_API dtStatus restoreTileState(dtMeshTile* tile, const unsigned char* data, const int maxDataSize);
	
	/// @}

	/// @{
	/// @name Encoding and Decoding
	/// These functions are generally meant for internal use only.

	/// Derives a standard polygon reference.
	///  @note This function is generally meant for internal use only.
	///  @param[in]	salt	The tile's salt value.
	///  @param[in]	it		The index of the tile.
	///  @param[in]	ip		The index of the polygon within the tile.
	inline dtPolyRef encodePolyId(unsigned int salt, unsigned int it, unsigned int ip) const
	{
		return ((dtPolyRef)salt << (m_polyBits+m_tileBits)) | ((dtPolyRef)it << m_polyBits) | (dtPolyRef)ip;
	}
	
	/// Decodes a standard polygon reference.
	///  @note This function is generally meant for internal use only.
	///  @param[in]	ref   The polygon reference to decode.
	///  @param[out]	salt	The tile's salt value.
	///  @param[out]	it		The index of the tile.
	///  @param[out]	ip		The index of the polygon within the tile.
	///  @see #encodePolyId
	inline void decodePolyId(dtPolyRef ref, unsigned int& salt, unsigned int& it, unsigned int& ip) const
	{
		const dtPolyRef saltMask = ((dtPolyRef)1<<m_saltBits)-1;
		const dtPolyRef tileMask = ((dtPolyRef)1<<m_tileBits)-1;
		const dtPolyRef polyMask = ((dtPolyRef)1<<m_polyBits)-1;
		salt = (unsigned int)((ref >> (m_polyBits+m_tileBits)) & saltMask);
		it = (unsigned int)((ref >> m_polyBits) & tileMask);
		ip = (unsigned int)(ref & polyMask);
	}

	/// Extracts a tile's salt value from the specified polygon reference.
	///  @note This function is generally meant for internal use only.
	///  @param[in]	ref		The polygon reference.
	///  @see #encodePolyId
	inline unsigned int decodePolyIdSalt(dtPolyRef ref) const
	{
		const dtPolyRef saltMask = ((dtPolyRef)1<<m_saltBits)-1;
		return (unsigned int)((ref >> (m_polyBits+m_tileBits)) & saltMask);
	}
	
	/// Extracts the tile's index from the specified polygon reference.
	///  @note This function is generally meant for internal use only.
	///  @param[in]	ref		The polygon reference.
	///  @see #encodePolyId
	inline unsigned int decodePolyIdTile(dtPolyRef ref) const
	{
		const dtPolyRef tileMask = ((dtPolyRef)1<<m_tileBits)-1;
		return (unsigned int)((ref >> m_polyBits) & tileMask);
	}
	
	/// Extracts the polygon's index (within its tile) from the specified polygon reference.
	///  @note This function is generally meant for internal use only.
	///  @param[in]	ref		The polygon reference.
	///  @see #encodePolyId
	inline unsigned int decodePolyIdPoly(dtPolyRef ref) const
	{
		const dtPolyRef polyMask = ((dtPolyRef)1<<m_polyBits)-1;
		return (unsigned int)(ref & polyMask);
	}

	/// Extracts the tile's index from the specified cluster reference.
	///  @note This function is generally meant for internal use only.
	///  @param[in]	ref		The cluster reference.
	inline unsigned int decodeClusterIdTile(dtClusterRef ref) const
	{
		return decodePolyIdTile(ref);
	}

	//@UE BEGIN 
#if WITH_NAVMESH_CLUSTER_LINKS
	/// Extracts the cluster's index (within its tile) from the specified cluster reference.
	///  @note This function is generally meant for internal use only.
	///  @param[in]	ref		The cluster reference.
	inline unsigned int decodeClusterIdCluster(dtClusterRef ref) const
	{
		return decodePolyIdPoly(ref);
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS

	/// Shift navigation mesh by provided offset
	NAVMESH_API void applyWorldOffset(const dtReal* offset);

	/// Helper for accessing links
	inline dtLink& getLink(dtMeshTile* tile, unsigned int linkIdx)
	{
		return (linkIdx < (unsigned int)tile->header->maxLinkCount) ? tile->links[linkIdx] : tile->dynamicLinks[linkIdx - tile->header->maxLinkCount]; //@UE
	}

	inline const dtLink& getLink(const dtMeshTile* tile, unsigned int linkIdx) const
	{
		return (linkIdx < (unsigned int)tile->header->maxLinkCount) ? tile->links[linkIdx] : tile->dynamicLinks[linkIdx - tile->header->maxLinkCount]; //@UE
	}

#if WITH_NAVMESH_CLUSTER_LINKS
	/// Helper for accessing cluster links
	inline dtClusterLink& getClusterLink(dtMeshTile* tile, unsigned int linkIdx)
	{
		return tile->dynamicLinksC[linkIdx - DT_CLINK_FIRST];
	}

	inline const dtClusterLink& getClusterLink(const dtMeshTile* tile, unsigned int linkIdx) const
	{
		return tile->dynamicLinksC[linkIdx - DT_CLINK_FIRST];
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS

	/// Helper for creating links in off-mesh connections
	NAVMESH_API void linkOffMeshHelper(dtMeshTile* tile0, unsigned int polyIdx0, const dtMeshTile* tile1, unsigned int polyIdx1, unsigned char side, unsigned char edge);

	inline bool isEmpty() const
	{
		// has no tile grid set up
		return (m_tileWidth > 0 && m_tileHeight > 0) == false;
	}

	inline unsigned int getSaltBits() const
	{
		return m_saltBits;
	}

	NAVMESH_API void applyAreaCostOrder(unsigned char* costOrder);
	
	/// Defers off-mesh point link connection during addTile.
	/// Call connectDeferredOffMeshLinks() after all tiles are loaded.
	void setDeferOffMeshLinking(bool bDefer) { m_bDeferOffMeshLinking = bDefer; }
	bool isDeferringOffMeshLinking() const { return m_bDeferOffMeshLinking; }

	/// Connects off-mesh point links for all tiles. Used after batch loading
	/// when off-mesh linking was deferred via setDeferOffMeshLinking(true).
	NAVMESH_API void connectDeferredOffMeshLinks();

	/// Returns neighbour tile count based on side of given tile.
	NAVMESH_API int getNeighbourTilesCountAt(const int x, const int y, const int side) const;

	bool getNeighbourCoords(const int x, const int y, const int side, int& outX, int& outY) const
	{
		outX = x;
		outY = y;
		switch (side)
		{
		case 0: ++outX; break;
		case 1: ++outX; ++outY; break;
		case 2: ++outY; break;
		case 3: --outX; ++outY; break;
		case 4: --outX; break;
		case 5: --outX; --outY; break;
		case 6: --outY; break;
		case 7: ++outX; --outY; break;
		};
		// @todo we might want to do some validation
		return true;
	}

	unsigned int getTileIndex(const dtMeshTile* tile) const
	{
		return (unsigned int)(tile - m_tiles);
	}
	//@UE END
	/// @}

	//@UE Begin LWCoords
	dtReal getWalkableHeight() const { return m_params.walkableHeight; }
	dtReal getWalkableRadius() const { return m_params.walkableRadius; }
	dtReal getWalkableClimb() const { return m_params.walkableClimb; }
	dtReal getBVQuantFactor(const unsigned char resolution) const { return m_params.resolutionParams[resolution].bvQuantFactor; }
	//@UE END LWCoords

private:

	// [UE] result struct for findConnectingPolys
	struct FConnectingPolyData
	{
		dtReal min;
		dtReal max;
		dtPolyRef ref;
	};

	/// Returns pointer to tile in the tile array.
	NAVMESH_API dtMeshTile* getTile(int i);

	/// Returns the tile at the coordinates.
	NAVMESH_API int getTilesAt(const int x, const int y,
				   dtMeshTile** tiles, const int maxTiles) const;

	/// Returns neighbour tile based on side.
	NAVMESH_API int getNeighbourTilesAt(const int x, const int y, const int side,
							dtMeshTile** tiles, const int maxTiles) const;
	
	/// [UE] Returns all polygons in neighbour tile based on portal defined by the segment.
	NAVMESH_API int findConnectingPolys(const dtReal* va, const dtReal* vb,
		const dtMeshTile* fromTile, int fromPolyIdx,
		const dtMeshTile* tile, int side,
		dtChunkArray<FConnectingPolyData>& cons) const;

	/// Builds internal polygons links for a tile.
	NAVMESH_API void connectIntLinks(dtMeshTile* tile);
	/// Builds internal polygons links for a tile.
	NAVMESH_API void baseOffMeshLinks(dtMeshTile* tile);

	/// Builds external polygon links for a tile.
	NAVMESH_API void connectExtLinks(dtMeshTile* tile, dtMeshTile* target, int side, bool updateCLinks);
	/// Builds external polygon links for a tile.
	NAVMESH_API void connectExtOffMeshLinks(dtMeshTile* tile, dtMeshTile* target, int side, bool updateCLinks);
	
	/// Removes external links at specified side.
	NAVMESH_API void unconnectExtLinks(dtMeshTile* tile, dtMeshTile* target);
	
	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	/// Try to connect clusters
	NAVMESH_API void connectClusterLink(dtMeshTile* tile0, unsigned int cluster0,
							dtMeshTile* tile1, unsigned int cluster1,
							unsigned char flags, bool bCheckExisting = true);
	
	/// Removes cluster links at specified side.
	NAVMESH_API void unconnectClusterLinks(dtMeshTile* tile, dtMeshTile* target);
#endif // WITH_NAVMESH_CLUSTER_LINKS
	//@UE END

	// TODO: These methods are duplicates from dtNavMeshQuery, but are needed for off-mesh connection finding.
	
	/// Queries polygons within a tile.
	NAVMESH_API int queryPolygonsInTile(const dtMeshTile* tile, const dtReal* qmin, const dtReal* qmax,
							dtPolyRef* polys, const int maxPolys, bool bExcludeUnwalkable = false) const;
	/// Find nearest polygon within a tile.
	NAVMESH_API dtPolyRef findNearestPolyInTile(const dtMeshTile* tile, const dtReal* center,
									const dtReal* extents, dtReal* nearestPt, bool bExcludeUnwalkable = false) const;
	NAVMESH_API dtPolyRef findCheapestNearPolyInTile(const dtMeshTile* tile, const dtReal* center,
										 const dtReal* extents, dtReal* nearestPt) const;
	/// Returns closest point on polygon.
	NAVMESH_API void closestPointOnPolyInTile(const dtMeshTile* tile, unsigned int ip,
								  const dtReal* pos, dtReal* closest) const;

private:
	/// Calculates whether the tile grid location can fit in the tile indices type (currently an int)
	///  @param[in]	tx			Tile X coord
	///  @param[in]	ty			Tile Y coord
	NAVMESH_API bool isTileLocInValidRange(const dtReal tx, const dtReal ty) const;
	NAVMESH_API void calcTileLoc(const dtReal* pos, dtReal* tx, dtReal* ty) const;

public:
	dtNavMeshParams m_params;			///< Current initialization params.
	dtReal m_orig[3];					///< Origin of the tile (0,0)
	dtReal m_tileWidth, m_tileHeight;	///< Dimensions of each tile.

	int m_maxTiles;						///< Max number of tiles.
	int m_tileLutSize;					///< Tile hash lookup size (must be pot).
	int m_tileLutMask;					///< Tile hash lookup mask.

	unsigned char m_areaCostOrder[DT_MAX_AREAS];
	bool m_bDeferOffMeshLinking;

	dtMeshTile** m_posLookup;			///< Tile hash lookup.
	dtMeshTile* m_nextFree;				///< Freelist of tiles.
	dtMeshTile* m_tiles;				///< List of tiles.
		
	unsigned int m_saltBits;			///< Number of salt bits in the tile ID.
	unsigned int m_tileBits;			///< Number of tile bits in the tile ID.
	unsigned int m_polyBits;			///< Number of poly bits in the tile ID.
};

/// Allocates a navigation mesh object using the Detour allocator.
/// @return A navigation mesh that is ready for initialization, or null on failure.
///  @ingroup detour
NAVMESH_API dtNavMesh* dtAllocNavMesh();

/// Frees the specified navigation mesh object using the Detour allocator.
///  @param[in]	navmesh		A navigation mesh allocated using #dtAllocNavMesh
///  @ingroup detour
NAVMESH_API void dtFreeNavMesh(dtNavMesh* navmesh);

// @UE BEGIN: helper for reading tiles
struct ReadTilesHelper
{
	static const int MaxTiles = 32;
	dtMeshTile* Tiles[MaxTiles];

	int NumAllocated;
	dtMeshTile** AllocatedTiles;

	ReadTilesHelper() : NumAllocated(0), AllocatedTiles(nullptr) {}
	~ReadTilesHelper()
	{
		dtFree(AllocatedTiles, DT_ALLOC_TEMP);
	}

	dtMeshTile** PrepareArray(int RequestedSize)
	{
		if (RequestedSize < MaxTiles)
		{
			return Tiles;
		}

		if (NumAllocated < RequestedSize)
		{
			dtFree(AllocatedTiles, DT_ALLOC_TEMP);
			AllocatedTiles = (dtMeshTile**)dtAlloc(RequestedSize * sizeof(dtMeshTile*), DT_ALLOC_TEMP);
			NumAllocated = AllocatedTiles != nullptr ? RequestedSize : 0;
		}

		return AllocatedTiles;
	}
};
// @UE END

#endif // DETOURNAVMESH_H

///////////////////////////////////////////////////////////////////////////

// This section contains detailed documentation for members that don't have
// a source file. It reduces clutter in the main section of the header.

/**

@typedef dtPolyRef
@par

Polygon references are subject to the same invalidate/preserve/restore 
rules that apply to #dtTileRef's.  If the #dtTileRef for the polygon's
tile changes, the polygon reference becomes invalid.

Changing a polygon's flags, area id, etc. does not impact its polygon
reference.

@typedef dtTileRef
@par

The following changes will invalidate a tile reference:

- The referenced tile has been removed from the navigation mesh.
- The navigation mesh has been initialized using a different set
  of #dtNavMeshParams.

A tile reference is preserved/restored if the tile is added to a navigation 
mesh initialized with the original #dtNavMeshParams and is added at the
original reference location. (E.g. The lastRef parameter is used with
dtNavMesh::addTile.)

Basically, if the storage structure of a tile changes, its associated
tile reference changes.


@var unsigned short dtPoly::neis[DT_VERTS_PER_POLYGON]
@par

Each entry represents data for the edge starting at the vertex of the same index. 
E.g. The entry at index n represents the edge data for vertex[n] to vertex[n+1].

A value of zero indicates the edge has no polygon connection. (It makes up the 
border of the navigation mesh.)

The information can be extracted as follows: 
@code 
neighborRef = neis[n] & 0xff; // Get the neighbor polygon reference.

if (neis[n] & #DT_EX_LINK)
{
    // The edge is an external (portal) edge.
}
@endcode

@var dtReal dtNavMesh::bvQuantFactor
@par

This value is used for converting between world and bounding volume coordinates.
For example:
@code
const dtReal cs = 1.0f / bvQuantFactor;
const dtBVNode* n = &tile->bvTree[i];
if (n->i >= 0)
{
    // This is a leaf node.
    dtReal worldMinX = tile->header->bmin[0] + n->bmin[0]*cs;
    dtReal worldMinY = tile->header->bmin[0] + n->bmin[1]*cs;
    // Etc...
}
@endcode

@struct dtMeshTile
@par

Tiles generally only exist within the context of a dtNavMesh object.

Some tile content is optional.  For example, a tile may not contain any
off-mesh connections.  In this case the associated pointer will be null.

If a detail mesh exists it will share vertices with the base polygon mesh.  
Only the vertices unique to the detail mesh will be stored in #detailVerts.

@warning Tiles returned by a dtNavMesh object are not guarenteed to be populated.
For example: The tile at a location might not have been loaded yet, or may have been removed.
In this case, pointers will be null.  So if in doubt, check the polygon count in the 
tile's header to determine if a tile has polygons defined.

@var dtReal dtOffMeshConnection::pos[6]
@par

For a properly built navigation mesh, vertex A will always be within the bounds of the mesh. 
Vertex B is not required to be within the bounds of the mesh.

*/
