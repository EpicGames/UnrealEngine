// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURNAVMESHBUILDER_H
#define DETOURNAVMESHBUILDER_H

#include "CoreMinimal.h"
#include "Detour/DetourAlloc.h"
#include "DetourLargeWorldCoordinates.h"

struct dtOffMeshLinkCreateParams
{
	/// Off-mesh connection vertices (point-point = A0 <> B0, segment-segment = A0-A1 <> B0-B1 ) [Unit: wu]
	dtReal vertsA0[3];
	dtReal vertsA1[3];
	dtReal vertsB0[3];
	dtReal vertsB1[3];
	/// Off-mesh connection radii. [Unit: wu]
	dtReal snapRadius;
	/// Off-mesh connection height, less than 0 = use step height [Unit: wu]
	dtReal snapHeight;
	/// The user defined ids of the off-mesh connection.
	unsigned long long int userID;
	/// User defined flags assigned to the polys of off-mesh connections
	unsigned short polyFlag;
	/// User defined area ids assigned to the off-mesh connections
	unsigned char area;
	/// Off-mesh connection type (point to point, segment to segment, bidirectional)
	unsigned char type;	
};

// @todo: This appears unused
struct dtDynamicAreaCreateParams
{
	/// Area id
	unsigned char area;
	/// Covex min,max height
	dtReal minH;
	dtReal maxH;
	/// X,Z coords of covex
	dtChunkArray<dtReal> verts;
};

/// Represents the source data used to build an navigation mesh tile.
/// @ingroup detour
struct dtNavMeshCreateParams
{

	/// @name Polygon Mesh Attributes
	/// Used to create the base navigation graph.
	/// See #rcPolyMesh for details related to these attributes.
	/// @{

	const unsigned short* verts;			///< The polygon mesh vertices. [(x, y, z) * #vertCount] [Unit: vx]
	int vertCount;							///< The number vertices in the polygon mesh. [Limit: >= 3]
	const unsigned short* polys;			///< The polygon data. [Size: #polyCount * 2 * #nvp]
	const unsigned short* polyFlags;		///< The user defined flags assigned to each polygon. [Size: #polyCount]
	const unsigned char* polyAreas;			///< The user defined area ids assigned to each polygon. [Size: #polyCount]
	int polyCount;							///< Number of polygons in the mesh. [Limit: >= 1]
	int nvp;								///< Number maximum number of vertices per polygon. [Limit: >= 3]

	// @UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	/// @}
	/// @name Cluster Attributes
	/// @{
	unsigned short* polyClusters;			///< Cluster Id for each polygon [Size: #polyCount]
	unsigned short clusterCount;			///< Number of unique clusters
#endif // WITH_NAVMESH_CLUSTER_LINKS
	// @UE END

	/// @}
	/// @name Height Detail Attributes (Optional)
	/// See #rcPolyMeshDetail for details related to these attributes.
	/// @{

	const unsigned int* detailMeshes;		///< The height detail sub-mesh data. [Size: 4 * #polyCount]
	const dtReal* detailVerts;				///< The detail mesh vertices. [Size: 3 * #detailVertsCount] [Unit: wu]
	int detailVertsCount;					///< The number of vertices in the detail mesh.
	const unsigned char* detailTris;		///< The detail mesh triangles. [Size: 4 * #detailTriCount]
	int detailTriCount;						///< The number of triangles in the detail mesh.

	/// @}
	/// @name Off-Mesh Connections Attributes (Optional)
	/// Used to define a custom edge within the navigation graph, an 
	/// off-mesh connection is a user defined traversable connection, 
	/// at least one side resides within a navigation mesh polygon.
	/// @{

	/// Off-mesh connection data. [Size: #offMeshConCount] [Unit: wu]
	const dtOffMeshLinkCreateParams* offMeshCons;
	/// The number of off-mesh connections. [Limit: >= 0]
	int offMeshConCount;

	/// @}
	/// @name Dynamic Area Attributes (Optional)
	/// Used to define a custom dynamic obstacles from convex volumes
	/// @{

	/// Dynamic Area data. [Size: #dynamicAreaCount] [Unit: wu]
	const dtDynamicAreaCreateParams* dynamicAreas; // @todo: this appears unused
	/// The number of dynamic areas. [Limit: >= 0]
	int dynamicAreaCount;


	/// @}
	/// @name Tile Attributes
	/// @note The tile grid/layer data can be left at zero if the destination is a single tile mesh.
	/// @{

	unsigned int userId;	///< The user defined id of the tile.
	int tileX;				///< The tile's x-grid location within the multi-tile destination mesh. (Along the x-axis.)
	int tileY;				///< The tile's y-grid location within the multi-tile destination mesh. (Along the z-axis.)
	int tileLayer;			///< The tile's layer within the layered destination mesh. (Along the y-axis.) [Limit: >= 0]
	dtReal bmin[3];			///< The minimum bounds of the tile. [(x, y, z)] [Unit: wu]
	dtReal bmax[3];			///< The maximum bounds of the tile. [(x, y, z)] [Unit: wu]

	/// @}
	/// @name General Configuration Attributes
	/// @{

	dtReal walkableHeight;	///< The agent height. [Unit: wu]
	dtReal walkableRadius;	///< The agent radius. [Unit: wu]
	dtReal walkableClimb;	///< The agent maximum traversable ledge. (Up/Down) [Unit: wu]
	dtReal cs;				///< The xz-plane cell size of the polygon mesh. [Limit: > 0] [Unit: wu]
	dtReal ch;				///< The y-axis cell height of the polygon mesh. [Limit: > 0] [Unit: wu]

	unsigned char tileResolutionLevel;	///< Tile resolution index 		//@UE

	/// True if a bounding volume tree should be built for the tile.
	/// @note The BVTree is not normally needed for layered navigation meshes.
	bool buildBvTree;

	//@UE BEGIN
	/// If true, minimize base link pool size else use previous larger base pool.
	bool bMinimizeLinkPoolSize;
	//@UE END

	/// @}
};

/// Builds navigation mesh tile data from the provided tile creation data.
/// @ingroup detour
///  @param[in]		params		Tile creation data.
///  @param[out]	outData		The resulting tile data.
///  @param[out]	outDataSize	The size of the tile data array.
/// @return True if the tile data was successfully created.
NAVMESH_API bool dtCreateNavMeshData(dtNavMeshCreateParams* params, unsigned char** outData, int* outDataSize);

/// Swaps the endianess of the tile data's header (#dtMeshHeader).
///  @param[in,out]	data		The tile data array.
///  @param[in]		dataSize	The size of the data array.
NAVMESH_API bool dtNavMeshHeaderSwapEndian(unsigned char* data, const int dataSize);

/// Swaps endianess of the tile data.
///  @param[in,out]	data		The tile data array.
///  @param[in]		dataSize	The size of the data array.
NAVMESH_API bool dtNavMeshDataSwapEndian(unsigned char* data, const int dataSize);

// @UE BEGIN
/// Offset and rotate around center the data in the tile
///  @param[in,out]	data			Data of the tile mesh. (See: #dtCreateNavMeshData)
///  @param[in]		dataSize		Data size of the tile mesh.
///  @param[in]		offsetX			X offset in tile coordinates.
///  @param[in]		offsetY			Y offset in tile coordinates.
///  @param[in]		tileWidth		Tile width.
///  @param[in]		tileHeight		Tile height.
///  @param[in]		rotationDeg		Rotation in degrees.
NAVMESH_API bool dtTransformTileData(unsigned char* data, const int dataSize, const int offsetX, const int offsetY, const dtReal tileWidth, const dtReal tileHeight, const dtReal rotationDeg, const dtReal bvQuantFactor);

/// Compute XY offset caused by the given rotation
///  @param[in]		position		Position to rotate. [(x, y, z)]
///  @param[in]		rotationCenter	Rotation center. [(x, y, z)]
///  @param[in]		rotationDeg		Rotation in degrees.
///  @param[in]		tileWidth		Tile width.
///  @param[in]		tileHeight		Tile height.
///  @param[out]	deltaX			Offset X in tile coordinates.
///  @param[out]	deltaY			Offset Y in tile coordinates.
NAVMESH_API void dtComputeTileOffsetFromRotation(const dtReal* position, const dtReal* rotationCenter, const dtReal rotationDeg, const dtReal tileWidth, const dtReal tileHeight, int& deltaX, int& deltaY);
// @UE END

#endif // DETOURNAVMESHBUILDER_H

// This section contains detailed documentation for members that don't have
// a source file. It reduces clutter in the main section of the header.

/**

@struct dtNavMeshCreateParams
@par

This structure is used to marshal data between the Recast mesh generation pipeline and Detour navigation components.

See the rcPolyMesh and rcPolyMeshDetail documentation for detailed information related to mesh structure.

Units are usually in voxels (vx) or world units (wu). The units for voxels, grid size, and cell size 
are all based on the values of #cs and #ch.

The standard navigation mesh build process is to create tile data using dtCreateNavMeshData, then add the tile 
to a navigation mesh using either the dtNavMesh single tile <tt>init()</tt> function or the dtNavMesh::addTile()
function.

@see dtCreateNavMeshData

*/
