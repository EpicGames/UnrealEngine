// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURNAVMESHQUERY_H
#define DETOURNAVMESHQUERY_H

#include "CoreMinimal.h"
#include "Detour/DetourAlloc.h"
#include "Detour/DetourCommon.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavMesh.h"
#include "Detour/DetourStatus.h"
#include "Misc/AssertionMacros.h"

//@UE BEGIN
#define WITH_FIXED_AREA_ENTERING_COST 1

// Note poly costs are still floats in UE so we are using FLT_MAX as unwalkable still.
#define DT_UNWALKABLE_POLY_COST FLT_MAX
//@UE END

// By default dtQueryFilter will use virtual calls.
// On certain platforms indirect or virtual function call is expensive. The default
// setting is to use non-virtual functions, the actual implementations of the functions
// are declared as inline for maximum speed. 
// To avoid virtual calls create dtQueryFilter with inIsVirtual = false.

// Special link filter is custom filter run only for offmesh links with assigned UserId
// Used by smart navlinks in UE
//
struct dtQuerySpecialLinkFilter
{
	virtual ~dtQuerySpecialLinkFilter() = default;

	/// Returns true if special link can be visited.  (I.e. Is traversable.)
	///  @param[in]		UserId		Unique Id of link
	
	virtual bool isLinkAllowed(const unsigned long long int UserId) const { return true; }

	UE_DEPRECATED(5.3, "LinkIds are now based on a 64 bit uint. Use the version of this function that takes an unsigned long long int")
	virtual bool isLinkAllowed(const int UserId) const final { return true; }

	/// Called before accessing in A* loop (can be called multiple time for updateSlicedFindPath)
	virtual void initialize() {}
};

// [UE: moved all filter variables to struct, DO NOT mess with virtual functions here!]
struct dtQueryFilterData
{
	dtReal m_areaCost[DT_MAX_AREAS];		///< Cost per area type. (Used by default implementation.)
#if WITH_FIXED_AREA_ENTERING_COST
	dtReal m_areaFixedCost[DT_MAX_AREAS];///< Fixed cost for entering an area type (Used by default implementation.)
#endif // WITH_FIXED_AREA_ENTERING_COST
	dtReal heuristicScale;				///< Search heuristic scale.
	dtReal lowestAreaCost;

	unsigned short m_includeFlags;		///< Flags for polygons that can be visited. (Used by default implementation.)
	unsigned short m_excludeFlags;		///< Flags for polygons that should not be visited. (Used by default implementation.)

	bool m_isBacktracking;
	//@UE BEGIN
	/// whether to ignore neighbour nodes that have already been visited.
	/// FALSE by default but you might want to set it to true if using heuristic
	/// scale much larger than 1 or experiencing cycles in our paths for any other
	/// reason
	bool m_shouldIgnoreClosedNodes;
	//@UE END

	NAVMESH_API dtQueryFilterData();
	
	NAVMESH_API bool equals(const dtQueryFilterData* other) const;
	NAVMESH_API void copyFrom(const dtQueryFilterData* source);
};

/// Defines polygon filtering and traversal costs for navigation mesh query operations.
/// @ingroup detour
class dtQueryFilter
{
protected:
	dtQueryFilterData data;
	bool isVirtual;
	
public:
	dtQueryFilter(bool inIsVirtual = true) : isVirtual(inIsVirtual)
	{}
	virtual ~dtQueryFilter() = default;
	
protected:

	/// inlined filter implementation. @see passFilter for parameter description
	inline bool passInlineFilter(const dtPolyRef ref,
		const dtMeshTile* tile,
		const dtPoly* poly) const
	{
		return (poly->flags & data.m_includeFlags) != 0 && (poly->flags & data.m_excludeFlags) == 0
			&& (data.m_areaCost[poly->getArea()] < DT_UNWALKABLE_POLY_COST)
#if WITH_FIXED_AREA_ENTERING_COST
			&& (data.m_areaFixedCost[poly->getArea()] < DT_UNWALKABLE_POLY_COST)
#endif // WITH_FIXED_AREA_ENTERING_COST
			;
	}

	/// virtual filter implementation (defaults to passInlineFilter). @see passFilter for parameter description
	virtual bool passVirtualFilter(const dtPolyRef ref,
		const dtMeshTile* tile,
		const dtPoly* poly) const
	{
		return passInlineFilter(ref, tile, poly);
	}

public:
	/// Returns true if it's a virtual filter.
	bool getIsVirtual() const { return isVirtual; }
	
	/// Returns true if the polygon can be visited.  (I.e. Is traversable.)
	///  @param[in]		ref		The reference id of the polygon test.
	///  @param[in]		tile	The tile containing the polygon.
	///  @param[in]		poly  The polygon to test.
	inline bool passFilter(const dtPolyRef ref,
							const dtMeshTile* tile,
							const dtPoly* poly) const
	{
		return !isVirtual ? passInlineFilter(ref, tile, poly) : passVirtualFilter(ref, tile, poly);
	}

protected:

	/// inlined scoring function. @see getCost for parameter description
	inline dtReal getInlineCost(const dtReal* pa, const dtReal* pb,
		const dtPolyRef prevRef, const dtMeshTile* prevTile, const dtPoly* prevPoly,
		const dtPolyRef curRef, const dtMeshTile* curTile, const dtPoly* curPoly,
		const dtPolyRef nextRef, const dtMeshTile* nextTile, const dtPoly* nextPoly) const
	{
#if WITH_FIXED_AREA_ENTERING_COST
		const dtReal areaChangeCost = nextPoly != 0 && nextPoly->getArea() != curPoly->getArea()
			? data.m_areaFixedCost[nextPoly->getArea()] : 0.f;

		return dtVdist(pa, pb) * data.m_areaCost[curPoly->getArea()] + areaChangeCost;
#else
		return dtVdist(pa, pb) * data.m_areaCost[curPoly->getArea()];
#endif // #if WITH_FIXED_AREA_ENTERING_COST
	}

	/// virtual scoring function implementation (defaults to getInlineCost). @see getCost for parameter description
	virtual dtReal getVirtualCost(const dtReal* pa, const dtReal* pb,
		const dtPolyRef prevRef, const dtMeshTile* prevTile, const dtPoly* prevPoly,
		const dtPolyRef curRef, const dtMeshTile* curTile, const dtPoly* curPoly,
		const dtPolyRef nextRef, const dtMeshTile* nextTile, const dtPoly* nextPoly) const
	{
		return getInlineCost(pa, pb,
			prevRef, prevTile, prevPoly,
			curRef, curTile, curPoly,
			nextRef, nextTile, nextPoly);
	}

//@UE BEGIN Adding support for LWCoords.
#if !DT_LARGE_WORLD_COORDINATES_DISABLED
	// This function is deprecated call the version on this function that takes FReal
	virtual float getVirtualCost(const float* pa, const float* pb,
		const dtPolyRef prevRef, const dtMeshTile* prevTile, const dtPoly* prevPoly,
		const dtPolyRef curRef, const dtMeshTile* curTile, const dtPoly* curPoly,
		const dtPolyRef nextRef, const dtMeshTile* nextTile, const dtPoly* nextPoly) const final
	{
		check(false); // This function is deprecated use the version of this function that takes FReal
		return 0.f;
	}
#endif
//@UE END Adding support for LWCoords.

public:
	/// Returns cost to move from the beginning to the end of a line segment
	/// that is fully contained within a polygon.
	///  @param[in]		pa			The start position on the edge of the previous and current polygon. [(x, y, z)]
	///  @param[in]		pb			The end position on the edge of the current and next polygon. [(x, y, z)]
	///  @param[in]		prevRef		The reference id of the previous polygon. [opt]
	///  @param[in]		prevTile	The tile containing the previous polygon. [opt]
	///  @param[in]		prevPoly	The previous polygon. [opt]
	///  @param[in]		curRef		The reference id of the current polygon.
	///  @param[in]		curTile		The tile containing the current polygon.
	///  @param[in]		curPoly		The current polygon.
	///  @param[in]		nextRef		The reference id of the next polygon. [opt]
	///  @param[in]		nextTile	The tile containing the next polygon. [opt]
	///  @param[in]		nextPoly	The next polygon. [opt]
	inline dtReal getCost(const dtReal* pa, const dtReal* pb,
						  const dtPolyRef prevRef, const dtMeshTile* prevTile, const dtPoly* prevPoly,
						  const dtPolyRef curRef, const dtMeshTile* curTile, const dtPoly* curPoly,
						  const dtPolyRef nextRef, const dtMeshTile* nextTile, const dtPoly* nextPoly) const
	{
		return !isVirtual ? getInlineCost(pa, pb, prevRef, prevTile, prevPoly, curRef, curTile, curPoly, nextRef, nextTile, nextPoly)
			: getVirtualCost(pa, pb, prevRef, prevTile, prevPoly, curRef, curTile, curPoly, nextRef, nextTile, nextPoly);
	}

	/// @name Getters and setters for the default implementation data.
	///@{

	/// Returns the traversal cost of the area.
	///  @param[in]		i		The id of the area.
	/// @returns The traversal cost of the area.
	inline dtReal getAreaCost(const int i) const { return data.m_areaCost[i]; }

	/// Sets the traversal cost of the area.
	///  @param[in]		i		The id of the area.
	///  @param[in]		cost	The new cost of traversing the area.
	inline void setAreaCost(const int i, const dtReal cost) { data.m_areaCost[i] = cost; data.lowestAreaCost = dtMin(data.lowestAreaCost, cost); }

//@UE BEGIN
	inline const dtReal* getAllAreaCosts() const { return data.m_areaCost; }
	
#if WITH_FIXED_AREA_ENTERING_COST
	/// Returns the fixed cost for entering an area.
	///  @param[in]		i		The id of the area.
	///  @returns The fixed cost of the area.
	inline dtReal getAreaFixedCost(const int i) const { return data.m_areaFixedCost[i]; }

	/// Sets the fixed cost for entering an area.
	///  @param[in]		i		The id of the area.
	///  @param[in]		cost	The new fixed cost of entering the area polygon.
	inline void setAreaFixedCost(const int i, const dtReal cost) { data.m_areaFixedCost[i] = cost; }

	inline const dtReal* getAllFixedAreaCosts() const { return data.m_areaFixedCost; }
#endif // WITH_FIXED_AREA_ENTERING_COST

	inline dtReal getModifiedHeuristicScale() const { return data.heuristicScale * ((data.lowestAreaCost > 0) ? data.lowestAreaCost : 1.0f); }

	/// Retrieves euclidean distance heuristic scale
	///  @returns heuristic scale
	inline dtReal getHeuristicScale() const { return data.heuristicScale; }

	/// Set euclidean distance heuristic scale
	inline void setHeuristicScale(const dtReal newScale) { data.heuristicScale = newScale; }

	/// Filters link in regards to its side. Used for backtracking.
	///  @returns should link with this side be accepted
	inline bool isValidLinkSide(const unsigned char side) const 
	{ 
		return (side & DT_LINK_FLAG_OFFMESH_CON) == 0 || (side & DT_LINK_FLAG_OFFMESH_CON_BIDIR) != 0
			|| (data.m_isBacktracking ? (side & DT_LINK_FLAG_OFFMESH_CON_BACKTRACKER) != 0
				: (side & DT_LINK_FLAG_OFFMESH_CON_BACKTRACKER) == 0);
	}

	/// Sets up filter for backtracking
	inline void setIsBacktracking(const bool isBacktracking) { data.m_isBacktracking = isBacktracking; }

	/// Retrieves information whether this filter is set up for backtracking
	///  @returns is backtracking
	inline bool getIsBacktracking() const { return data.m_isBacktracking; }

	/// Instruct filter whether it can reopen nodes already on closed list
	inline void setShouldIgnoreClosedNodes(const bool shouldIgnore) { data.m_shouldIgnoreClosedNodes = shouldIgnore; }

	/// Retrieves information whether this filter allows reopening closed nodes
	///  @returns should consider reopening nodes already on closed list
	inline bool getShouldIgnoreClosedNodes() const { return data.m_shouldIgnoreClosedNodes; }
//@UE END

	/// Returns the include flags for the filter.
	/// Any polygons that include one or more of these flags will be
	/// included in the operation.
	inline unsigned short getIncludeFlags() const { return data.m_includeFlags; }

	/// Sets the include flags for the filter.
	/// @param[in]		flags	The new flags.
	inline void setIncludeFlags(const unsigned short flags) { data.m_includeFlags = flags; }

	/// Returns the exclude flags for the filter.
	/// Any polygons that include one ore more of these flags will be
	/// excluded from the operation.
	inline unsigned short getExcludeFlags() const { return data.m_excludeFlags; }

	/// Sets the exclude flags for the filter.
	/// @param[in]		flags		The new flags.
	inline void setExcludeFlags(const unsigned short flags) { data.m_excludeFlags = flags; }

	///@}

	/// Check if two filters have the same data values
	inline bool equals(const dtQueryFilter* other) const { return data.equals(&(other->data)); }
	inline bool equals(const dtQueryFilter& other) const { return data.equals(&(other.data)); }

	/// Copy data values from source filter
	inline void copyFrom(const dtQueryFilter* other) { data.copyFrom(&(other->data)); }
	inline void copyFrom(const dtQueryFilter& other) { data.copyFrom(&(other.data)); }

};

struct dtQueryResultPack
{
	dtPolyRef ref;
	dtReal cost;
	dtReal pos[3];
	unsigned int flag;

	dtQueryResultPack() : ref(0), cost(0), flag(0) {}
	dtQueryResultPack(dtPolyRef inRef, dtReal inCost, const dtReal* inPos, unsigned int inFlag);
};

struct dtQueryResult
{
	inline void reserve(int n) { data.resize(n); data.resize(0); }
	inline int size() const { return data.size(); }

	inline dtPolyRef getRef(int idx) const { return data[idx].ref; }
	inline dtReal getCost(int idx) const { return data[idx].cost; }
	inline const dtReal* getPos(int idx) const { return data[idx].pos; }
	inline unsigned int getFlag(int idx) const { return data[idx].flag; }
	NAVMESH_API void getPos(int idx, dtReal* pos);

	NAVMESH_API void copyRefs(dtPolyRef* refs, int nmax);
	NAVMESH_API void copyCosts(dtReal* costs, int nmax);
	NAVMESH_API void copyPos(dtReal* pos, int nmax);
	NAVMESH_API void copyFlags(unsigned char* flags, int nmax);
	NAVMESH_API void copyFlags(unsigned int* flags, int nmax);

protected:
	dtChunkArray<dtQueryResultPack> data;

	inline int addItem(dtPolyRef ref, dtReal cost, const dtReal* pos, unsigned int flag) { data.push(dtQueryResultPack(ref, cost, pos, flag)); return data.size() - 1; }

	inline void setRef(int idx, dtPolyRef ref) { data[idx].ref = ref; }
	inline void setCost(int idx, dtReal cost) { data[idx].cost = cost; }
	inline void setFlag(int idx, unsigned int flag) { data[idx].flag = flag; }
	NAVMESH_API void setPos(int idx, const dtReal* pos);

	friend class dtNavMeshQuery;
};

/// Provides the ability to perform pathfinding related queries against
/// a navigation mesh.
/// @ingroup detour
class dtNavMeshQuery
{
public:
	NAVMESH_API dtNavMeshQuery();
	NAVMESH_API ~dtNavMeshQuery();
	
	/// Initializes the query object.
	///  @param[in]		nav			Pointer to the dtNavMesh object to use for all queries.
	///  @param[in]		maxNodes	Maximum number of search nodes. [Limits: 0 < value <= 65536]
	///  @param[in]		linkFilter	Special link filter used for every query
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus init(const dtNavMesh* nav, const int maxNodes, dtQuerySpecialLinkFilter* linkFilter = 0);

	/// UE: updates special link filter for this query
	NAVMESH_API void updateLinkFilter(dtQuerySpecialLinkFilter* linkFilter);
	
	/// @name Standard Pathfinding Functions
	// /@{

	/// Finds a path from the start polygon to the end polygon.
	///  @param[in]		startRef	The reference id of the start polygon.
	///  @param[in]		endRef		The reference id of the end polygon.
	///  @param[in]		startPos	A position within the start polygon. [(x, y, z)]
	///  @param[in]		endPos		A position within the end polygon. [(x, y, z)]
	///  @param[in]		costLimit	Cost limit of nodes allowed to be added to the open list	//@UE
	///  @param[in]		filter		The polygon filter to apply to the query.
	///  @param[out]	result		Results for path corridor, fills in refs and costs for each poly from start to end
	///	 @param[out]	totalCost			If provided will get filled will total cost of path
	NAVMESH_API dtStatus findPath(dtPolyRef startRef, dtPolyRef endRef,
					  const dtReal* startPos, const dtReal* endPos, const dtReal costLimit, //@UE
					  const dtQueryFilter* filter,
					  dtQueryResult& result, dtReal* totalCost) const;
	
	/// Finds the straight path from the start to the end position within the polygon corridor.
	///  @param[in]		startPos			Path start position. [(x, y, z)]
	///  @param[in]		endPos				Path end position. [(x, y, z)]
	///  @param[in]		path				An array of polygon references that represent the path corridor.
	///  @param[in]		pathSize			The number of polygons in the @p path array.
	///  @param[out]	result				Fills in positions, refs and flags
	///  @param[in]		options				Query options. (see: #dtStraightPathOptions)
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findStraightPath(const dtReal* startPos, const dtReal* endPos,
							  const dtPolyRef* path, const int pathSize,
							  dtQueryResult& result, const int options = 0) const;

	///@}
	/// @name Sliced Pathfinding Functions
	/// Common use case:
	///	-# Call initSlicedFindPath() to initialize the sliced path query.
	///	-# Call updateSlicedFindPath() until it returns complete.
	///	-# Call finalizeSlicedFindPath() to get the path.
	///@{ 

	/// Initializes a sliced path query.
	///  @param[in]		startRef					The refrence id of the start polygon.
	///  @param[in]		endRef						The reference id of the end polygon.
	///  @param[in]		startPos					A position within the start polygon. [(x, y, z)]
	///  @param[in]		endPos						A position within the end polygon. [(x, y, z)]
	///  @param[in]		costLimit					Cost limit of nodes allowed to be added to the open list	//@UE
	///  @param[in]		requireNavigableEndLocation	Define if the end location is required to be a valid navmesh polygon //@UE
	///  @param[in]		filter						The polygon filter to apply to the query.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus initSlicedFindPath(dtPolyRef startRef, dtPolyRef endRef,
								const dtReal* startPos, const dtReal* endPos, const dtReal costLimit, const bool requireNavigableEndLocation, //@UE
								const dtQueryFilter* filter);

	/// Updates an in-progress sliced path query.
	///  @param[in]		maxIter		The maximum number of iterations to perform.
	///  @param[out]	doneIters	The actual number of iterations completed. [opt]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus updateSlicedFindPath(const int maxIter, int* doneIters);

	/// Finalizes and returns the results of a sliced path query.
	///  @param[out]	path		An ordered list of polygon references representing the path. (Start to end.) 
	///  							[(polyRef) * @p pathCount]
	///  @param[out]	pathCount	The number of polygons returned in the @p path array.
	///  @param[in]		maxPath		The max number of polygons the path array can hold. [Limit: >= 1]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus finalizeSlicedFindPath(dtPolyRef* path, int* pathCount, const int maxPath);
	
	/// Finalizes and returns the results of an incomplete sliced path query, returning the path to the furthest
	/// polygon on the existing path that was visited during the search.
	///  @param[out]	existing		An array of polygon references for the existing path.
	///  @param[out]	existingSize	The number of polygon in the @p existing array.
	///  @param[out]	path			An ordered list of polygon references representing the path. (Start to end.) 
	///  								[(polyRef) * @p pathCount]
	///  @param[out]	pathCount		The number of polygons returned in the @p path array.
	///  @param[in]		maxPath			The max number of polygons the @p path array can hold. [Limit: >= 1]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus finalizeSlicedFindPathPartial(const dtPolyRef* existing, const int existingSize,
										   dtPolyRef* path, int* pathCount, const int maxPath);

	///@}
	/// @name Dijkstra Search Functions
	/// @{ 

	/// Finds the polygons along the navigation graph that touch the specified circle.
	///  @param[in]		startRef		The reference id of the polygon where the search starts.
	///  @param[in]		centerPos		The center of the search circle. [(x, y, z)]
	///  @param[in]		radius			The radius of the search circle.
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[out]	resultRef		The reference ids of the polygons touched by the circle. [opt]
	///  @param[out]	resultParent	The reference ids of the parent polygons for each result. 
	///  								Zero if a result polygon has no parent. [opt]
	///  @param[out]	resultCost		The search cost from @p centerPos to the polygon. [opt]
	///  @param[out]	resultCount		The number of polygons found.
	///  @param[in]		maxResult		The maximum number of polygons the result arrays can hold.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findPolysAroundCircle(dtPolyRef startRef, const dtReal* centerPos, const dtReal radius,
								   const dtQueryFilter* filter,
								   dtPolyRef* resultRef, dtPolyRef* resultParent, dtReal* resultCost,
								   int* resultCount, const int maxResult) const;
	
	/// Finds the polygons along the naviation graph that touch the specified convex polygon.
	///  @param[in]		startRef		The reference id of the polygon where the search starts.
	///  @param[in]		verts			The vertices describing the convex polygon. (CCW) 
	///  								[(x, y, z) * @p nverts]
	///  @param[in]		nverts			The number of vertices in the polygon.
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[out]	resultRef		The reference ids of the polygons touched by the search polygon. [opt]
	///  @param[out]	resultParent	The reference ids of the parent polygons for each result. Zero if a 
	///  								result polygon has no parent. [opt]
	///  @param[out]	resultCost		The search cost from the centroid point to the polygon. [opt]
	///  @param[out]	resultCount		The number of polygons found.
	///  @param[in]		maxResult		The maximum number of polygons the result arrays can hold.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findPolysAroundShape(dtPolyRef startRef, const dtReal* verts, const int nverts,
								  const dtQueryFilter* filter,
								  dtPolyRef* resultRef, dtPolyRef* resultParent, dtReal* resultCost,
								  int* resultCount, const int maxResult) const;
	
	//@UE BEGIN
	/// Finds the polygons along the navigation graph that are no more than given path length away from centerPos.
	///  @param[in]		startRef		The reference id of the polygon where the search starts.
	///  @param[in]		centerPos		The center of the search circle. [(x, y, z)]
	///  @param[in]		pathDistance	The path distance limit of the search
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[out]	resultRef		The reference ids of the polygons touched by the circle. [opt]
	///  @param[out]	resultCount		The number of polygons found. [opt]
	///  @param[in]		maxResult		The maximum number of polygons the result arrays can hold.
	/// @returns The status flags for the querydtReal
	NAVMESH_API dtStatus findPolysInPathDistance(dtPolyRef startRef, const dtReal* centerPos, const dtReal pathDistance,
									const dtQueryFilter* filter, dtPolyRef* resultRef,
									int* resultCount, const int maxResult) const;

	/// @}
	/// @name Local Query Functions
	///@{

	/// Finds the polygon nearest to the specified center point.
	///  @param[in]		center		The center of the search box. [(x, y, z)]
	///  @param[in]		extents		The search distance along each axis. [(x, y, z)]
	///  @param[in]		filter		The polygon filter to apply to the query.
	///  @param[out]	nearestRef	The reference id of the nearest polygon.
	///  @param[out]	nearestPt	The nearest point on the polygon. [opt] [(x, y, z)]
	///  @param[in]		referencePt	If supplied replaces @param center in terms of distance measurements. Note that extents are not precise when using a referencePt. [opt] [(x, y, z)]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findNearestPoly(const dtReal* center, const dtReal* extents,
							 const dtQueryFilter* filter,
							 dtPolyRef* nearestRef, dtReal* nearestPt, const dtReal* referencePt = 0) const;

	/// Finds the polygon 2D-nearest to the specified center point.
	///  @param[in]		center		The center of the search box. [(x, y, z)]
	///  @param[in]		extents		The search distance along each axis. [(x, y, z)]
	///  @param[in]		filter		The polygon filter to apply to the query.
	///  @param[out]	nearestRef	The reference id of the nearest polygon.
	///  @param[out]	nearestPt	The nearest point on the polygon. [opt] [(x, y, z)]
	///  @param[in]		referencePt	If supplied replaces @param center in terms of distance measurements. Note that extents are not precise when using a referencePt. [opt] [(x, y, z)]
	///  @param[in]		tolerance	Radius around best 2D point for picking vertical match
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findNearestPoly2D(const dtReal* center, const dtReal* extents,
							const dtQueryFilter* filter,
							dtPolyRef* outProjectedRef, dtReal* outProjectedPt,
							const dtReal* referencePt = 0, dtReal tolerance = 0) const;

	/// Finds the nearest polygon in height containing the specified center point.
	///  @param[in]		center		The center of the search box. [(x, y, z)]
	///  @param[in]		extents		The search distance along each axis. [(x, y, z)]
	///  @param[in]		filter		The polygon filter to apply to the query.
	///  @param[out]	nearestRef	The reference id of the nearest polygon.
	///  @param[out]	nearestPt	The nearest point on the polygon. [opt] [(x, y, z)]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findNearestContainingPoly(const dtReal* center, const dtReal* extents,
									   const dtQueryFilter* filter,
									   dtPolyRef* nearestRef, dtReal* nearestPt) const;
	//@UE END

	/// Finds polygons that overlap the search box.
	///  @param[in]		center		The center of the search box. [(x, y, z)]
	///  @param[in]		extents		The search distance along each axis. [(x, y, z)]
	///  @param[in]		filter		The polygon filter to apply to the query.
	///  @param[out]	polys		The reference ids of the polygons that overlap the query box.
	///  @param[out]	polyCount	The number of polygons in the search result.
	///  @param[in]		maxPolys	The maximum number of polygons the search result can hold.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus queryPolygons(const dtReal* center, const dtReal* extents,
						   const dtQueryFilter* filter,
						   dtPolyRef* polys, int* polyCount, const int maxPolys) const;

	/// Finds the non-overlapping navigation polygons in the local neighbourhood around the center position.
	///  @param[in]		startRef		The reference id of the polygon where the search starts.
	///  @param[in]		centerPos		The center of the query circle. [(x, y, z)]
	///  @param[in]		radius			The radius of the query circle.
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[out]	resultRef		The reference ids of the polygons touched by the circle.
	///  @param[out]	resultParent	The reference ids of the parent polygons for each result. 
	///  								Zero if a result polygon has no parent. [opt]
	///  @param[out]	resultCount		The number of polygons found.
	///  @param[in]		maxResult		The maximum number of polygons the result arrays can hold.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findLocalNeighbourhood(dtPolyRef startRef, const dtReal* centerPos, const dtReal radius,
									const dtQueryFilter* filter,
									dtPolyRef* resultRef, dtPolyRef* resultParent,
									int* resultCount, const int maxResult) const;

	/// [UE] Finds the wall segments in local neighbourhood
	NAVMESH_API dtStatus findWallsInNeighbourhood(dtPolyRef startRef, const dtReal* centerPos, const dtReal radius,
									  const dtQueryFilter* filter, 
									  dtPolyRef* neiRefs, int* neiCount, const int maxNei,
									  dtReal* resultWalls, dtPolyRef* resultRefs, int* resultCount, const int maxResult) const;


	/// [UE] Finds the wall segments that overlap the polygon shape.
	NAVMESH_API dtStatus findWallsOverlappingShape(dtPolyRef startRef, const dtReal* verts, const int nverts,
									   const dtQueryFilter* filter, 
									   dtPolyRef* neiRefs, int* neiCount, const int maxNei,
									   dtReal* resultWalls, dtPolyRef* resultRefs, int* resultCount, const int maxResult) const;

	/// [UE] Finds the wall segments that overlap the polygon shape.
	NAVMESH_API dtStatus findWallsAroundPath(const dtPolyRef* path, const int pathCount, const dtReal* searchAreaPoly, const int searchAreaPolyCount,
									   const dtReal maxAreaEnterCost, const dtQueryFilter* filter,
									   dtPolyRef* neiRefs, int* neiCount, const int maxNei,
									   dtReal* resultWalls, dtPolyRef* resultRefs, int* resultCount, const int maxResult) const;
	
	/// Moves from the start to the end position constrained to the navigation mesh.
	///  @param[in]		startRef		The reference id of the start polygon.
	///  @param[in]		startPos		A position of the mover within the start polygon. [(x, y, x)]
	///  @param[in]		endPos			The desired end position of the mover. [(x, y, z)]
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[out]	resultPos		The result position of the mover. [(x, y, z)]
	///  @param[out]	visited			The reference ids of the polygons visited during the move.
	///  @param[out]	visitedCount	The number of polygons visited during the move.
	///  @param[in]		maxVisitedSize	The maximum number of polygons the @p visited array can hold.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus moveAlongSurface(dtPolyRef startRef, const dtReal* startPos, const dtReal* endPos,
							  const dtQueryFilter* filter,
							  dtReal* resultPos, dtPolyRef* visited, int* visitedCount, const int maxVisitedSize) const;
	
	/// Casts a 'walkability' ray along the surface of the navigation mesh from 
	/// the start position toward the end position.
	///  @param[in]		startRef	The reference id of the start polygon.
	///  @param[in]		startPos	A position within the start polygon representing 
	///  							the start of the ray. [(x, y, z)]
	///  @param[in]		endPos		The position to cast the ray toward. [(x, y, z)]
	///  @param[out]	t			The hit parameter. (DT_REAL_MAX if no wall hit.)
	///  @param[out]	hitNormal	The normal of the nearest wall hit. [(x, y, z)]
	///  @param[in]		filter		The polygon filter to apply to the query.
	///  @param[out]	path		The reference ids of the visited polygons. [opt]
	///  @param[out]	pathCount	The number of visited polygons. [opt]
	///  @param[in]		maxPath		The maximum number of polygons the @p path array can hold.
	///  @param[in]		walkableArea Specific area that can be visited or 255 to skip it and test all areas.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus raycast(dtPolyRef startRef, const dtReal* startPos, const dtReal* endPos,
					 const dtQueryFilter* filter,
					 dtReal* t, dtReal* hitNormal, dtPolyRef* path, int* pathCount, const int maxPath) const;
	
	/// Finds the distance from the specified position to the nearest polygon wall.
	///  @param[in]		startRef		The reference id of the polygon containing @p centerPos.
	///  @param[in]		centerPos		The center of the search circle. [(x, y, z)]
	///  @param[in]		maxRadius		The radius of the search circle.
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[out]	hitDist			The distance to the nearest wall from @p centerPos.
	///  @param[out]	hitPos			The nearest position on the wall that was hit. [(x, y, z)]
	///  @param[out]	hitNormal		The normalized ray formed from the wall point to the 
	///  								source point. [(x, y, z)]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findDistanceToWall(dtPolyRef startRef, const dtReal* centerPos, const dtReal maxRadius,
								const dtQueryFilter* filter,
								dtReal* hitDist, dtReal* hitPos, dtReal* hitNormal) const;
	
	/// Returns the segments for the specified polygon, optionally including portals.
	///  @param[in]		ref				The reference id of the polygon.
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[out]	segmentVerts	The segments. [(ax, ay, az, bx, by, bz) * segmentCount]
	///  @param[out]	segmentRefs		The reference ids of each segment's neighbor polygon. 
	///  								Or zero if the segment is a wall. [opt] [(parentRef) * @p segmentCount] 
	///  @param[out]	segmentCount	The number of segments returned.
	///  @param[in]		maxSegments		The maximum number of segments the result arrays can hold.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus getPolyWallSegments(dtPolyRef ref, const dtQueryFilter* filter,
								 dtReal* segmentVerts, dtPolyRef* segmentRefs, int* segmentCount,
								 const int maxSegments) const;

	/// Returns random location on navmesh.
	/// Polygons are chosen weighted by area. The search runs in linear related to number of polygon.
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[in]		frand			Function returning a random number [0..1).
	///  @param[out]	randomRef		The reference id of the random location.
	///  @param[out]	randomPt		The random location. 
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findRandomPoint(const dtQueryFilter* filter, float(*frand)(),
							 dtPolyRef* randomRef, dtReal* randomPt) const;

	/// Returns random location on navmesh within the reach of specified location.
	/// Polygons are chosen weighted by area. The search runs in linear related to number of polygon.
	/// The location is not exactly constrained by the circle, but it limits the visited polygons.
	///  @param[in]		startRef		The reference id of the polygon where the search starts.
	///  @param[in]		centerPos		The center of the search circle. [(x, y, z)]
	///  @param[in]		filter			The polygon filter to apply to the query.
	///  @param[in]		frand			Function returning a random number [0..1).
	///  @param[out]	randomRef		The reference id of the random location.
	///  @param[out]	randomPt		The random location. [(x, y, z)]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findRandomPointAroundCircle(dtPolyRef startRef, const dtReal* centerPos, const dtReal maxRadius,
										 const dtQueryFilter* filter, float(*frand)(),
										 dtPolyRef* randomRef, dtReal* randomPt) const;

	/// Returns random location inside the specified polygon.
	///  @param[in]		ref				The reference id of the polygon.
	///  @param[in]		frand			Function returning a random number [0..1).
	///  @param[out]	randomPt		The random location. [(x, y, z)]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findRandomPointInPoly(dtPolyRef ref, float(*frand)(), dtReal* randomPt) const;

	//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
	/// Check if there is a path from start polygon to the end polygon using cluster graph
	/// (cheap, does not care about link costs)
	///  @param[in]		startRef			The reference id of the start polygon.
	///  @param[in]		endRef				The reference id of the end polygon.
	NAVMESH_API dtStatus testClusterPath(dtPolyRef startRef, dtPolyRef endRef) const; 

	/// Returns random location on navmesh within specified cluster.
	///  @param[in]		frand			Function returning a random number [0..1).
	///  @param[out]	randomRef		The reference id of the random location.
	///  @param[out]	randomPt		The random location. 
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus findRandomPointInCluster(dtClusterRef clusterRef, float(*frand)(),
									  dtPolyRef* randomRef, dtReal* randomPt) const;

	/// Gets the cluster containing given polygon
	///  @param[in]		polyRef		The reference id of the polygon.
	///  @param[out]	clusterRef	The reference id of the cluster
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus getPolyCluster(dtPolyRef polyRef, dtClusterRef& clusterRef) const;
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END
	
	/// Finds the closest point on the specified polygon.
	///  @param[in]		ref			The reference id of the polygon.
	///  @param[in]		pos			The position to check. [(x, y, z)]
	///  @param[out]	closest		The closest point on the polygon. [(x, y, z)]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus closestPointOnPoly(dtPolyRef ref, const dtReal* pos, dtReal* closest) const;
	
	/// Returns a point on the boundary closest to the source point if the source point is outside the 
	/// polygon's xz-bounds.
	///  @param[in]		ref			The reference id to the polygon.
	///  @param[in]		pos			The position to check. [(x, y, z)]
	///  @param[out]	closest		The closest point. [(x, y, z)]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus closestPointOnPolyBoundary(dtPolyRef ref, const dtReal* pos, dtReal* closest) const;

	/// Finds the point's projection on the specified polygon.
	///  @param[in]		ref			The reference id of the polygon.
	///  @param[in]		pos			The position to check. [(x, y, z)]
	///  @param[out]	closest		The projected point on the polygon. [(x, y, z)]
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus projectedPointOnPoly(dtPolyRef ref, const dtReal* pos, dtReal* projected) const;
	
	/// Checks if specified pos is inside given polygon specified by ref
	///  @param[in]		ref			The reference id of the polygon.
	///  @param[in]		pos			The position to check. [(x, y, z)]
	///  @param[out]	result		The result of the check, whether the point is inside (true) or not (false)
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus isPointInsidePoly(dtPolyRef ref, const dtReal* pos, bool& result) const;

	/// Gets the height of the polygon at the provided position using the height detail. (Most accurate.)
	///  @param[in]		ref			The reference id of the polygon.
	///  @param[in]		pos			A position within the xz-bounds of the polygon. [(x, y, z)]
	///  @param[out]	height		The height at the surface of the polygon.
	/// @returns The status flags for the query.
	NAVMESH_API dtStatus getPolyHeight(dtPolyRef ref, const dtReal* pos, dtReal* height) const;

	/// @}
	/// @name Miscellaneous Functions
	/// @{

	/// Returns true if the polygon reference is valid and passes the filter restrictions.
	///  @param[in]		ref			The polygon reference to check.
	///  @param[in]		filter		The filter to apply.
	NAVMESH_API bool isValidPolyRef(dtPolyRef ref, const dtQueryFilter* filter) const;

	/// Returns true if the polygon reference is in the closed list. 
	///  @param[in]		ref		The reference id of the polygon to check.
	/// @returns True if the polygon is in closed list.
	NAVMESH_API bool isInClosedList(dtPolyRef ref) const;
	
	/// Returns true if the cluster link was used in previous search. 
	///  @param[in]		cFrom		The reference id of the start cluster.
	///  @param[in]		cto			The reference id of the end cluster.
	/// @returns True if the cluster link was searched.
	NAVMESH_API bool wasClusterLinkSearched(dtPolyRef cFrom, dtPolyRef cTo) const;

	/// Gets the node pool.
	/// @returns The node pool.
	class dtNodePool* getNodePool() const { return m_nodePool; }
	
	/// Gets the navigation mesh the query object is using.
	/// @return The navigation mesh the query object is using.
	const dtNavMesh* getAttachedNavMesh() const { return m_nav; }

	/// Gets best node ref and cost from sliced pathfinding data
	void getCurrentBestResult(struct dtNode*& bestNode, dtReal& bestCost) const { bestNode = m_query.lastBestNode; bestCost = m_query.lastBestNodeCost; }

	int getQueryNodes() const { return m_queryNodes; }
	/// @}
	
private:
	
	/// Returns neighbour tile based on side.
	NAVMESH_API dtMeshTile* getNeighbourTileAt(int x, int y, int side) const;

	/// Queries polygons within a tile.
	NAVMESH_API int queryPolygonsInTile(const dtMeshTile* tile, const dtReal* qmin, const dtReal* qmax, const dtQueryFilter* filter,
							dtPolyRef* polys, const int maxPolys) const;
	/// Find nearest polygon within a tile.
	NAVMESH_API dtPolyRef findNearestPolyInTile(const dtMeshTile* tile, const dtReal* center, const dtReal* extents,
									const dtQueryFilter* filter, dtReal* nearestPt) const;
	/// Returns closest point on polygon.
	NAVMESH_API void closestPointOnPolyInTile(const dtMeshTile* tile, const dtPoly* poly, const dtReal* pos, dtReal* closest) const;

	/// Returns projected point on polygon.
	NAVMESH_API dtStatus projectedPointOnPolyInTile(const dtMeshTile* tile, const dtPoly* poly, const dtReal* pos, dtReal* projected) const;
	
	//@UE BEGIN
	/// Handle 0 extents by having the vector components set to a minimum value.
	static void dtApplyEpsilon(dtReal* extents);
	
	// exposing function to be able to generate navigation corridors as sequence of point pairs
public:
	/// Returns portal points between two polygons.
	NAVMESH_API dtStatus getPortalPoints(dtPolyRef from, dtPolyRef to, dtReal* left, dtReal* right,
							 unsigned char& fromType, unsigned char& toType) const;
	NAVMESH_API dtStatus getPortalPoints(dtPolyRef from, const dtPoly* fromPoly, const dtMeshTile* fromTile,
							 dtPolyRef to, const dtPoly* toPoly, const dtMeshTile* toTile,
							 dtReal* left, dtReal* right) const;
	
	/// Returns edge mid point between two polygons.
	NAVMESH_API dtStatus getEdgeMidPoint(dtPolyRef from, dtPolyRef to, dtReal* mid) const;
	NAVMESH_API dtStatus getEdgeMidPoint(dtPolyRef from, const dtPoly* fromPoly, const dtMeshTile* fromTile,
							 dtPolyRef to, const dtPoly* toPoly, const dtMeshTile* toTile,
							 dtReal* mid) const;
	//@UE END

	// Appends vertex to a straight path
	NAVMESH_API dtStatus appendVertex(const dtReal* pos, const unsigned char flags, const dtPolyRef ref,
						  dtQueryResult& result, const bool bOverrideIdenticalPosition = true) const;

	// Appends intermediate portal points to a straight path.
	NAVMESH_API dtStatus appendPortals(const int startIdx, const int endIdx, const dtReal* endPos, const dtPolyRef* path,
						   dtQueryResult& result, const int options) const;
	
	inline bool passLinkFilterByRef(const dtMeshTile* tile, const dtPolyRef ref) const
	{
		return passLinkFilter(tile, m_nav->decodePolyIdPoly(ref));
	}

	inline bool passLinkFilter(const dtMeshTile* tile, const int polyIdx) const
	{
		const int linkIdx = polyIdx - tile->header->offMeshBase;

		return !(m_linkFilter && polyIdx >= tile->header->offMeshBase
			&& linkIdx < tile->header->offMeshConCount
			&& tile->offMeshCons[linkIdx].userId != 0
			&& m_linkFilter->isLinkAllowed(tile->offMeshCons[linkIdx].userId) == false);
	}

	//@UE BEGIN
	inline void setRequireNavigableEndLocation(const bool value) { m_query.requireNavigableEndLocation = value; }
	inline bool isRequiringNavigableEndLocation() const { return m_query.requireNavigableEndLocation; }
	//@UE END

private:
	const dtNavMesh* m_nav;							///< Pointer to navmesh data.
	dtQuerySpecialLinkFilter* m_linkFilter;			///< Pointer to optional special link filter

	struct dtQueryData
	{
		dtStatus status;
		struct dtNode* lastBestNode;
		dtReal lastBestNodeCost;
		dtPolyRef startRef, endRef;
		dtReal startPos[3], endPos[3];
		dtReal costLimit; 					//@UE ///< Cost limit of nodes allowed to be added to the open list
		const dtQueryFilter* filter;
		unsigned char requireNavigableEndLocation : 1;	//@UE
	};
	dtQueryData m_query;				///< Sliced query state.

	class dtNodePool* m_tinyNodePool;	///< Pointer to small node pool.
	class dtNodePool* m_nodePool;		///< Pointer to node pool.
	class dtNodeQueue* m_openList;		///< Pointer to open list queue.

	mutable int m_queryNodes;
};

/// Allocates a query object using the Detour allocator.
/// @return An allocated query object, or null on failure.
/// @ingroup detour
NAVMESH_API dtNavMeshQuery* dtAllocNavMeshQuery();

/// Frees the specified query object using the Detour allocator.
///  @param[in]		query		A query object allocated using #dtAllocNavMeshQuery
/// @ingroup detour
NAVMESH_API void dtFreeNavMeshQuery(dtNavMeshQuery* query);

#endif // DETOURNAVMESHQUERY_H
