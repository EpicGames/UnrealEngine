// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURNODE_H
#define DETOURNODE_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavMesh.h"

enum dtNodeFlags
{
	DT_NODE_OPEN = 0x01,
	DT_NODE_CLOSED = 0x02,
};

typedef unsigned short dtNodeIndex;
static const dtNodeIndex DT_NULL_IDX = (dtNodeIndex)~0;

struct dtNode
{
	dtReal pos[3];				///< Position of the node.
	dtReal cost;				///< Cost from previous node to current node.
	dtReal total;				///< Cost up to the node.
	unsigned int pidx : 30;		///< Index to parent node.
	unsigned int flags : 2;		///< Node flags 0/open/closed.
	dtPolyRef id;				///< Polygon ref the node corresponds to.
};


class dtNodePool
{
public:
	dtNodePool(int maxNodes, int hashSize);
	~dtNodePool();
	inline void operator=(const dtNodePool&) {}
	void clear();
	dtNode* getNode(dtPolyRef id);
	dtNode* findNode(dtPolyRef id);

	inline unsigned int getNodeIdx(const dtNode* node) const
	{
		if (!node) return 0;
		return (unsigned int)(node - m_nodes)+1;
	}

	inline dtNode* getNodeAtIdx(unsigned int idx)
	{
		if (!idx) return 0;
		return &m_nodes[idx-1];
	}

	inline const dtNode* getNodeAtIdx(unsigned int idx) const
	{
		if (!idx) return 0;
		return &m_nodes[idx-1];
	}
	
	inline int getMemUsed() const
	{
		return sizeof(*this) +
			sizeof(dtNode)*m_maxNodes +
			sizeof(dtNodeIndex)*m_maxNodes +
			sizeof(dtNodeIndex)*m_hashSize;
	}
	
	inline int getMaxNodes() const { return m_maxNodes; }
	//@UE BEGIN
	// If using a shared query instance it's possible that m_maxNodes is greater
	// than pool size requested by callee. There's no point in reallocating the
	// pool so we artificially limit the number of available nodes
	inline int getMaxRuntimeNodes() const { return m_maxRuntimeNodes; }
	//@UE END
	inline int getNodeCount() const { return m_nodeCount; }
	
	inline int getHashSize() const { return m_hashSize; }
	inline dtNodeIndex getFirst(int bucket) const { return m_first[bucket]; }
	inline dtNodeIndex getNext(int i) const { return m_next[i]; }

	//@UE BEGIN
	// overrides m_maxNodes for runtime purposes
	inline void setMaxRuntimeNodes(const int newMaxRuntimeNodes) { m_maxRuntimeNodes = newMaxRuntimeNodes; }
	//@UE END

private:
	
	dtNode* m_nodes;
	dtNodeIndex* m_first;
	dtNodeIndex* m_next;
	const int m_maxNodes;
	const int m_hashSize;
	//@UE BEGIN
	int m_maxRuntimeNodes;
	//@UE END
	int m_nodeCount;
};

class dtNodeQueue
{
public:
	dtNodeQueue(int n);
	~dtNodeQueue();
	inline void operator=(dtNodeQueue&) {}
	
	inline void clear()
	{
		m_size = 0;
	}
	
	inline dtNode* top()
	{
		return m_heap[0];
	}
	
	inline dtNode* pop()
	{
		dtNode* result = m_heap[0];
		m_size--;
		trickleDown(0, m_heap[m_size]);
		return result;
	}
	
	inline void push(dtNode* node)
	{
		m_size++;
		bubbleUp(m_size-1, node);
	}
	
	inline void modify(dtNode* node)
	{
		for (int i = 0; i < m_size; ++i)
		{
			if (m_heap[i] == node)
			{
				bubbleUp(i, node);
				return;
			}
		}
	}
	
	inline bool empty() const { return m_size == 0; }
	
	inline int getMemUsed() const
	{
		return sizeof(*this) +
		sizeof(dtNode*)*(m_capacity+1);
	}
	
	inline int getCapacity() const { return m_capacity; }
	
private:
	void bubbleUp(int i, dtNode* node);
	void trickleDown(int i, dtNode* node);
	
	dtNode** m_heap;
	const int m_capacity;
	int m_size;
};		


#endif // DETOURNODE_H
