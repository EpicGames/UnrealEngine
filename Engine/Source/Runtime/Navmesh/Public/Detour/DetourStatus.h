// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURSTATUS_H
#define DETOURSTATUS_H

#include "CoreMinimal.h"

typedef unsigned int dtStatus;

// High level status.
static const unsigned int DT_FAILURE = 1u << 31;			// Operation failed.
static const unsigned int DT_SUCCESS = 1u << 30;			// Operation succeed.
static const unsigned int DT_IN_PROGRESS = 1u << 29;		// Operation still in progress.

// Detail information for status.
static const unsigned int DT_STATUS_DETAIL_MASK = 0x0ffffff;
//static const unsigned int DT_WRONG_MAGIC = 1 << 0;		// Input data is not recognized. NOTE: UE magic removed to save memory
static const unsigned int DT_WRONG_VERSION = 1 << 1;	// Input data is in wrong version.
static const unsigned int DT_OUT_OF_MEMORY = 1 << 2;	// Operation ran out of memory.
static const unsigned int DT_INVALID_PARAM = 1 << 3;	// An input parameter was invalid.
static const unsigned int DT_BUFFER_TOO_SMALL = 1 << 4;	// Result buffer for the query was too small to store all results.
static const unsigned int DT_OUT_OF_NODES = 1 << 5;		// Query ran out of nodes during search.
static const unsigned int DT_PARTIAL_RESULT = 1 << 6;	// Query did not reach the end location, returning best guess. 
static const unsigned int DT_INVALID_CYCLE_PATH = 1 << 7;	// Query found a path containing a cycle


// Returns true of status is success.
inline bool dtStatusSucceed(dtStatus status)
{
	return (status & DT_SUCCESS) != 0;
}

// Returns true of status is failure.
inline bool dtStatusFailed(dtStatus status)
{
	return (status & DT_FAILURE) != 0;
}

// Returns true of status is in progress.
inline bool dtStatusInProgress(dtStatus status)
{
	return (status & DT_IN_PROGRESS) != 0;
}

// Returns true if specific detail is set.
inline bool dtStatusDetail(dtStatus status, unsigned int detail)
{
	return (status & detail) != 0;
}

#endif // DETOURSTATUS_H
