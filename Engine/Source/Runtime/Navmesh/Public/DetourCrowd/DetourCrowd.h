// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURCROWD_H
#define DETOURCROWD_H

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavMesh.h"
#include "Detour/DetourNavMeshQuery.h"
#include "DetourCrowd/DetourLocalBoundary.h"
#include "DetourCrowd/DetourObstacleAvoidance.h"
#include "DetourCrowd/DetourPathCorridor.h"
#include "DetourCrowd/DetourPathQueue.h"
#include "DetourCrowd/DetourSharedBoundary.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class FString;
class dtProximityGrid;

/// The maximum number of neighbors that a crowd agent can take into account
/// for steering decisions.
/// @ingroup crowd
static const int DT_CROWDAGENT_MAX_NEIGHBOURS = 6;

/// The maximum number of corners a crowd agent will look ahead in the path.
/// This value is used for sizing the crowd agent corner buffers.
/// Due to the behavior of the crowd manager, the actual number of useful
/// corners will be one less than this number.
/// @ingroup crowd
static const int DT_CROWDAGENT_MAX_CORNERS = 4;

/// The maximum number of crowd avoidance configurations supported by the
/// crowd manager.
/// @ingroup crowd
/// @see dtObstacleAvoidanceParams, dtCrowd::setObstacleAvoidanceParams(), dtCrowd::getObstacleAvoidanceParams(),
///		 dtCrowdAgentParams::obstacleAvoidanceType
static const int DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS = 8;

/// UE: The maximum number of unique filters used by crowd agents 
static const int DT_CROWD_MAX_FILTERS = 16;

/// Provides neighbor data for agents managed by the crowd.
/// @ingroup crowd
/// @see dtCrowdAgent::neis, dtCrowd
struct dtCrowdNeighbour
{
	int idx;		///< The index of the neighbor in the crowd.
	dtReal dist;		///< The distance between the current agent and the neighbor.
};

/// The type of navigation mesh polygon the agent is currently traversing.
/// @ingroup crowd
enum CrowdAgentState
{
	DT_CROWDAGENT_STATE_INVALID,		///< The agent is not in a valid state.
	DT_CROWDAGENT_STATE_WALKING,		///< The agent is traversing a normal navigation mesh polygon.
	DT_CROWDAGENT_STATE_OFFMESH,		///< The agent is traversing an off-mesh connection.
	DT_CROWDAGENT_STATE_WAITING,		///< [UE] The agent is waiting for external movement to finish
};

/// Configuration parameters for a crowd agent.
/// @ingroup crowd
struct dtCrowdAgentParams
{
	/// User defined data attached to the agent.
	void* userData;

	/// UE: special link filter used by this agent
	TSharedPtr<dtQuerySpecialLinkFilter> linkFilter;

	dtReal radius;						///< Agent radius. [Limit: >= 0]
	dtReal height;						///< Agent height. [Limit: > 0]
	dtReal maxAcceleration;				///< Maximum allowed acceleration. [Limit: >= 0]
	dtReal maxSpeed;					///< Maximum allowed speed. [Limit: >= 0]

	/// Defines how close a collision element must be before it is considered for steering behaviors. [Limits: > 0]
	dtReal collisionQueryRange;

	dtReal pathOptimizationRange;		///< The path visibility optimization range. [Limit: > 0]

	/// How aggresive the agent manager should be at avoiding collisions with this agent. [Limit: >= 0]
	dtReal separationWeight;

	/// [UE] Mutliplier for avoidance velocities
	dtReal avoidanceQueryMultiplier;

	/// [UE] Groups flags attached to the agent
	unsigned int avoidanceGroup;
	/// [UE] Avoid agents when they group is matching mask
	unsigned int groupsToAvoid;
	/// [UE] Don't avoid agents when they group is matching mask
	unsigned int groupsToIgnore;

	/// Flags that impact steering behavior. (See: #UpdateFlags)
	unsigned short updateFlags;

	/// The index of the avoidance configuration to use for the agent. 
	/// [Limits: 0 <= value <= #DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS]
	unsigned char obstacleAvoidanceType;	

	/// UE: Id of navigation filter used by this agent
	/// [Limits: 0 <= value <= #DT_CROWD_MAX_FILTERS]
	unsigned char filter;
};

enum MoveRequestState
{
	DT_CROWDAGENT_TARGET_NONE = 0,
	DT_CROWDAGENT_TARGET_FAILED,
	DT_CROWDAGENT_TARGET_VALID,
	DT_CROWDAGENT_TARGET_REQUESTING,
	DT_CROWDAGENT_TARGET_WAITING_FOR_QUEUE,
	DT_CROWDAGENT_TARGET_WAITING_FOR_PATH,
	DT_CROWDAGENT_TARGET_VELOCITY,
};

/// Represents an agent managed by a #dtCrowd object.
/// @ingroup crowd
struct dtCrowdAgent
{
	/// The path corridor the agent is using.
	dtPathCorridor corridor;

	/// The local boundary data for the agent.
	dtLocalBoundary boundary;
	
	/// Time since the agent's path corridor was optimized.
	dtReal topologyOptTime;
	
	/// The known neighbors of the agent.
	dtCrowdNeighbour neis[DT_CROWDAGENT_MAX_NEIGHBOURS];

	/// The desired speed.
	dtReal desiredSpeed;

	dtReal npos[3];		///< The current agent position. [(x, y, z)]
	dtReal disp[3];
	dtReal dvel[3];		///< The desired velocity of the agent. [(x, y, z)]
	dtReal nvel[3];
	dtReal vel[3];		///< The actual velocity of the agent. [(x, y, z)]

	/// The agent's configuration parameters.
	dtCrowdAgentParams params;

	/// The local path corridor corners for the agent. (Staight path.) [(x, y, z) * #ncorners]
	dtReal cornerVerts[DT_CROWDAGENT_MAX_CORNERS*3];

	/// The reference id of the polygon being entered at the corner. [(polyRef) * #ncorners]
	dtPolyRef cornerPolys[DT_CROWDAGENT_MAX_CORNERS];

	dtReal targetReplanTime;				/// <Time since the agent's target was replanned.
	dtReal targetPos[3];					///< Target position of the movement request (or velocity in case of DT_CROWDAGENT_TARGET_VELOCITY).
	dtPolyRef targetRef;				///< Target polyref of the movement request.
	dtPathQueueRef targetPathqRef;		///< Path finder ref.

	/// The number of neighbors.
	int nneis;

	/// The local path corridor corner flags. (See: #dtStraightPathFlags) [(flags) * #ncorners]
	unsigned char cornerFlags[DT_CROWDAGENT_MAX_CORNERS];

	/// The number of corners.
	int ncorners;
	
	unsigned char targetReplan;			///< Flag indicating that the current path is being replanned.
	unsigned char targetState;			///< State of the movement request.

	/// 1 if the agent is active, or 0 if the agent is in an unused slot in the agent pool.
	unsigned char active;

	/// The type of mesh polygon the agent is traversing. (See: #CrowdAgentState)
	unsigned char state;
};

struct dtCrowdAgentAnimation
{
	dtReal initPos[3], startPos[3], endPos[3];
	dtPolyRef polyRef;
	dtReal t, tmax;
	unsigned char active;
};

/// Crowd agent update flags.
/// @ingroup crowd
/// @see dtCrowdAgentParams::updateFlags
enum UpdateFlags
{
	DT_CROWD_ANTICIPATE_TURNS		= 1 << 0,
	DT_CROWD_OBSTACLE_AVOIDANCE		= 1 << 1,
	DT_CROWD_SEPARATION				= 1 << 2,
	DT_CROWD_OPTIMIZE_VIS			= 1 << 3,	///< Use #dtPathCorridor::optimizePathVisibility() to optimize the agent path.
	DT_CROWD_OPTIMIZE_TOPO			= 1 << 4,	///< Use dtPathCorridor::optimizePathTopology() to optimize the agent path.
	DT_CROWD_OPTIMIZE_VIS_MULTI		= 1 << 5,	///< [UE] Multiple calls for optimizePathVisibility instead of checking last point
	DT_CROWD_OFFSET_PATH			= 1 << 6,	///< [UE] Offset path points from corners by agent radius
	DT_CROWD_SLOWDOWN_AT_GOAL		= 1 << 7,	///< [UE] Slowdown before reaching goal
};

// [UE] Flags used by boundary segments (dtLocalBoundary::Segment)
enum CrowdBoundaryFlags
{
	DT_CROWD_BOUNDARY_IGNORE = 1 << 0,
};

struct dtCrowdAgentDebugInfo
{
	int idx;
	dtReal optStart[3], optEnd[3];
	dtObstacleAvoidanceDebugData* vod;
	TMap<int32, FString> agentLog;
};

/// Provides local steering behaviors for a group of agents. 
/// @ingroup crowd
class dtCrowd
{
	int m_maxAgents;
	int m_numActiveAgents;
	dtCrowdAgent* m_agents;
	dtCrowdAgent** m_activeAgents;
	dtCrowdAgentAnimation* m_agentAnims;
	
	dtPathQueue m_pathq;
	dtSharedBoundary m_sharedBoundary;

	dtObstacleAvoidanceParams m_obstacleQueryParams[DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS];
	dtObstacleAvoidanceQuery* m_obstacleQuery;
	
	dtProximityGrid* m_grid;
	
	dtPolyRef* m_pathResult;
	
	dtReal m_ext[3];
	dtQueryFilter m_filters[DT_CROWD_MAX_FILTERS];
	dtQueryFilter m_raycastFilter;
	
	dtReal m_maxAgentRadius;

	// [UE] time between attempts to restore agents state
	dtReal m_agentStateCheckInterval;

	// [UE] radius multiplier for offseting path around corners
	dtReal m_pathOffsetRadiusMultiplier;

	// [UE] separation filter
	dtReal m_separationDirFilter;

	int m_maxPathResult;

	int m_velocitySampleCount;

	dtNavMeshQuery* m_navquery;

	// [UE] if set, path visibility optimization can't leave current area type
	bool m_raycastSingleArea;
	// [UE] if set, offmesh connections won't be cut from corridor
	bool m_keepOffmeshConnections;
	// [UE] if set, crowd agents will use early reach test
	bool m_earlyReachTest;

	NAVMESH_API void updateTopologyOptimization(dtCrowdAgent** agents, const int nagents, const dtReal dt);
	NAVMESH_API void updateMoveRequest(const dtReal dt);
	NAVMESH_API void checkPathValidity(dtCrowdAgent** agents, const int nagents, const dtReal dt);

	NAVMESH_API bool requestMoveTargetReplan(const int idx, dtPolyRef ref, const dtReal* pos);

	NAVMESH_API void purge();
	
public:
	NAVMESH_API dtCrowd();
	NAVMESH_API ~dtCrowd();
	
	/// Initializes the crowd.  
	///  @param[in]		maxAgents		The maximum number of agents the crowd can manage. [Limit: >= 1]
	///  @param[in]		maxAgentRadius	The maximum radius of any agent that will be added to the crowd. [Limit: > 0]
	///  @param[in]		nav				The navigation mesh to use for planning.
	/// @return True if the initialization succeeded.
	NAVMESH_API bool init(const int maxAgents, const dtReal maxAgentRadius, dtNavMesh* nav);

	/// [UE] Initializes the avoidance query.  
	///  @param[in]		maxNeighbors		The maximum number of processed neighbors
	///  @param[in]		maxWalls			The maximum number of processed wall segments
	///  @param[in]		maxCustomPatterns	The maximum number of custom sampling patterns
	/// @return True if the initialization succeeded.
	NAVMESH_API bool initAvoidance(const int maxNeighbors, const int maxWalls, const int maxCustomPatterns);
	
	/// Sets the shared avoidance configuration for the specified index.
	///  @param[in]		idx		The index. [Limits: 0 <= value < #DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS]
	///  @param[in]		params	The new configuration.
	NAVMESH_API void setObstacleAvoidanceParams(const int idx, const dtObstacleAvoidanceParams* params);

	/// Gets the shared avoidance configuration for the specified index.
	///  @param[in]		idx		The index of the configuration to retreive. 
	///							[Limits:  0 <= value < #DT_CROWD_MAX_OBSTAVOIDANCE_PARAMS]
	/// @return The requested configuration.
	NAVMESH_API const dtObstacleAvoidanceParams* getObstacleAvoidanceParams(const int idx) const;
	
	/// [UE] Sets the shared avoidance sampling pattern for the specified index.
	///  @param[in]		idx			The index.
	///  @param[in]		angles		radians from direction of desired velocity [Count: nsamples]
	///  @param[in]		radii		normalized radii (0..1) for each sample [Count: nsamples]
	///  @param[in]		nsamples	The number of samples
	NAVMESH_API void setObstacleAvoidancePattern(int idx, const dtReal* angles, const dtReal* radii, int nsamples);

	/// [UE] Gets the shared avoidance sampling pattern for the specified index.
	///  @param[in]		idx			The index.
	///  @param[in]		angles		radians from direction of desired velocity [Count: nsamples]
	///  @param[in]		radii		normalized radii (0..1) for each sample [Count: nsamples]
	///  @param[in]		nsamples	The number of samples
	/// @return true if pattern was found
	NAVMESH_API bool getObstacleAvoidancePattern(int idx, dtReal* angles, dtReal* radii, int* nsamples);

	/// Gets the specified agent from the pool.
	///	 @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	/// @return The requested agent.
	NAVMESH_API const dtCrowdAgent* getAgent(const int idx);

	/// The maximum number of agents that can be managed by the object.
	/// @return The maximum number of agents.
	NAVMESH_API const int getAgentCount() const;
	
	/// Adds a new agent to the crowd.
	///  @param[in]		pos		The requested position of the agent. [(x, y, z)]
	///  @param[in]		params	The configutation of the agent.
	///  @param[in]		filter	[UE] query filter used by agent
	/// @return The index of the agent in the agent pool. Or -1 if the agent could not be added.
	NAVMESH_API int addAgent(const dtReal* pos, const dtCrowdAgentParams& params, const dtQueryFilter* filter);

	/// Updates the specified agent's configuration.
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	///  @param[in]		params	The new agent configuration.
	NAVMESH_API void updateAgentParameters(const int idx, const dtCrowdAgentParams& params);

	/// [UE] Updates the specified agent's query filter.
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	///  @param[in]		filter	The new filter.
	/// @return True if the request was successfully submitted.
	NAVMESH_API bool updateAgentFilter(const int idx, const dtQueryFilter* filter);

	/// [UE] Refresh state of agent, used after completing movement through offmesh links
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	///  @param[in]		repath	If set, agent will invalidate its path
	NAVMESH_API void updateAgentState(const int idx, bool repath);

	/// Removes the agent from the crowd.
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	NAVMESH_API void removeAgent(const int idx);
	
	/// Submits a new move request for the specified agent.
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	///  @param[in]		ref		The position's polygon reference.
	///  @param[in]		pos		The position within the polygon. [(x, y, z)]
	/// @return True if the request was successfully submitted.
	NAVMESH_API bool requestMoveTarget(const int idx, dtPolyRef ref, const dtReal* pos);

	/// Submits a new move request for the specified agent.
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	///  @param[in]		vel		The movement velocity. [(x, y, z)]
	/// @return True if the request was successfully submitted.
	NAVMESH_API bool requestMoveVelocity(const int idx, const dtReal* vel);

	/// Resets any request for the specified agent.
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	/// @return True if the request was successfully reseted.
	NAVMESH_API bool resetMoveTarget(const int idx);

	/// [UE] Switch to waiting state
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	NAVMESH_API bool setAgentWaiting(const int idx);

	/// [UE] Switch to offmesh link state
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	NAVMESH_API bool setAgentBackOnLink(const int idx);

	/// [UE] Resets agent's velocity
	///  @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	NAVMESH_API bool resetAgentVelocity(const int idx);

	/// Gets the active agents int the agent pool.
	///  @param[out]	agents		An array of agent pointers. [(#dtCrowdAgent *) * maxAgents]
	///  @param[in]		maxAgents	The size of the crowd agent array.
	/// @return The number of agents returned in @p agents.
	NAVMESH_API int getActiveAgents(dtCrowdAgent** agents, const int maxAgents);

	/// Cache list of active agents
	/// @return The number of active agents
	NAVMESH_API int cacheActiveAgents();

	/// Updates the steering and positions of all agents.
	///  @param[in]		dt		The time, in seconds, to update the simulation. [Limit: > 0]
	///  @param[out]	debug	A debug object to load with debug information. [Opt]
	NAVMESH_API void update(const dtReal dt, dtCrowdAgentDebugInfo* debug);
	
	/// [UE] Split update into several smaller components: path validity, path cache and path optimizations
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepPaths(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Split update into several smaller components: neighbors and boundaries
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepProximityData(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Split update into several smaller components: next corner for move, trigger offmesh links
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepNextMovePoint(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Split update into several smaller components: steering
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepSteering(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Split update into several smaller components: avoidance
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepAvoidance(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Split update into several smaller components: integrate velocities and handle collisions
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepMove(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Split update into several smaller components: corridor updates at new position
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepCorridor(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Split update into several smaller components: offmesh anims
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepOffMeshAnim(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Split update into several smaller components: offmesh link velocity (instead of playing animation)
	/// @param[in]		dt		Delta time in seconds
	/// @param[in]		nagents	Number of active agents
	NAVMESH_API void updateStepOffMeshVelocity(const dtReal dt, dtCrowdAgentDebugInfo* debug);

	/// [UE] Set time between attempts to restore agents state
	/// @param[in]		t		Time in seconds
	NAVMESH_API void setAgentCheckInterval(const dtReal t);

	/// [UE] Set agent corridor, works only just after requesting move target
	/// when agent didn't start any pathfinding operations yet
	/// Use with caution!
	/// @param[in]		idx		The agent index. [Limits: 0 <= value < #getAgentCount()]
	/// @param[in]		path	The path corridor. [(polyRef) * @p npolys]
	/// @param[in]		npath	The number of polygons in the path.
	NAVMESH_API bool setAgentCorridor(const int idx, const dtPolyRef* path, const int npath);

	/// [UE] Set visibility optimization to use single area raycasts
	/// This will prevent from cutting through polys marked as different area
	/// which could have been avoided in corridor's path
	/// @param[in]		bEnable	New state of single area raycast mode
	NAVMESH_API void setSingleAreaVisibilityOptimization(bool bEnable);

	/// [UE] Set offmesh connection pruning
	/// This will allow removing offmesh connection poly ref from corridor
	/// as soon as offmesh connection anim is triggered (default behavior)
	NAVMESH_API void setPruneStartedOffmeshConnections(bool bRemoveFromCorridor);

	/// [UE]
	NAVMESH_API void setEarlyReachTestOptimization(bool bEnable);

	/// [UE] Set agent radius multiplier for offseting path from corners
	NAVMESH_API void setPathOffsetRadiusMultiplier(dtReal RadiusMultiplier);

	/// [UE] Set separation filter param
	NAVMESH_API void setSeparationFilter(dtReal InFilter);

	/// [UE] Check if agent moved away from its path corridor
	NAVMESH_API bool isOutsideCorridor(const int idx) const;

	/// Gets the filter used by the crowd.
	///  @param[in]		idx		[UE] The agent index. [Limits: 0 <= value < #getAgentCount()]
	/// @return The filter used by the crowd.
	NAVMESH_API const dtQueryFilter* getFilter(const int idx) const;

	/// Gets the filter used by the crowd.
	///  @param[in]		idx		[UE] The agent index. [Limits: 0 <= value < #getAgentCount()]
	/// @return The filter used by the crowd.
	NAVMESH_API dtQueryFilter* getEditableFilter(const int idx);

	/// Gets the search extents [(x, y, z)] used by the crowd for query operations. 
	/// @return The search extents used by the crowd. [(x, y, z)]
	const dtReal* getQueryExtents() const { return m_ext; }
	
	/// Gets the velocity sample count.
	/// @return The velocity sample count.
	inline int getVelocitySampleCount() const { return m_velocitySampleCount; }
	
	/// Gets the crowd's proximity grid.
	/// @return The crowd's proximity grid.
	const dtProximityGrid* getGrid() const { return m_grid; }

	/// Gets the crowd's path request queue.
	/// @return The crowd's path request queue.
	const dtPathQueue* getPathQueue() const { return &m_pathq; }

	/// Gets the query object used by the crowd.
	const dtNavMeshQuery* getNavMeshQuery() const { return m_navquery; }

	/// Gets shared boundary cache
	const dtSharedBoundary* getSharedBoundary() const { return &m_sharedBoundary; }

	/// Gets all cached active agents 
	dtCrowdAgent** getActiveAgents() const { return m_activeAgents; }

	inline int getNumActiveAgents() const { return m_numActiveAgents; }

	inline int getAgentIndex(const dtCrowdAgent* agent) const  { return (int)(agent - m_agents); }

	/// Gets all agent animations
	const dtCrowdAgentAnimation* getAgentAnims() const { return m_agentAnims;  }
};

/// Allocates a crowd object using the Detour allocator.
/// @return A crowd object that is ready for initialization, or null on failure.
///  @ingroup crowd
NAVMESH_API dtCrowd* dtAllocCrowd();

/// Frees the specified crowd object using the Detour allocator.
///  @param[in]		ptr		A crowd object allocated using #dtAllocCrowd
///  @ingroup crowd
NAVMESH_API void dtFreeCrowd(dtCrowd* ptr);


#endif // DETOURCROWD_H

///////////////////////////////////////////////////////////////////////////

// This section contains detailed documentation for members that don't have
// a source file. It reduces clutter in the main section of the header.

/**

@defgroup crowd Crowd

Members in this module implement local steering and dynamic avoidance features.

The crowd is the big beast of the navigation features. It not only handles a 
lot of the path management for you, but also local steering and dynamic 
avoidance between members of the crowd. I.e. It can keep your agents from 
running into each other.

Main class: #dtCrowd

The #dtNavMeshQuery and #dtPathCorridor classes provide perfectly good, easy 
to use path planning features. But in the end they only give you points that 
your navigation client should be moving toward. When it comes to deciding things 
like agent velocity and steering to avoid other agents, that is up to you to 
implement. Unless, of course, you decide to use #dtCrowd.

Basically, you add an agent to the crowd, providing various configuration 
settings such as maximum speed and acceleration. You also provide a local 
target to more toward. The crowd manager then provides, with every update, the 
new agent position and velocity for the frame. The movement will be 
constrained to the navigation mesh, and steering will be applied to ensure 
agents managed by the crowd do not collide with each other.

This is very powerful feature set. But it comes with limitations.

The biggest limitation is that you must give control of the agent's position 
completely over to the crowd manager. You can update things like maximum speed 
and acceleration. But in order for the crowd manager to do its thing, it can't 
allow you to constantly be giving it overrides to position and velocity. So 
you give up direct control of the agent's movement. It belongs to the crowd.

The second biggest limitation revolves around the fact that the crowd manager 
deals with local planning. So the agent's target should never be more than 
256 polygons aways from its current position. If it is, you risk 
your agent failing to reach its target. So you may still need to do long 
distance planning and provide the crowd manager with intermediate targets.

Other significant limitations:

- All agents using the crowd manager will use the same #dtQueryFilter.
- Crowd management is relatively expensive. The maximum agents under crowd 
  management at any one time is between 20 and 30.  A good place to start
  is a maximum of 25 agents for 0.5ms per frame.

@note This is a summary list of members.  Use the index or search 
feature to find minor members.

@struct dtCrowdAgentParams
@see dtCrowdAgent, dtCrowd::addAgent(), dtCrowd::updateAgentParameters()

@var dtCrowdAgentParams::obstacleAvoidanceType
@par

#dtCrowd permits agents to use different avoidance configurations.  This value 
is the index of the #dtObstacleAvoidanceParams within the crowd.

@see dtObstacleAvoidanceParams, dtCrowd::setObstacleAvoidanceParams(), 
	 dtCrowd::getObstacleAvoidanceParams()

@var dtCrowdAgentParams::collisionQueryRange
@par

Collision elements include other agents and navigation mesh boundaries.

This value is often based on the agent radius and/or maximum speed. E.g. radius * 8

@var dtCrowdAgentParams::pathOptimizationRange
@par

Only applicalbe if #updateFlags includes the #DT_CROWD_OPTIMIZE_VIS flag.

This value is often based on the agent radius. E.g. radius * 30

@see dtPathCorridor::optimizePathVisibility()

@var dtCrowdAgentParams::separationWeight
@par

A higher value will result in agents trying to stay farther away from each other at 
the cost of more difficult steering in tight spaces.

*/
