// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURLOCALBOUNDARY_H
#define DETOURLOCALBOUNDARY_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavMesh.h"

class dtNavMeshQuery;
class dtQueryFilter;
class dtSharedBoundary;

class dtLocalBoundary
{
	static const int MAX_LOCAL_SEGS = 8;
	static const int MAX_LOCAL_POLYS = 16;
	
	struct Segment
	{
		dtReal s[6];	///< Segment start/end
		dtReal d;	///< Distance for pruning.
		int flags;
	};

	dtPolyRef m_polys[MAX_LOCAL_POLYS];
	dtReal m_center[3];
	Segment m_segs[MAX_LOCAL_SEGS];
	int m_nsegs;
	int m_npolys;

	NAVMESH_API void addSegment(const dtReal dist, const dtReal* seg, int flags = 0);
	
public:
	NAVMESH_API dtLocalBoundary();
	NAVMESH_API ~dtLocalBoundary();
	
	NAVMESH_API void reset();

	// [UE: new sections: link removal, path corridor, direction]
	NAVMESH_API void update(dtPolyRef ref, const dtReal* pos, const dtReal collisionQueryRange,
		const bool bIgnoreAtEnd, const dtReal* endPos,
		const dtPolyRef* path, const int npath,
		const dtReal* moveDir,
		dtNavMeshQuery* navquery, const dtQueryFilter* filter);

	NAVMESH_API void update(const dtSharedBoundary* sharedData, const int sharedIdx,
		const dtReal* pos, const dtReal collisionQueryRange,
		const bool bIgnoreAtEnd, const dtReal* endPos,
		const dtPolyRef* path, const int npath, const dtReal* moveDir,
		dtNavMeshQuery* navquery, const dtQueryFilter* filter);
	
	NAVMESH_API bool isValid(dtNavMeshQuery* navquery, const dtQueryFilter* filter);
	
	inline const dtReal* getCenter() const { return m_center; }
	inline int getSegmentCount() const { return m_nsegs; }
	inline const dtReal* getSegment(int i) const { return m_segs[i].s; }
	inline const int getSegmentFlags(int i) const { return m_segs[i].flags; }
};

#endif // DETOURLOCALBOUNDARY_H
