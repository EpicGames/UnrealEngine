// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOUROBSTACLEAVOIDANCE_H
#define DETOUROBSTACLEAVOIDANCE_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"

struct dtObstacleCircle
{
	dtReal p[3];			///< Position of the obstacle
	dtReal vel[3];			///< Velocity of the obstacle
	dtReal dvel[3];			///< Velocity of the obstacle
	dtReal rad;				///< Radius of the obstacle
	dtReal dp[3], np[3];	///< Use for side selection during sampling.
};

struct dtObstacleSegment
{
	dtReal p[3], q[3];		///< End points of the obstacle segment
	unsigned char touch : 1;
	unsigned char canIgnore : 1;
};

class dtObstacleAvoidanceDebugData
{
public:
	NAVMESH_API dtObstacleAvoidanceDebugData();
	NAVMESH_API ~dtObstacleAvoidanceDebugData();
	
	NAVMESH_API bool init(const int maxSamples);
	NAVMESH_API void reset();
	NAVMESH_API void addSample(const dtReal* vel, const dtReal ssize, const dtReal pen,
				   const dtReal vpen, const dtReal vcpen, const dtReal spen, const dtReal tpen);
	
	NAVMESH_API void normalizeSamples();
	
	inline int getSampleCount() const { return m_nsamples; }
	inline const dtReal* getSampleVelocity(const int i) const { return &m_vel[i*3]; }
	inline dtReal getSampleSize(const int i) const { return m_ssize[i]; }
	inline dtReal getSamplePenalty(const int i) const { return m_pen[i]; }
	inline dtReal getSampleDesiredVelocityPenalty(const int i) const { return m_vpen[i]; }
	inline dtReal getSampleCurrentVelocityPenalty(const int i) const { return m_vcpen[i]; }
	inline dtReal getSamplePreferredSidePenalty(const int i) const { return m_spen[i]; }
	inline dtReal getSampleCollisionTimePenalty(const int i) const { return m_tpen[i]; }

private:
	int m_nsamples;
	int m_maxSamples;
	dtReal* m_vel;
	dtReal* m_ssize;
	dtReal* m_pen;
	dtReal* m_vpen;
	dtReal* m_vcpen;
	dtReal* m_spen;
	dtReal* m_tpen;
};

NAVMESH_API dtObstacleAvoidanceDebugData* dtAllocObstacleAvoidanceDebugData();
NAVMESH_API void dtFreeObstacleAvoidanceDebugData(dtObstacleAvoidanceDebugData* ptr);


static const int DT_MAX_PATTERN_DIVS = 32;		///< Max numver of adaptive divs.
static const int DT_MAX_PATTERN_RINGS = 4;		///< Max number of adaptive rings.
static const int DT_MAX_CUSTOM_SAMPLES = 16;	///< Max number of custom samples in single pattern

struct dtObstacleAvoidanceParams
{
	dtReal velBias;
	dtReal weightDesVel;
	dtReal weightCurVel;
	dtReal weightSide;
	dtReal weightToi;
	dtReal horizTime;
	dtReal sideBiasThreshold;	///< Threshold for side selection in prepare(), widens the dead zone for head-on encounters //@UE
	dtReal weightRightBias;		///< Weight for preferring right-side passing (reduces head-on oscillation) //@UE
	unsigned char patternIdx;	///< [UE] index of custom sampling pattern or 0xff for adaptive
	unsigned char adaptiveDivs;	///< adaptive
	unsigned char adaptiveRings;	///< adaptive
	unsigned char adaptiveDepth;	///< adaptive
};

// [UE] custom sampling patterns
struct dtObstacleAvoidancePattern
{
	dtReal angles[DT_MAX_CUSTOM_SAMPLES];	///< sample's angle (radians) from desired velocity direction
	dtReal radii[DT_MAX_CUSTOM_SAMPLES];		///< sample's radius (0...1)
	int nsamples;							///< Number of samples
};

class dtObstacleAvoidanceQuery
{
public:
	NAVMESH_API dtObstacleAvoidanceQuery();
	NAVMESH_API ~dtObstacleAvoidanceQuery();
	
	NAVMESH_API bool init(const int maxCircles, const int maxSegments, const int maxCustomPatterns);
	
	NAVMESH_API void reset();

	NAVMESH_API void addCircle(const dtReal* pos, const dtReal rad,
				   const dtReal* vel, const dtReal* dvel);
				   
	NAVMESH_API void addSegment(const dtReal* p, const dtReal* q, int flags = 0);

	// [UE] store new sampling pattern
	NAVMESH_API bool setCustomSamplingPattern(int idx, const dtReal* angles, const dtReal* radii, int nsamples);

	// [UE] get custom sampling pattern
	NAVMESH_API bool getCustomSamplingPattern(int idx, dtReal* angles, dtReal* radii, int* nsamples);

	// [UE] sample velocity using custom patterns
	NAVMESH_API int sampleVelocityCustom(const dtReal* pos, const dtReal rad,
					 		 const dtReal vmax, const dtReal vmult,
							 const dtReal* vel, const dtReal* dvel, dtReal* nvel,
							 const dtObstacleAvoidanceParams* params,
							 dtObstacleAvoidanceDebugData* debug = 0);

	NAVMESH_API int sampleVelocityAdaptive(const dtReal* pos, const dtReal rad,
							   const dtReal vmax, const dtReal vmult,
							   const dtReal* vel, const dtReal* dvel, dtReal* nvel,
							   const dtObstacleAvoidanceParams* params, 
							   dtObstacleAvoidanceDebugData* debug = 0);
	
	// [UE] main sampling function
	inline int sampleVelocity(const dtReal* pos, const dtReal rad,
		const dtReal vmax, const dtReal vmult,
		const dtReal* vel, const dtReal* dvel, dtReal* nvel,
		const dtObstacleAvoidanceParams* params,
		dtObstacleAvoidanceDebugData* debug = 0)
	{
		return (params->patternIdx == 0xff) ?
			sampleVelocityAdaptive(pos, rad, vmax, vmult, vel, dvel, nvel, params, debug) :
			sampleVelocityCustom(pos, rad, vmax, vmult, vel, dvel, nvel, params, debug);
	}

	inline int getObstacleCircleCount() const { return m_ncircles; }
	const dtObstacleCircle* getObstacleCircle(const int i) { return &m_circles[i]; }

	inline int getObstacleSegmentCount() const { return m_nsegments; }
	const dtObstacleSegment* getObstacleSegment(const int i) { return &m_segments[i]; }

	// [UE] sampling pattern count accessors
	inline int getCustomPatternCount() const { return m_maxPatterns; }

private:

	NAVMESH_API void prepare(const dtReal* pos, const dtReal* dvel);

	NAVMESH_API dtReal processSample(const dtReal* vcand, const dtReal cs,
						const dtReal* pos, const dtReal rad,
						const dtReal* vel, const dtReal* dvel,
						dtObstacleAvoidanceDebugData* debug);

	NAVMESH_API dtObstacleCircle* insertCircle(const dtReal dist);
	NAVMESH_API dtObstacleSegment* insertSegment(const dtReal dist);

	dtObstacleAvoidanceParams m_params;
	dtReal m_invHorizTime;
	dtReal m_vmax;
	dtReal m_invVmax;

	dtObstacleAvoidancePattern* m_customPatterns;
	dtObstacleCircle* m_circles;
	dtObstacleSegment* m_segments;

	int m_maxPatterns;

	int m_maxCircles;
	int m_ncircles;

	int m_maxSegments;
	int m_nsegments;
};

NAVMESH_API dtObstacleAvoidanceQuery* dtAllocObstacleAvoidanceQuery();
NAVMESH_API void dtFreeObstacleAvoidanceQuery(dtObstacleAvoidanceQuery* ptr);


#endif // DETOUROBSTACLEAVOIDANCE_H
