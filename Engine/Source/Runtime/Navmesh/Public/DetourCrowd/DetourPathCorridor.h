// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOUTPATHCORRIDOR_H
#define DETOUTPATHCORRIDOR_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavMesh.h"
#include "HAL/Platform.h"

class dtNavMeshQuery;
class dtQueryFilter;

/// Represents a dynamic polygon corridor used to plan agent movement.
/// @ingroup crowd, detour
class dtPathCorridor
{
	dtReal m_pos[3];
	dtReal m_target[3];
	dtReal m_prevMovePoint[3];
	dtReal m_nextExpectedCorner[3];
	dtReal m_nextExpectedCorner2[3];
	dtReal m_moveSegAngle;
	uint32 m_hasNextExpectedCorner : 1;
	uint32 m_hasNextExpectedCorner2 : 1;
	uint32 m_isInSkipRange : 1;
	uint32 m_enableEarlyReach : 1;

	dtPolyRef* m_path;
	int m_npath;
	int m_maxPath;
	
public:
	NAVMESH_API dtPathCorridor();
	NAVMESH_API ~dtPathCorridor();
	
	/// Allocates the corridor's path buffer. 
	///  @param[in]		maxPath		The maximum path size the corridor can handle.
	/// @return True if the initialization succeeded.
	NAVMESH_API bool init(const int maxPath);
	
	/// Resets the path corridor to the specified position.
	///  @param[in]		ref		The polygon reference containing the position.
	///  @param[in]		pos		The new position in the corridor. [(x, y, z)]
	NAVMESH_API void reset(dtPolyRef ref, const dtReal* pos);
	
	/// Finds the corners in the corridor from the position toward the target. (The straightened path.)
	///  @param[out]	cornerVerts		The corner vertices. [(x, y, z) * cornerCount] [Size: <= maxCorners]
	///  @param[out]	cornerFlags		The flag for each corner. [(flag) * cornerCount] [Size: <= maxCorners]
	///  @param[out]	cornerPolys		The polygon reference for each corner. [(polyRef) * cornerCount] 
	///  								[Size: <= @p maxCorners]
	///  @param[in]		maxCorners		The maximum number of corners the buffers can hold.
	///  @param[in]		navquery		The query object used to build the corridor.
	///  @param[in]		filter			The filter to apply to the operation.
	///  @param[in]		pathOffsetDistance	[UE] Radius for path offsetting
	///  @param[in]		earlyReachDistance	[UE] Radius for early reach detection
	///  @param[in]		bAllowEarlyReach [UE] Check if corner skipping for EarlyReachTest is available now
	/// @return The number of corners returned in the corner buffers. [0 <= value <= @p maxCorners]
	NAVMESH_API int findCorners(dtReal* cornerVerts, unsigned char* cornerFlags,
					dtPolyRef* cornerPolys, const int maxCorners,
					dtNavMeshQuery* navquery, const dtQueryFilter* filter,
					dtReal pathOffsetDistance, dtReal earlyReachDistance, bool bAllowEarlyReach = true);
	
	/// Attempts to optimize the path if the specified point is visible from the current position.
	///  @param[in]		next					The point to search toward. [(x, y, z])
	///  @param[in]		pathOptimizationRange	The maximum range to search. [Limit: > 0]
	///  @param[in]		navquery				The query object used to build the corridor.
	///  @param[in]		filter					The filter to apply to the operation.			
	NAVMESH_API bool optimizePathVisibility(const dtReal* next, const dtReal pathOptimizationRange,
								dtNavMeshQuery* navquery, const dtQueryFilter* filter);
	
	/// Attempts to optimize the path using a local area search. (Partial replanning.) 
	///  @param[in]		navquery	The query object used to build the corridor.
	///  @param[in]		filter		The filter to apply to the operation.	
	NAVMESH_API bool optimizePathTopology(dtNavMeshQuery* navquery, const dtQueryFilter* filter);
	
	NAVMESH_API bool moveOverOffmeshConnection(dtPolyRef offMeshConRef, dtPolyRef* refs,
								   const dtReal* agentPos,
								   dtReal* startPos, dtReal* endPos,
								   dtNavMeshQuery* navquery);

	/// [UE] check if offmesh connection can be traversed, but don't modify corridor yet
	NAVMESH_API bool canMoveOverOffmeshConnection(dtPolyRef offMeshConRef, dtPolyRef* refs,
		const dtReal* agentPos, dtReal* startPos, dtReal* endPos,
		dtNavMeshQuery* navquery) const;

	/// [UE] remove offmesh connection from corridor
	NAVMESH_API void pruneOffmeshConenction(dtPolyRef offMeshConRef);

	NAVMESH_API bool fixPathStart(dtPolyRef safeRef, const dtReal* safePos);

	NAVMESH_API bool trimInvalidPath(dtPolyRef safeRef, const dtReal* safePos,
						 dtNavMeshQuery* navquery, const dtQueryFilter* filter);
	
	/// Checks the current corridor path to see if its polygon references remain valid. 
	///  @param[in]		maxLookAhead	The number of polygons from the beginning of the corridor to search.
	///  @param[in]		navquery		The query object used to build the corridor.
	///  @param[in]		filter			The filter to apply to the operation.	
	NAVMESH_API bool isValid(const int maxLookAhead, dtNavMeshQuery* navquery, const dtQueryFilter* filter);
	
	/// Moves the position from the current location to the desired location, adjusting the corridor 
	/// as needed to reflect the change.
	///  @param[in]		npos		The desired new position. [(x, y, z)]
	///  @param[in]		navquery	The query object used to build the corridor.
	///  @param[in]		filter		The filter to apply to the operation.
	NAVMESH_API bool movePosition(const dtReal* npos, dtNavMeshQuery* navquery, const dtQueryFilter* filter);

	/// Moves the target from the curent location to the desired location, adjusting the corridor
	/// as needed to reflect the change. 
	///  @param[in]		npos		The desired new target position. [(x, y, z)]
	///  @param[in]		navquery	The query object used to build the corridor.
	///  @param[in]		filter		The filter to apply to the operation.
	NAVMESH_API void moveTargetPosition(const dtReal* npos, dtNavMeshQuery* navquery, const dtQueryFilter* filter);
	
	/// Loads a new path and target into the corridor.
	///  @param[in]		target		The target location within the last polygon of the path. [(x, y, z)]
	///  @param[in]		path		The path corridor. [(polyRef) * @p npolys]
	///  @param[in]		npath		The number of polygons in the path.
	NAVMESH_API void setCorridor(const dtReal* target, const dtPolyRef* polys, const int npath);
	
	/// Gets the current position within the corridor. (In the first polygon.)
	/// @return The current position within the corridor.
	inline const dtReal* getPos() const { return m_pos; }

	/// Gets the current target within the corridor. (In the last polygon.)
	/// @return The current target within the corridor.
	inline const dtReal* getTarget() const { return m_target; }
	
	/// The polygon reference id of the first polygon in the corridor, the polygon containing the position.
	/// @return The polygon reference id of the first polygon in the corridor. (Or zero if there is no path.)
	inline dtPolyRef getFirstPoly() const { return m_npath ? m_path[0] : 0; }

	/// The polygon reference id of the last polygon in the corridor, the polygon containing the target.
	/// @return The polygon reference id of the last polygon in the corridor. (Or zero if there is no path.)
	inline dtPolyRef getLastPoly() const { return m_npath ? m_path[m_npath-1] : 0; }
	
	/// The corridor's path.
	/// @return The corridor's path. [(polyRef) * #getPathCount()]
	inline const dtPolyRef* getPath() const { return m_path; }

	/// The number of polygons in the current corridor path.
	/// @return The number of polygons in the current corridor path.
	inline int getPathCount() const { return m_npath; } 	

	inline void setEarlyReachTest(bool enable) { m_enableEarlyReach = enable; }

	inline dtReal getSegmentAngle() const { return m_moveSegAngle; }
	inline const dtReal* getNextFixedCorner() const { return &m_nextExpectedCorner[0]; }
	inline const dtReal* getNextFixedCorner2() const { return &m_nextExpectedCorner2[0]; }
	inline bool hasNextFixedCorner() const { return m_hasNextExpectedCorner; }
	inline bool hasNextFixedCorner2() const { return m_hasNextExpectedCorner2; }
};

int dtMergeCorridorStartMoved(dtPolyRef* path, const int npath, const int maxPath,
							  const dtPolyRef* visited, const int nvisited);

int dtMergeCorridorEndMoved(dtPolyRef* path, const int npath, const int maxPath,
							const dtPolyRef* visited, const int nvisited);

int dtMergeCorridorStartShortcut(dtPolyRef* path, const int npath, const int maxPath,
								 const dtPolyRef* visited, const int nvisited);

#endif // DETOUTPATHCORRIDOR_H
