// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURPATHQUEUE_H
#define DETOURPATHQUEUE_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavMesh.h"
#include "Detour/DetourNavMeshQuery.h"
#include "Detour/DetourStatus.h"
#include "Templates/SharedPointer.h"

class dtNavMeshQuery;
class dtQueryFilter;
struct dtQuerySpecialLinkFilter;

static const unsigned int DT_PATHQ_INVALID = 0;

typedef unsigned int dtPathQueueRef;

class dtPathQueue
{
	struct PathQuery
	{
		dtPathQueueRef ref;
		/// Path find start and end location.
		dtReal startPos[3], endPos[3];
		dtPolyRef startRef, endRef;
		dtReal costLimit;
		unsigned char requireNavigableEndLocation : 1;	// @UE
		/// Result.
		dtPolyRef* path;
		const dtQueryFilter* filter;
		TSharedPtr<dtQuerySpecialLinkFilter> linkFilter;
		int npath;
		/// State.
		dtStatus status;
		int keepAlive;
	};
	
	static const int MAX_QUEUE = 8;
	PathQuery m_queue[MAX_QUEUE];
	dtNavMeshQuery* m_navquery;
	dtPathQueueRef m_nextHandle;
	int m_maxPathSize;
	int m_queueHead;
	
	void purge();
	
public:
	dtPathQueue();
	~dtPathQueue();
	
	bool init(const int maxPathSize, const int maxSearchNodeCount, dtNavMesh* nav);
	
	void update(const int maxIters);
	
	dtPathQueueRef request(dtPolyRef startRef, dtPolyRef endRef,
						   const dtReal* startPos, const dtReal* endPos, const dtReal costLimit, const bool requireNavigableEndLocation, //@UE
						   const dtQueryFilter* filter,
						   TSharedPtr<dtQuerySpecialLinkFilter> linkFilter);
	
	dtStatus getRequestStatus(dtPathQueueRef ref) const;
	
	dtStatus getPathResult(dtPathQueueRef ref, dtPolyRef* path, int* pathSize, const int maxPath);
	
	inline const dtNavMeshQuery* getNavQuery() const { return m_navquery; }

};

#endif // DETOURPATHQUEUE_H
