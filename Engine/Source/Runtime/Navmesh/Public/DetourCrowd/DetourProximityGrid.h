// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURPROXIMITYGRID_H
#define DETOURPROXIMITYGRID_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"

class dtProximityGrid
{
	dtReal m_cellSize;
	dtReal m_invCellSize;
	
	struct Item
	{
		unsigned short id;
		short x,y;
		unsigned short next;
	};
	Item* m_pool;
	int m_poolHead;
	int m_poolSize;
	
	unsigned short* m_buckets;
	int m_bucketsSize;
	
	int m_bounds[4];
	
public:
	dtProximityGrid();
	~dtProximityGrid();
	
	bool init(const int maxItems, const dtReal cellSize);
	
	void clear();
	
	void addItem(const unsigned short id,
				 const dtReal minx, const dtReal miny,
				 const dtReal maxx, const dtReal maxy);
	
	int queryItems(const dtReal minx, const dtReal miny,
				   const dtReal maxx, const dtReal maxy,
				   unsigned short* ids, const int maxIds) const;
	
	int getItemCountAt(const int x, const int y) const;
	
	inline const int* getBounds() const { return m_bounds; }
	inline const dtReal getCellSize() const { return m_cellSize; }
};

dtProximityGrid* dtAllocProximityGrid();
void dtFreeProximityGrid(dtProximityGrid* ptr);


#endif // DETOURPROXIMITYGRID_H

