// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourNavMesh.h"
#include "Detour/DetourNavMeshQuery.h"
#include "HAL/Platform.h"

struct dtSharedBoundaryEdge
{
	dtReal v0[3];
	dtReal v1[3];
	dtPolyRef p0;
	dtPolyRef p1;
};

struct dtSharedBoundaryData
{
	dtReal Center[3];
	dtReal Radius;
	dtReal AccessTime;
	dtQueryFilter* Filter;
	uint8 SingleAreaId;
	
	TArray<dtSharedBoundaryEdge> Edges;
	TSet<dtPolyRef> Polys;

	dtSharedBoundaryData() : Filter(nullptr) {}
};

class dtSharedBoundary
{
public:
	TSparseArray<dtSharedBoundaryData> Data;
	dtQueryFilter SingleAreaFilter;
	dtReal CurrentTime;
	dtReal NextClearTime;

	void Initialize();
	void Tick(dtReal DeltaTime);

	int32 FindData(dtReal* Center, dtReal Radius, dtPolyRef ReqPoly, dtQueryFilter* NavFilter) const;
	int32 FindData(dtReal* Center, dtReal Radius, dtPolyRef ReqPoly, uint8 SingleAreaId) const;

	int32 CacheData(dtReal* Center, dtReal Radius, dtPolyRef CenterPoly, dtNavMeshQuery* NavQuery, dtQueryFilter* NavFilter);
	int32 CacheData(dtReal* Center, dtReal Radius, dtPolyRef CenterPoly, dtNavMeshQuery* NavQuery, uint8 SingleAreaId);

	void FindEdges(dtSharedBoundaryData& Data, dtPolyRef CenterPoly, dtNavMeshQuery* NavQuery, dtQueryFilter* NavFilter);
	bool HasSample(int32 Idx) const;

private:

	bool IsValid(int32 Idx, dtNavMeshQuery* NavQuery, dtQueryFilter* NavFilter) const;
};
