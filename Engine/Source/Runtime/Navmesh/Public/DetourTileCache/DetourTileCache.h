// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURTILECACHE_H
#define DETOURTILECACHE_H

#include "CoreMinimal.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourStatus.h"

struct dtTileCacheAlloc;
struct dtTileCacheCompressor;

typedef unsigned int dtObstacleRef;

typedef unsigned int dtCompressedTileRef;

/// Flags for addTile
enum dtCompressedTileFlags
{
	DT_COMPRESSEDTILE_FREE_DATA = 0x01,					///< Navmesh owns the tile memory and should free it.
};

struct dtCompressedTile
{
	unsigned int salt;						///< Counter describing modifications to the tile.
	struct dtTileCacheLayerHeader* header;
	unsigned char* compressed;
	int compressedSize;
	unsigned char* data;
	int dataSize;
	unsigned int flags;
	dtCompressedTile* next;
};

enum ObstacleState
{
	DT_OBSTACLE_EMPTY,
	DT_OBSTACLE_PROCESSING,
	DT_OBSTACLE_PROCESSED,
	DT_OBSTACLE_REMOVING,
};

static const int DT_MAX_TOUCHED_TILES = 8;
struct dtTileCacheObstacle
{
	dtReal pos[3];
	dtReal radius, height;
	dtCompressedTileRef touched[DT_MAX_TOUCHED_TILES];
	dtCompressedTileRef pending[DT_MAX_TOUCHED_TILES];
	unsigned short salt;
	unsigned char state;
	unsigned char ntouched;
	unsigned char npending;
	dtTileCacheObstacle* next;
};

struct dtTileCacheParams
{
	dtReal orig[3];
	dtReal cs, ch;
	int width, height;
	dtReal walkableHeight;
	dtReal walkableRadius;
	dtReal walkableClimb;
	dtReal maxSimplificationError;
	dtReal simplificationElevationRatio; // UE
	int maxTiles;
	int maxObstacles;
//@UE BEGIN
	dtReal detailSampleDist;
	dtReal detailSampleMaxError;
	int minRegionArea;
	int mergeRegionArea;
	int regionChunkSize;
	int regionPartitioning;
	bool bMinimizeLinkPoolSize;
//@UE END
};

struct dtTileCacheMeshProcess
{
	virtual void markAreas(struct dtTileCacheLayer* layer, const dtReal* orig, const dtReal cs, const dtReal ch) = 0;
	
//@UE BEGIN Adding support for LWCoords.
#if !DT_LARGE_WORLD_COORDINATES_DISABLED
	// This function is deprecated use the version that uses dtReal
	virtual void markAreas(struct dtTileCacheLayer* layer, const float* orig, const float cs, const float ch) final {};
#endif // DT_LARGE_WORLD_COORDINATES_DISABLED
//@UE END Adding support for LWCoords.

	virtual void process(struct dtNavMeshCreateParams* params,
						 unsigned char* polyAreas, unsigned short* polyFlags) = 0;
};


class dtTileCache
{
public:
	NAVMESH_API dtTileCache();
	NAVMESH_API ~dtTileCache();
	
	struct dtTileCacheAlloc* getAlloc() { return m_talloc; }
	struct dtTileCacheCompressor* getCompressor() { return m_tcomp; }
	struct dtTileCacheMeshProcess* getProcessor() { return m_tmproc; }
	const dtTileCacheParams* getParams() const { return &m_params; }
	
	inline int getTileCount() const { return m_params.maxTiles; }
	inline const dtCompressedTile* getTile(const int i) const { return &m_tiles[i]; }
	
	inline int getObstacleCount() const { return m_params.maxObstacles; }
	inline const dtTileCacheObstacle* getObstacle(const int i) const { return &m_obstacles[i]; }
	
	NAVMESH_API const dtTileCacheObstacle* getObstacleByRef(dtObstacleRef ref);
	
	NAVMESH_API dtObstacleRef getObstacleRef(const dtTileCacheObstacle* obmin) const;
	
	NAVMESH_API dtStatus init(const dtTileCacheParams* params,
				  struct dtTileCacheAlloc* talloc,
				  struct dtTileCacheCompressor* tcomp,
				  struct dtTileCacheMeshProcess* tmproc);
	
	NAVMESH_API int getTilesAt(const int tx, const int ty, dtCompressedTileRef* tiles, const int maxTiles) const ;
	
	NAVMESH_API dtCompressedTile* getTileAt(const int tx, const int ty, const int tlayer);
	NAVMESH_API dtCompressedTileRef getTileRef(const dtCompressedTile* tile) const;
	NAVMESH_API const dtCompressedTile* getTileByRef(dtCompressedTileRef ref) const;
	
	NAVMESH_API dtStatus addTile(unsigned char* data, const int dataSize, unsigned char flags, dtCompressedTileRef* result);
	
	NAVMESH_API dtStatus removeTile(dtCompressedTileRef ref, unsigned char** data, int* dataSize);
	
	NAVMESH_API dtStatus addObstacle(const dtReal* pos, const dtReal radius, const dtReal height, dtObstacleRef* result);
	NAVMESH_API dtStatus removeObstacle(const dtObstacleRef ref);
	
	NAVMESH_API dtStatus queryTiles(const dtReal* bmin, const dtReal* bmax,
						dtCompressedTileRef* results, int* resultCount, const int maxResults) const;
	
	NAVMESH_API dtStatus update(const dtReal /*dt*/, class dtNavMesh* navmesh);
	
	NAVMESH_API dtStatus buildNavMeshTilesAt(const int tx, const int ty, class dtNavMesh* navmesh);
	
	NAVMESH_API dtStatus buildNavMeshTile(const dtCompressedTileRef ref, class dtNavMesh* navmesh);
	
	NAVMESH_API void calcTightTileBounds(const struct dtTileCacheLayerHeader* header, dtReal* bmin, dtReal* bmax) const;
	
	NAVMESH_API void getObstacleBounds(const struct dtTileCacheObstacle* ob, dtReal* bmin, dtReal* bmax) const;
	

	/// Encodes a tile id.
	inline dtCompressedTileRef encodeTileId(unsigned int salt, unsigned int it) const
	{
		return ((dtCompressedTileRef)salt << m_tileBits) | (dtCompressedTileRef)it;
	}
	
	/// Decodes a tile salt.
	inline unsigned int decodeTileIdSalt(dtCompressedTileRef ref) const
	{
		const dtCompressedTileRef saltMask = ((dtCompressedTileRef)1<<m_saltBits)-1;
		return (unsigned int)((ref >> m_tileBits) & saltMask);
	}
	
	/// Decodes a tile id.
	inline unsigned int decodeTileIdTile(dtCompressedTileRef ref) const
	{
		const dtCompressedTileRef tileMask = ((dtCompressedTileRef)1<<m_tileBits)-1;
		return (unsigned int)(ref & tileMask);
	}

	/// Encodes an obstacle id.
	inline dtObstacleRef encodeObstacleId(unsigned int salt, unsigned int it) const
	{
		return ((dtObstacleRef)salt << 16) | (dtObstacleRef)it;
	}
	
	/// Decodes an obstacle salt.
	inline unsigned int decodeObstacleIdSalt(dtObstacleRef ref) const
	{
		const dtObstacleRef saltMask = ((dtObstacleRef)1<<16)-1;
		return (unsigned int)((ref >> 16) & saltMask);
	}
	
	/// Decodes an obstacle id.
	inline unsigned int decodeObstacleIdObstacle(dtObstacleRef ref) const
	{
		const dtObstacleRef tileMask = ((dtObstacleRef)1<<16)-1;
		return (unsigned int)(ref & tileMask);
	}
	
	
private:
	
	enum ObstacleRequestAction
	{
		REQUEST_ADD,
		REQUEST_REMOVE,
	};
	
	struct ObstacleRequest
	{
		int action;
		dtObstacleRef ref;
	};
	
	int m_tileLutSize;						///< Tile hash lookup size (must be pot).
	int m_tileLutMask;						///< Tile hash lookup mask.
	
	dtCompressedTile** m_posLookup;			///< Tile hash lookup.
	dtCompressedTile* m_nextFreeTile;		///< Freelist of tiles.
	dtCompressedTile* m_tiles;				///< List of tiles.
	
	unsigned int m_saltBits;				///< Number of salt bits in the tile ID.
	unsigned int m_tileBits;				///< Number of tile bits in the tile ID.
	
	dtTileCacheParams m_params;
	
	dtTileCacheAlloc* m_talloc;
	dtTileCacheCompressor* m_tcomp;
	dtTileCacheMeshProcess* m_tmproc;
	
	dtTileCacheObstacle* m_obstacles;
	dtTileCacheObstacle* m_nextFreeObstacle;
	
	static const int MAX_REQUESTS = 64;
	ObstacleRequest m_reqs[MAX_REQUESTS];
	int m_nreqs;
	
	static const int MAX_UPDATE = 64;
	dtCompressedTileRef m_update[MAX_UPDATE];
	int m_nupdate;
	
};

NAVMESH_API dtTileCache* dtAllocTileCache();
NAVMESH_API void dtFreeTileCache(dtTileCache* tc);

#endif
