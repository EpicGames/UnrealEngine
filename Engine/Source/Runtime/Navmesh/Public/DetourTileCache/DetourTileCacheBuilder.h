// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef DETOURTILECACHEBUILDER_H
#define DETOURTILECACHEBUILDER_H

#include "CoreMinimal.h"
#include "Detour/DetourAlloc.h"
#include "Detour/DetourLargeWorldCoordinates.h"
#include "Detour/DetourStatus.h"
#include "HAL/PlatformString.h"

static const int DT_TILECACHE_MAGIC = 'D'<<24 | 'T'<<16 | 'L'<<8 | 'R'; ///< 'DTLR';
static const int DT_TILECACHE_VERSION = 1;

static const unsigned char DT_TILECACHE_NULL_AREA = 0;
static const unsigned char DT_TILECACHE_WALKABLE_AREA = 63;
static const unsigned short DT_TILECACHE_NULL_IDX = 0xffff;

struct dtTileCacheLayerHeader
{
	short version;							///< Data version
	int tx,ty,tlayer;
	unsigned short hmin, hmax;				///< Height min/max range (@todo: remove)
	unsigned short width, height;			///< Dimension of the layer.
	unsigned short minx, maxx, miny, maxy;	///< Usable sub-region.
	// These should be at the bottom, as they are less often used than the rest of the data.
	dtReal bmin[3], bmax[3];
};

struct dtTileCacheLayer
{
	dtTileCacheLayerHeader* header;
	unsigned short regCount;					///< Region count.
	unsigned short* heights;
	unsigned char* areas;
	unsigned char* cons;
	unsigned short* regs;
};

struct dtTileCacheContour
{
	unsigned short* verts;	///< x, y, z, portal direction.
	int nverts;
	unsigned short reg;
	unsigned char area;
};

struct dtTileCacheContourSet
{
	int nconts;
	dtTileCacheContour* conts;
};

struct dtTileCachePolyMesh
{
	int nvp;
	int nverts;				///< Number of vertices.
	int npolys;				///< Number of polygons.
	unsigned short* verts;	///< Vertices of the mesh, 3 elements per vertex.
	unsigned short* polys;	///< Polygons of the mesh, nvp*2 elements per polygon.
	unsigned short* flags;	///< Per polygon flags.
	unsigned char* areas;	///< Area ID of polygons.
	unsigned short* regs;	///< Region ID of polygon
};

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
struct dtTileCacheClusterSet
{
	int nclusters;				///< The number of clusters
	int nregs;					///< The number of regions
	int npolys;					///< The number of polys
	unsigned short* regMap;		///< Cluster Id for each region [size: #nregs]
	unsigned short* polyMap;	///< Cluster Id for each poly [size: #npolys]
};
#endif //WITH_NAVMESH_CLUSTER_LINKS

struct dtTileCachePolyMeshDetail
{
	int nmeshes;			///< The number of sub-meshes defined by #meshes.
	int nverts;				///< The number of vertices in #verts.
	int ntris;				///< The number of triangles in #tris.
	unsigned int* meshes;	///< The sub-mesh data. [Size: 4*#nmeshes] 
	dtReal* verts;			///< The mesh vertices. [Size: 3*#nverts] 
	unsigned char* tris;	///< The mesh triangles. [Size: 4*#ntris] 
};

struct dtTileCacheDistanceField
{
	unsigned short maxDist;	///< Max distance
	unsigned short* data;	///< distance for every cell in layer
};

class dtTileCacheLogContext
{
public:
	/// Logs a message.
	///  @param[in]		category	The category of the message.
	///  @param[in]		format		The message.
	NAVMESH_API void dtLog(const char* format, ...);

protected:

	/// Logs a message.
	///  @param[in]		category	The category of the message.
	///  @param[in]		msg			The formatted message.
	///  @param[in]		len			The length of the formatted message.
	virtual void doDtLog(const char* /*msg*/, const int /*len*/) {}
};

//@UE END

struct dtTileCacheAlloc
{
	virtual ~dtTileCacheAlloc() = default;

	virtual void reset()
	{
	}
	
	virtual void* alloc(const int size)
	{
		return dtAlloc(size, DT_ALLOC_TEMP);
	}
	
	virtual void free(void* ptr)
	{
		dtFree(ptr, DT_ALLOC_TEMP);
	}
};

struct dtTileCacheCompressor
{
	virtual int maxCompressedSize(const int bufferSize) = 0;
	virtual dtStatus compress(const unsigned char* buffer, const int bufferSize,
							  unsigned char* compressed, const int maxCompressedSize, int* compressedSize) = 0;
	virtual dtStatus decompress(const unsigned char* compressed, const int compressedSize,
								unsigned char* buffer, const int maxBufferSize, int* bufferSize) = 0;
};

//@UE BEGIN: moved from DetourTileCacheBuilder.cpp
template<class T> class dtFixedArray
{
	dtTileCacheAlloc* m_alloc;
	T* m_ptr;
	const int m_size;
	inline T* operator=(T* p);
	inline void operator=(dtFixedArray<T>& p);
	inline dtFixedArray();
public:
	inline dtFixedArray(dtTileCacheAlloc* a, const int s) : m_alloc(a), m_ptr((T*)a->alloc(sizeof(T)*s)), m_size(s) {}
	inline ~dtFixedArray() { if (m_alloc) m_alloc->free(m_ptr); }
	inline operator T*() { return m_ptr; }
	inline int size() const { return m_size; }
	inline void set(unsigned char v) { memset((void*)m_ptr, v, sizeof(T)*m_size); }
};

inline int getDirOffsetX(int dir)
{
	const int offset[4] = { -1, 0, 1, 0, };
	return offset[dir&0x03];
}

inline int getDirOffsetY(int dir)
{
	const int offset[4] = { 0, 1, 0, -1 };
	return offset[dir&0x03];
}
//@UE END

NAVMESH_API dtStatus dtBuildTileCacheLayer(dtTileCacheCompressor* comp,
							   dtTileCacheLayerHeader* header,
							   const unsigned short* heights,
							   const unsigned char* areas,
							   const unsigned char* cons,
							   unsigned char** outData, int* outDataSize);

NAVMESH_API void dtFreeTileCacheLayer(dtTileCacheAlloc* alloc, dtTileCacheLayer* layer);

NAVMESH_API dtStatus dtDecompressTileCacheLayer(dtTileCacheAlloc* alloc, dtTileCacheCompressor* comp,
									const unsigned char* compressed, const int compressedSize,
									dtTileCacheLayer** layerOut);

NAVMESH_API dtTileCacheContourSet* dtAllocTileCacheContourSet(dtTileCacheAlloc* alloc);
NAVMESH_API void dtFreeTileCacheContourSet(dtTileCacheAlloc* alloc, dtTileCacheContourSet* cset);

NAVMESH_API dtTileCachePolyMesh* dtAllocTileCachePolyMesh(dtTileCacheAlloc* alloc);
NAVMESH_API void dtFreeTileCachePolyMesh(dtTileCacheAlloc* alloc, dtTileCachePolyMesh* lmesh);

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
NAVMESH_API dtTileCacheClusterSet* dtAllocTileCacheClusterSet(dtTileCacheAlloc* alloc);
NAVMESH_API void dtFreeTileCacheClusterSet(dtTileCacheAlloc* alloc, dtTileCacheClusterSet* clusters);
#endif // WITH_NAVMESH_CLUSTER_LINKS

NAVMESH_API dtTileCachePolyMeshDetail* dtAllocTileCachePolyMeshDetail(dtTileCacheAlloc* alloc);
NAVMESH_API void dtFreeTileCachePolyMeshDetail(dtTileCacheAlloc* alloc, dtTileCachePolyMeshDetail* dmesh);

NAVMESH_API dtTileCacheDistanceField* dtAllocTileCacheDistanceField(dtTileCacheAlloc* alloc);
NAVMESH_API void dtFreeTileCacheDistanceField(dtTileCacheAlloc* alloc, dtTileCacheDistanceField* dfield);
//@UE END

NAVMESH_API dtStatus dtMarkCylinderArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* pos, const dtReal radius, const dtReal height, const unsigned char areaId);

//@UE BEGIN: more shapes
NAVMESH_API dtStatus dtMarkBoxArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* pos, const dtReal* extent, const unsigned char areaId);

NAVMESH_API dtStatus dtMarkConvexArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* verts, const int nverts, const dtReal hmin, const dtReal hmax, const unsigned char areaId);

NAVMESH_API dtStatus dtReplaceCylinderArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* pos, const dtReal radius, const dtReal height, const unsigned char areaId,
	const unsigned char filterAreaId);

NAVMESH_API dtStatus dtReplaceBoxArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* pos, const dtReal* extent, const unsigned char areaId, const unsigned char filterAreaId);

NAVMESH_API dtStatus dtReplaceConvexArea(dtTileCacheLayer& layer, const dtReal* orig, const dtReal cs, const dtReal ch,
	const dtReal* verts, const int nverts, const dtReal hmin, const dtReal hmax, const unsigned char areaId,
	const unsigned char filterAreaId);

NAVMESH_API dtStatus dtReplaceArea(dtTileCacheLayer& layer, const unsigned char areaId, const unsigned char filterAreaId);

//@UE END


//@UE BEGIN: renamed building regions to dtBuildTileCacheRegionsMonotone, added new region generation
NAVMESH_API dtStatus dtBuildTileCacheDistanceField(dtTileCacheAlloc* alloc, dtTileCacheLayer& layer, dtTileCacheDistanceField& dfield);

NAVMESH_API dtStatus dtBuildTileCacheRegions(dtTileCacheAlloc* alloc,
								const int minRegionArea, const int mergeRegionArea,
								dtTileCacheLayer& layer, dtTileCacheDistanceField dfield);

NAVMESH_API dtStatus dtBuildTileCacheRegionsMonotone(dtTileCacheAlloc* alloc,
								const int minRegionArea, const int mergeRegionArea,
								dtTileCacheLayer& layer);

NAVMESH_API dtStatus dtBuildTileCacheRegionsChunky(dtTileCacheAlloc* alloc,
								const int minRegionArea, const int mergeRegionArea,
								dtTileCacheLayer& layer, int regionChunkSize);
//@UE END

NAVMESH_API dtStatus dtBuildTileCacheContours(dtTileCacheAlloc* alloc,
								dtTileCacheLayer& layer,
								const int walkableClimb, const dtReal maxError, const dtReal simplificationElevationRatio, // UE
								const dtReal cs, const dtReal ch,
								dtTileCacheContourSet& lcset
								//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
								, dtTileCacheClusterSet& lclusters
#endif // WITH_NAVMESH_CLUSTER_LINKS
								, const bool skipContourSimplification = false
								//@UE END
								);

NAVMESH_API dtStatus dtBuildTileCachePolyMesh(dtTileCacheAlloc* alloc,
								dtTileCacheLogContext* ctx,
								dtTileCacheContourSet& lcset,
								dtTileCachePolyMesh& mesh,
								const int walkableClimb);

//@UE BEGIN
NAVMESH_API dtStatus dtBuildTileCachePolyMeshDetail(dtTileCacheAlloc* alloc,
								const dtReal cs, const dtReal ch,
								const dtReal sampleDist, const dtReal sampleMaxError,
								dtTileCacheLayer& layer,
								dtTileCachePolyMesh& lmesh,
								dtTileCachePolyMeshDetail& dmesh);

#if WITH_NAVMESH_CLUSTER_LINKS
NAVMESH_API dtStatus dtBuildTileCacheClusters(dtTileCacheAlloc* alloc,
								dtTileCacheClusterSet& lclusters,
								dtTileCachePolyMesh& lmesh);
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

/// Swaps the endianess of the compressed tile data's header (#dtTileCacheLayerHeader).
/// Tile layer data does not need endian swapping as it consist only of bytes.
/// UE: not anymore, there are short types as well now
///  @param[in,out]	data		The tile data array.
///  @param[in]		dataSize	The size of the data array.
NAVMESH_API bool dtTileCacheHeaderSwapEndian(unsigned char* data, const int dataSize);


#endif // DETOURTILECACHEBUILDER_H
