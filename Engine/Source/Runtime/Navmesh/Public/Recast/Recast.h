// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//
 
#ifndef RECAST_H
#define RECAST_H

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "RecastLargeWorldCoordinates.h"

//@UE BEGIN Adding support for LWCoords.
/// The value of PI used by Recast.
static const rcReal RC_PI = 3.14159265358979323846;

inline float rcSin(float x)
{
	return sinf(x);
}

inline double rcSin(double x)
{
	return sin(x);
}

inline float rcCos(float x)
{
	return cosf(x);
}

inline double rcCos(double x)
{
	return cos(x);
}

inline float rcFloor(float x)
{
	return floorf(x);
}

inline double rcFloor(double x)
{
	return floor(x);
}

inline float rcCeil(float x)
{
	return ceilf(x);
}

inline double rcCeil(double x)
{
	return ceil(x);
}

inline float rcAbs(float x)
{
	return fabsf(x);
}

inline double rcAbs(double x)
{
	return fabs(x);
}
//@UE END Adding support for LWCoords.

/// Recast log categories.
/// @see rcContext
enum rcLogCategory
{
	RC_LOG_PROGRESS = 1,	///< A progress log entry.
	RC_LOG_WARNING,			///< A warning log entry.
	RC_LOG_ERROR,			///< An error log entry.
};

/// Recast performance timer categories.
/// @see rcContext
enum rcTimerLabel
{
	/// The user defined total time of the build.
	RC_TIMER_TOTAL,
	/// A user defined build time.
	RC_TIMER_TEMP,
	/// The time to rasterize the triangles. (See: #rcRasterizeTriangle)
	RC_TIMER_RASTERIZE_TRIANGLES,
	/// The time to build the compact heightfield. (See: #rcBuildCompactHeightfield)
	RC_TIMER_BUILD_COMPACTHEIGHTFIELD,
	/// The total time to build the contours. (See: #rcBuildContours)
	RC_TIMER_BUILD_CONTOURS,
	/// The time to trace the boundaries of the contours. (See: #rcBuildContours)
	RC_TIMER_BUILD_CONTOURS_TRACE,
	/// The time to simplify the contours. (See: #rcBuildContours)
	RC_TIMER_BUILD_CONTOURS_SIMPLIFY,
	/// The time to link clusters from contours. (See: #rcBuildClusters)
	RC_TIMER_BUILD_CLUSTERS, 
	/// The time to filter ledge spans. (See: #rcFilterLedgeSpans)
	RC_TIMER_FILTER_BORDER,
	/// The time to filter low height spans. (See: #rcFilterWalkableLowHeightSpans)
	RC_TIMER_FILTER_WALKABLE,
	/// The time to apply the median filter. (See: #rcMedianFilterWalkableArea)
	RC_TIMER_MEDIAN_AREA,
	/// The time to filter low obstacles. (See: #rcFilterLowHangingWalkableObstacles)
	RC_TIMER_FILTER_LOW_OBSTACLES,
	/// The time to build the polygon mesh. (See: #rcBuildPolyMesh)
	RC_TIMER_BUILD_POLYMESH,
	/// The time to merge polygon meshes. (See: #rcMergePolyMeshes)
	RC_TIMER_MERGE_POLYMESH,
	/// The time to erode the walkable area. (See: #rcErodeWalkableArea)
	RC_TIMER_ERODE_AREA,
	/// The time to mark a box area. (See: #rcMarkBoxArea)
	RC_TIMER_MARK_BOX_AREA,
	/// The time to mark a cylinder area. (See: #rcMarkCylinderArea)
	RC_TIMER_MARK_CYLINDER_AREA,
	/// The time to mark a convex polygon area. (See: #rcMarkConvexPolyArea)
	RC_TIMER_MARK_CONVEXPOLY_AREA,
	/// The total time to build the distance field. (See: #rcBuildDistanceField)
	RC_TIMER_BUILD_DISTANCEFIELD,
	/// The time to build the distances of the distance field. (See: #rcBuildDistanceField)
	RC_TIMER_BUILD_DISTANCEFIELD_DIST,
	/// The time to blur the distance field. (See: #rcBuildDistanceField)
	RC_TIMER_BUILD_DISTANCEFIELD_BLUR,
	/// The total time to build the regions. (See: #rcBuildRegions, #rcBuildRegionsMonotone)
	RC_TIMER_BUILD_REGIONS,
	/// The total time to apply the watershed algorithm. (See: #rcBuildRegions)
	RC_TIMER_BUILD_REGIONS_WATERSHED,
	/// The time to expand regions while applying the watershed algorithm. (See: #rcBuildRegions)
	RC_TIMER_BUILD_REGIONS_EXPAND,
	/// The time to flood regions while applying the watershed algorithm. (See: #rcBuildRegions)
	RC_TIMER_BUILD_REGIONS_FLOOD,
	/// The time to filter out small regions. (See: #rcBuildRegions, #rcBuildRegionsMonotone)
	RC_TIMER_BUILD_REGIONS_FILTER,
	/// The time to build heightfield layers. (See: #rcBuildHeightfieldLayers)
	RC_TIMER_BUILD_LAYERS, 
	/// The time to build the polygon mesh detail. (See: #rcBuildPolyMeshDetail)
	RC_TIMER_BUILD_POLYMESHDETAIL,
	/// The time to merge polygon mesh details. (See: #rcMergePolyMeshDetails)
	RC_TIMER_MERGE_POLYMESHDETAIL,
	/// The maximum number of timers.  (Used for iterating timers.)
	RC_MAX_TIMERS
};

NAVMESH_API DECLARE_LOG_CATEGORY_EXTERN(LogRecast, Log, All);

/// Provides an interface for optional logging and performance tracking of the Recast 
/// build process.
/// @ingroup recast
class rcContext
{
public:

	/// Contructor.
	///  @param[in]		state	TRUE if the logging and performance timers should be enabled.  [Default: true]
	inline rcContext(bool state = true) : m_logEnabled(state), m_timerEnabled(state) {}
	virtual ~rcContext() = default;

	/// Enables or disables logging.
	///  @param[in]		state	TRUE if logging should be enabled.
	inline void enableLog(bool state) { m_logEnabled = state; }

	/// Clears all log entries.
	inline void resetLog() { if (m_logEnabled) doResetLog(); }

	/// Logs a message.
	///  @param[in]		category	The category of the message.
	///  @param[in]		format		The message.
	NAVMESH_API void log(const rcLogCategory category, const char* format, ...);

	/// Enables or disables the performance timers.
	///  @param[in]		state	TRUE if timers should be enabled.
	inline void enableTimer(bool state) { m_timerEnabled = state; }

	/// Clears all peformance timers. (Resets all to unused.)
	inline void resetTimers() { if (m_timerEnabled) doResetTimers(); }

	/// Starts the specified performance timer.
	///  @param	label	The category of timer.
	inline void startTimer(const rcTimerLabel label) { if (m_timerEnabled) doStartTimer(label); }

	/// Stops the specified performance timer.
	///  @param	label	The category of the timer.
	inline void stopTimer(const rcTimerLabel label) { if (m_timerEnabled) doStopTimer(label); }

	/// Returns the total accumulated time of the specified performance timer.
	///  @param	label	The category of the timer.
	///  @return The accumulated time of the timer, or -1 if timers are disabled or the timer has never been started.
	inline int getAccumulatedTime(const rcTimerLabel label) const { return m_timerEnabled ? doGetAccumulatedTime(label) : -1; }

protected:

	/// Clears all log entries.
	virtual void doResetLog() {}

	/// Logs a message.
	///  @param[in]		category	The category of the message.
	///  @param[in]		msg			The formatted message.
	///  @param[in]		len			The length of the formatted message.
	virtual void doLog(const rcLogCategory /*category*/, const char* /*msg*/, const int /*len*/) {}

	/// Clears all timers. (Resets all to unused.)
	virtual void doResetTimers() {}

	/// Starts the specified performance timer.
	///  @param[in]		label	The category of timer.
	virtual void doStartTimer(const rcTimerLabel /*label*/) {}

	/// Stops the specified performance timer.
	///  @param[in]		label	The category of the timer.
	virtual void doStopTimer(const rcTimerLabel /*label*/) {}

	/// Returns the total accumulated time of the specified performance timer.
	///  @param[in]		label	The category of the timer.
	///  @return The accumulated time of the timer, or -1 if timers are disabled or the timer has never been started.
	virtual int doGetAccumulatedTime(const rcTimerLabel /*label*/) const { return -1; }
	
	/// True if logging is enabled.
	bool m_logEnabled;

	/// True if the performance timers are enabled.
	bool m_timerEnabled;
};

/// Region partitioning methods
/// @see rcConfig
enum rcRegionPartitioning
{
	RC_REGION_MONOTONE,		///< monotone partitioning
	RC_REGION_WATERSHED,	///< watershed partitioning
	RC_REGION_CHUNKY,		///< monotone partitioning on small chunks
};

//@UE BEGIN
/// Specifies the size of borders around the heightfield. 
struct rcBorderSize
{
	int low;	///< Size of the border in the negative direction of the axis [Limit: >= 0] [Units: vx]
	int high;	///< Size of the border in the positive direction of the axis [Limit: >= 0] [Units: vx]
};
//@UE END

/// Specifies a configuration to use when performing Recast builds.
/// @ingroup recast
struct rcConfig
{
	/// The width of the field along the x-axis. [Limit: >= 0] [Units: vx]
	int width;

	/// The height of the field along the z-axis. [Limit: >= 0] [Units: vx]
	int height;
	
	/// The width/height size of tile's on the xz-plane. [Limit: >= 0] [Units: vx]
	int tileSize;
	
	/// The size of the non-navigable border around the heightfield.
	rcBorderSize borderSize;													//@UE														

	/// The xz-plane cell size to use for fields. [Limit: > 0] [Units: wu] 
	rcReal cs;

	/// The y-axis cell size to use for fields. [Limit: > 0] [Units: wu]
	rcReal ch;

	/// The minimum bounds of the field's AABB. [(x, y, z)] [Units: wu]
	rcReal bmin[3];

	/// The maximum bounds of the field's AABB. [(x, y, z)] [Units: wu]
	rcReal bmax[3];

	/// The maximum slope that is considered walkable. [Limits: 0 <= value < 90] [Units: Degrees] 
	rcReal walkableSlopeAngle;

	/// Minimum floor to 'ceiling' height that will still allow the floor area to 
	/// be considered walkable. [Limit: >= 3] [Units: vx] 
	int walkableHeight;
	
	/// Maximum ledge height that is considered to still be traversable. [Limit: >=0] [Units: vx] 
	int walkableClimb;
	
	/// The distance to erode/shrink the walkable area of the heightfield away from 
	/// obstructions.  [Limit: >=0] [Units: vx] 
	int walkableRadius;

	/// Maximum step height in relation to cs and walkableSlopeAngle [Limit: >=0] [Units: wu]
	rcReal maxStepFromWalkableSlope;
	
	/// The maximum allowed length for contour edges along the border of the mesh. [Limit: >=0] [Units: vx] 
	int maxEdgeLen;
	
	/// The maximum distance a simplified contour's border edges should deviate 
	/// the original raw contour. [Limit: >=0] [Units: wu]
	rcReal maxSimplificationError;

	/// When simplifying contours, how much is the vertical error taken into account when comparing with MaxSimplificationError. [Limit: >=0]
	/// Use 0 to deactivate (Recast behavior), use 1 as a typical value.
	rcReal simplificationElevationRatio;	// UE
	
	/// The minimum number of cells allowed to form isolated island areas. [Limit: >=0] [Units: vx] 
	int minRegionArea;
	
	/// Any regions with a span count smaller than this value will, if possible, 
	/// be merged with larger regions. [Limit: >=0] [Units: vx] 
	int mergeRegionArea;

	/// Size of region chunk [Units: vx]
	int regionChunkSize;

	/// Region partitioning method: creating poly mesh
	int regionPartitioning;

	/// The maximum number of vertices allowed for polygons generated during the 
	/// contour to polygon conversion process. [Limit: >= 3] 
	int maxVertsPerPoly;
	
	/// Sets the sampling distance to use when generating the detail mesh.
	/// (For height detail only.) [Limits: 0 or >= 0.9] [Units: wu] 
	rcReal detailSampleDist;
	
	/// The maximum distance the detail mesh surface should deviate from heightfield
	/// data. (For height detail only.) [Limit: >=0] [Units: wu] 
	rcReal detailSampleMaxError;
};

/// Defines the number of bits allocated to rcSpanData::smin and rcSpanData::smax.
/// Using 29 bits increases the size of rcSpanData to 8 bytes but it does not impact the size of rcSpan since padding was already present.
/// It also increases the size of rcSpanCache to 12 bytes.
/// Size of rcTempSpan also increases to 8 bytes.
static constexpr int RC_SPAN_HEIGHT_BITS = 29;	// UE

/// Defines the maximum value for rcSpanData::smin and rcSpanData::smax.
static const int RC_SPAN_MAX_HEIGHT = (1<<RC_SPAN_HEIGHT_BITS)-1;

/// The number of spans allocated per span spool.
/// @see rcSpanPool
static const int RC_SPANS_PER_POOL = 2048;

typedef unsigned int rcSpanUInt;

/// Represents data of span in a heightfield.
/// @see rcHeightfield
struct rcSpanData
{
	rcSpanUInt smin : RC_SPAN_HEIGHT_BITS;	///< The lower limit of the span. [Limit: < #smax]
	rcSpanUInt smax : RC_SPAN_HEIGHT_BITS;	///< The upper limit of the span. [Limit: <= #RC_SPAN_MAX_HEIGHT]
	unsigned int area : 6;					///< The area id assigned to the span.
};

struct rcSpanCache
{
	unsigned short x;
	unsigned short y;
	rcSpanData data;
};

/// Represents a span in a heightfield.
/// @see rcHeightfield
struct rcSpan
{
	rcSpanData data;				///< Span data.
	rcSpan* next;					///< The next span higher up in column.
};

/// A memory pool used for quick allocation of spans within a heightfield.
/// @see rcHeightfield
struct rcSpanPool
{
	rcSpanPool* next;					///< The next span pool.
	rcSpan items[RC_SPANS_PER_POOL];	///< Array of spans in the pool.
};


#define EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER 1

#if EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
struct rcRowExt
{
	int MinCol;
	int MaxCol;
};
struct rcEdgeHit
{
	unsigned char Hits[2];
};
struct rcTempSpan
{
	int sminmax[2];			///< The lower and upper limit of the span. [Limit: < #smax]
};
#endif // EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER

/// A dynamic heightfield representing obstructed space.
/// @ingroup recast
struct rcHeightfield
{
	int width;			///< The width of the heightfield. (Along the x-axis in cell units.)
	int height;			///< The height of the heightfield. (Along the z-axis in cell units.)
	rcReal bmin[3];  	///< The minimum bounds in world space. [(x, y, z)]
	rcReal bmax[3];		///< The maximum bounds in world space. [(x, y, z)]
	rcReal cs;			///< The size of each cell. (On the xz-plane.)
	rcReal ch;			///< The height of each cell. (The minimum increment along the y-axis.)
	rcSpan** spans;		///< Heightfield of spans (width*height).
	rcSpanPool* pools;	///< Linked list of span pools.
	rcSpan* freelist;	///< The next free span.

#if EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER
	rcEdgeHit* EdgeHits; ///< h + 1 bit flags that indicate what edges cross the z cell boundaries
	rcRowExt* RowExt;		///< h structs that give the current x range for this z row
	rcTempSpan* tempspans;		///< Heightfield of temp spans (width*height).
	rcSpanData* tempSpanColumns;	///< Heightfield of 1 span per cell for vertical column rasterization //UE
#endif // EPIC_ADDITION_USE_NEW_RECAST_RASTERIZER

};

/// Provides information on the content of a cell column in a compact heightfield. 
struct rcCompactCell
{
	unsigned int index : 24;	///< Index to the first span in the column.
	unsigned int count : 8;		///< Number of spans in the column.
};

/// Represents a span of unobstructed space within a compact heightfield.
struct rcCompactSpan
{
	rcSpanUInt y;				///< The lower extent of the span. (Measured from the heightfield's base.)
	unsigned int con;			///< Packed neighbor connection data.
	unsigned short reg;			///< The id of the region the span belongs to. (Or zero if not in a region.)
	unsigned char h;			///< The height of the span.  (Measured from #y.)
};

/// A compact, static heightfield representing unobstructed space.
/// @ingroup recast
struct rcCompactHeightfield
{
	int width;					///< The width of the heightfield. (Along the x-axis in cell units.)
	int height;					///< The height of the heightfield. (Along the z-axis in cell units.)
	int spanCount;				///< The number of spans in the heightfield.
	int walkableHeight;			///< The walkable height used during the build of the field.  (See: rcConfig::walkableHeight)
	int walkableClimb;			///< The walkable climb used during the build of the field. (See: rcConfig::walkableClimb)
	rcBorderSize borderSize;	///< The AABB border size used during the build of the field. (See: rcConfig::borderSize)		//@UE
	unsigned short maxDistance;	///< The maximum distance value of any span within the field. 
	unsigned short maxRegions;	///< The maximum region id of any span within the field. 
	rcReal bmin[3];				///< The minimum bounds in world space. [(x, y, z)]
	rcReal bmax[3];				///< The maximum bounds in world space. [(x, y, z)]
	rcReal cs;					///< The size of each cell. (On the xz-plane.)
	rcReal ch;					///< The height of each cell. (The minimum increment along the y-axis.)
	rcCompactCell* cells;		///< Array of cells. [Size: #width*#height]
	rcCompactSpan* spans;		///< Array of spans. [Size: #spanCount]
	unsigned short* dist;		///< Array containing border distance data. [Size: #spanCount]
	unsigned char* areas;		///< Array containing area id data. [Size: #spanCount]
};

/// Represents a heightfield layer within a layer set.
/// @see rcHeightfieldLayerSet
struct rcHeightfieldLayer
{
	rcReal bmin[3];				///< The minimum bounds in world space. [(x, y, z)]
	rcReal bmax[3];				///< The maximum bounds in world space. [(x, y, z)]
	rcReal cs;					///< The size of each cell. (On the xz-plane.)
	rcReal ch;					///< The height of each cell. (The minimum increment along the y-axis.)
	int width;					///< The width of the heightfield. (Along the x-axis in cell units.)
	int height;					///< The height of the heightfield. (Along the z-axis in cell units.)
	int minx;					///< The minimum x-bounds of usable data.
	int maxx;					///< The maximum x-bounds of usable data.
	int miny;					///< The minimum y-bounds of usable data. (Along the z-axis.)
	int maxy;					///< The maximum y-bounds of usable data. (Along the z-axis.)
	int hmin;					///< The minimum height bounds of usable data. (Along the y-axis.)	// @todo: remove
	int hmax;					///< The maximum height bounds of usable data. (Along the y-axis.)
	unsigned short* heights;	///< The heightfield. [Size: (width - borderSize*2) * (h - borderSize*2)]
	unsigned char* areas;		///< Area ids. [Size: Same as #heights]
	unsigned char* cons;		///< Packed neighbor connection information. [Size: Same as #heights]
};

/// Represents a set of heightfield layers.
/// @ingroup recast
/// @see rcAllocHeightfieldLayerSet, rcFreeHeightfieldLayerSet 
struct rcHeightfieldLayerSet
{
	rcHeightfieldLayer* layers;			///< The layers in the set. [Size: #nlayers]
	int nlayers;						///< The number of layers in the set.
};

/// Represents a simple, non-overlapping contour in field space.
struct rcContour
{
	int* verts;			///< Simplified contour vertex and connection data. [Size: 4 * #nverts]
	int nverts;			///< The number of vertices in the simplified contour. 
	int* rverts;		///< Raw contour vertex and connection data. [Size: 4 * #nrverts]
	int nrverts;		///< The number of vertices in the raw contour. 
	unsigned short reg;	///< The region id of the contour.
	unsigned char area;	///< The area id of the contour.
};

/// Represents a group of related contours.
/// @ingroup recast
struct rcContourSet
{
	rcContour* conts;			///< An array of the contours in the set. [Size: #nconts]
	int nconts;					///< The number of contours in the set.
	rcReal bmin[3];  			///< The minimum bounds in world space. [(x, y, z)]
	rcReal bmax[3];				///< The maximum bounds in world space. [(x, y, z)]
	rcReal cs;					///< The size of each cell. (On the xz-plane.)
	rcReal ch;					///< The height of each cell. (The minimum increment along the y-axis.)
	int width;					///< The width of the set. (Along the x-axis in cell units.) 
	int height;					///< The height of the set. (Along the z-axis in cell units.) 
	rcBorderSize borderSize;	///< The AABB border size used to generate the source data from which the contours were derived.		//@UE
};

// @UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
/// Represents group of clusters
/// @ingroup recast
struct rcClusterSet
{
	int nclusters;			///< The number of clusters
	rcReal* center;			///< Center points per clusters [Size: 3 * #nclusters]
	unsigned short* nlinks;	///< Number of links per cluster [Size: #nclusters]
	unsigned short* links;	///< Neighbor Ids per cluster [Size: sum of #nlinks]
};
#endif // WITH_NAVMESH_CLUSTER_LINKS
// @UE END 
 
/// Represents a polygon mesh suitable for use in building a navigation mesh. 
/// @ingroup recast
struct rcPolyMesh
{
	unsigned short* verts;	///< The mesh vertices. [Form: (x, y, z) * #nverts]
	unsigned short* polys;	///< Polygon and neighbor data. [Length: #maxpolys * 2 * #nvp]
	unsigned short* regs;	///< The region id assigned to each polygon. [Length: #maxpolys]
	unsigned short* flags;	///< The user defined flags for each polygon. [Length: #maxpolys]
	unsigned char* areas;	///< The area id assigned to each polygon. [Length: #maxpolys]
	int nverts;				///< The number of vertices.
	int npolys;				///< The number of polygons.
	int maxpolys;			///< The number of allocated polygons.
	int nvp;				///< The maximum number of vertices per polygon.
	rcReal bmin[3];			///< The minimum bounds in world space. [(x, y, z)]
	rcReal bmax[3];			///< The maximum bounds in world space. [(x, y, z)]
	rcReal cs;				///< The size of each cell. (On the xz-plane.)
	rcReal ch;				///< The height of each cell. (The minimum increment along the y-axis.)
	rcBorderSize borderSize;///< The AABB border size used to generate the source data from which the mesh was derived.		//@UE
};

/// Contains triangle meshes that represent detailed height data associated 
/// with the polygons in its associated polygon mesh object.
/// @ingroup recast
struct rcPolyMeshDetail
{
	unsigned int* meshes;	///< The sub-mesh data. [Size: 4*#nmeshes] 
	rcReal* verts;			///< The mesh vertices. [Size: 3*#nverts] 
	unsigned char* tris;	///< The mesh triangles. [Size: 4*#ntris] 
	int nmeshes;			///< The number of sub-meshes defined by #meshes.
	int nverts;				///< The number of vertices in #verts.
	int ntris;				///< The number of triangles in #tris.
};

/// @name Allocation Functions
/// Functions used to allocate and de-allocate Recast objects.
/// @see rcAllocSetCustom
/// @{

/// Allocates a heightfield object using the Recast allocator.
///  @return A heightfield that is ready for initialization, or null on failure.
///  @ingroup recast
///  @see rcCreateHeightfield, rcFreeHeightField
NAVMESH_API rcHeightfield* rcAllocHeightfield();

/// Frees the specified heightfield object using the Recast allocator.
///  @param[in]		hf	A heightfield allocated using #rcAllocHeightfield
///  @ingroup recast
///  @see rcAllocHeightfield
NAVMESH_API void rcFreeHeightField(rcHeightfield* hf);

/// Allocates a compact heightfield object using the Recast allocator.
///  @return A compact heightfield that is ready for initialization, or null on failure.
///  @ingroup recast
///  @see rcBuildCompactHeightfield, rcFreeCompactHeightfield
NAVMESH_API rcCompactHeightfield* rcAllocCompactHeightfield();

/// Frees the specified compact heightfield object using the Recast allocator.
///  @param[in]		chf		A compact heightfield allocated using #rcAllocCompactHeightfield
///  @ingroup recast
///  @see rcAllocCompactHeightfield
NAVMESH_API void rcFreeCompactHeightfield(rcCompactHeightfield* chf);

/// Allocates a heightfield layer set using the Recast allocator.
///  @return A heightfield layer set that is ready for initialization, or null on failure.
///  @ingroup recast
///  @see rcBuildHeightfieldLayers, rcFreeHeightfieldLayerSet
NAVMESH_API rcHeightfieldLayerSet* rcAllocHeightfieldLayerSet();

/// Frees the specified heightfield layer set using the Recast allocator.
///  @param[in]		lset	A heightfield layer set allocated using #rcAllocHeightfieldLayerSet
///  @ingroup recast
///  @see rcAllocHeightfieldLayerSet
NAVMESH_API void rcFreeHeightfieldLayerSet(rcHeightfieldLayerSet* lset);

/// Allocates a contour set object using the Recast allocator.
///  @return A contour set that is ready for initialization, or null on failure.
///  @ingroup recast
///  @see rcBuildContours, rcFreeContourSet
NAVMESH_API rcContourSet* rcAllocContourSet();

/// Frees the specified contour set using the Recast allocator.
///  @param[in]		cset	A contour set allocated using #rcAllocContourSet
///  @ingroup recast
///  @see rcAllocContourSet
NAVMESH_API void rcFreeContourSet(rcContourSet* cset);

// @UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
/// Allocates a cluster set object using the Recast allocator.
///  @return A cluster set that is ready for initialization, or null on failure.
///  @ingroup recast
///  @see rcBuildClusters, rcFreeClusterSet
NAVMESH_API rcClusterSet* rcAllocClusterSet();

/// Frees the specified cluster set using the Recast allocator.
///  @param[in]		clset	A cluster set allocated using #rcAllocClusterSet
///  @ingroup recast
///  @see rcAllocClusterSet
NAVMESH_API void rcFreeClusterSet(rcClusterSet* clset);
#endif // WITH_NAVMESH_CLUSTER_LINKS
// @UE END 

/// Allocates a polygon mesh object using the Recast allocator.
///  @return A polygon mesh that is ready for initialization, or null on failure.
///  @ingroup recast
///  @see rcBuildPolyMesh, rcFreePolyMesh
NAVMESH_API rcPolyMesh* rcAllocPolyMesh();

/// Frees the specified polygon mesh using the Recast allocator.
///  @param[in]		pmesh	A polygon mesh allocated using #rcAllocPolyMesh
///  @ingroup recast
///  @see rcAllocPolyMesh
NAVMESH_API void rcFreePolyMesh(rcPolyMesh* pmesh);

/// Allocates a detail mesh object using the Recast allocator.
///  @return A detail mesh that is ready for initialization, or null on failure.
///  @ingroup recast
///  @see rcBuildPolyMeshDetail, rcFreePolyMeshDetail
NAVMESH_API rcPolyMeshDetail* rcAllocPolyMeshDetail();

/// Frees the specified detail mesh using the Recast allocator.
///  @param[in]		dmesh	A detail mesh allocated using #rcAllocPolyMeshDetail
///  @ingroup recast
///  @see rcAllocPolyMeshDetail
NAVMESH_API void rcFreePolyMeshDetail(rcPolyMeshDetail* dmesh);

/// @}

/// Heighfield border flag.
/// If a heightfield region ID has this bit set, then the region is a border 
/// region and its spans are considered unwalkable.
/// (Used during the region and contour build process.)
/// @see rcCompactSpan::reg
static const unsigned short RC_BORDER_REG = 0x8000;

/// Border vertex flag.
/// If a region ID has this bit set, then the associated element lies on
/// a tile border. If a contour vertex's region ID has this bit set, the 
/// vertex will later be removed in order to match the segments and vertices 
/// at tile boundaries.
/// (Used during the build process.)
/// @see rcCompactSpan::reg, #rcContour::verts, #rcContour::rverts
static const int RC_BORDER_VERTEX = 0x10000;

/// Area border flag.
/// If a region ID has this bit set, then the associated element lies on
/// the border of an area.
/// (Used during the region and contour build process.)
/// @see rcCompactSpan::reg, #rcContour::verts, #rcContour::rverts
static const int RC_AREA_BORDER = 0x20000;

/// Contour build flags.
/// @see rcBuildContours
enum rcBuildContoursFlags
{
	RC_CONTOUR_TESS_WALL_EDGES = 0x01,	///< Tessellate solid (impassable) edges during contour simplification.
	RC_CONTOUR_TESS_AREA_EDGES = 0x02,	///< Tessellate edges between areas during contour simplification.
};

// UE
enum rcFilterLowAreaFlags
{
	RC_LOW_FILTER_SEED_SPANS = 0x01,	///< initial seeding on spans
	RC_LOW_FILTER_POST_PROCESS = 0x02,	///< additional filtering at the end
};

// UE
enum rcRasterizationFlags
{
	RC_PROJECT_TO_BOTTOM = 1 << 0,		///< Will create spans from the triangle surface to the bottom of the heightfield
	RC_RASTERIZE_AS_FILLED_CONVEX = 1 << 1,		///< Will rasterize all the triangles of a list into a single span for each (x,z) and then add all those spans in the heightfield //UE
};

// UE
enum rcNeighborSlopeFilterMode
{
	RC_SLOPE_FILTER_RECAST,							// Use walkableClimb value to filter
	RC_SLOPE_FILTER_NONE,							// Skip slope filtering
	RC_SLOPE_FILTER_USE_HEIGHT_FROM_WALKABLE_SLOPE	// Use maximum step height computed from walkableSlopeAngle
};

/// Applied to the region id field of contour vertices in order to extract the region id.
/// The region id field of a vertex may have several flags applied to it.  So the
/// fields value can't be used directly.
/// @see rcContour::verts, rcContour::rverts
static const int RC_CONTOUR_REG_MASK = 0xffff;

/// An value which indicates an invalid index within a mesh.
/// @note This does not necessarily indicate an error.
/// @see rcPolyMesh::polys
static const unsigned short RC_MESH_NULL_IDX = 0xffff;

/// Represents the null area.
/// When a data element is given this value it is considered to no longer be 
/// assigned to a usable area.  (E.g. It is unwalkable.)
static const unsigned char RC_NULL_AREA = 0;

/// The default area id used to indicate a walkable polygon. 
/// This is also the maximum allowed area id, and the only non-null area id 
/// recognized by some steps in the build process. 
static const unsigned char RC_WALKABLE_AREA = 63;

/// The value returned by #rcGetCon if the specified direction is not connected
/// to another span. (Has no neighbor.)
static const int RC_NOT_CONNECTED = 0xff;

/// @name General helper functions
/// @{

/// Swaps the values of the two parameters.
///  @param[in,out]	a	Value A
///  @param[in,out]	b	Value B
template<class T> inline void rcSwap(T& a, T& b) { T t = a; a = b; b = t; }

/// Returns the minimum of two values.
///  @param[in]		a	Value A
///  @param[in]		b	Value B
///  @return The minimum of the two values.
template<class T> inline T rcMin(T a, T b) { return a < b ? a : b; }
/// When used with a mixture of rcReal and other types (in practice floats and doubles mixed here) this overridden function will be preferred by the compiler.
inline rcReal rcMin(rcReal a, rcReal b) { return rcMin<rcReal>(a, b); }

/// Returns the maximum of two values.
///  @param[in]		a	Value A
///  @param[in]		b	Value B
///  @return The maximum of the two values.
template<class T> inline T rcMax(T a, T b) { return a > b ? a : b; }
/// When used with a mixture of rcReal and other types (in practice floats and doubles mixed here) this overridden function will be preferred by the compiler.
inline rcReal rcMax(rcReal a, rcReal b) { return rcMax<rcReal>(a, b); }

/// Returns the absolute value.
///  @param[in]		a	The value.
///  @return The absolute value of the specified value.
template<class T> inline T rcAbs(T a) { return a < 0 ? -a : a; }

/// Returns the square of the value.
///  @param[in]		a	The value.
///  @return The square of the value.
template<class T> inline T rcSqr(T a) { return a*a; }

/// Clamps the value to the specified range.
///  @param[in]		v	The value to clamp.
///  @param[in]		mn	The minimum permitted return value.
///  @param[in]		mx	The maximum permitted return value.
///  @return The value, clamped to the specified range.
template<class T> inline T rcClamp(T v, T mn, T mx) { return v < mn ? mn : (v > mx ? mx : v); }
/// When used with a mixture of rcReal and other types (in practice floats and doubles mixed here) this overridden function will be preferred by the compiler.
inline rcReal rcClamp(rcReal v, rcReal mn, rcReal mx) { return v < mn ? mn : (v > mx ? mx : v); }

/// Returns the square root of the value.
///  @param[in]		x	The value.
///  @return The square root of the vlaue.
rcReal rcSqrt(rcReal x);

/// @}
/// @name Vector helper functions.
/// @{

/// Derives the cross product of two vectors. (@p v1 x @p v2)
///  @param[out]	dest	The cross product. [(x, y, z)]
///  @param[in]		v1		A Vector [(x, y, z)]
///  @param[in]		v2		A vector [(x, y, z)]
inline void rcVcross(rcReal* dest, const rcReal* v1, const rcReal* v2)
{
	dest[0] = v1[1]*v2[2] - v1[2]*v2[1];
	dest[1] = v1[2]*v2[0] - v1[0]*v2[2];
	dest[2] = v1[0]*v2[1] - v1[1]*v2[0];
}

/// Derives the dot product of two vectors. (@p v1 . @p v2)
///  @param[in]		v1	A Vector [(x, y, z)]
///  @param[in]		v2	A vector [(x, y, z)]
/// @return The dot product.
inline rcReal rcVdot(const rcReal* v1, const rcReal* v2)
{
	return v1[0]*v2[0] + v1[1]*v2[1] + v1[2]*v2[2];
}

/// Performs a scaled vector addition. (@p v1 + (@p v2 * @p s))
///  @param[out]	dest	The result vector. [(x, y, z)]
///  @param[in]		v1		The base vector. [(x, y, z)]
///  @param[in]		v2		The vector to scale and add to @p v1. [(x, y, z)]
///  @param[in]		s		The amount to scale @p v2 by before adding to @p v1.
inline void rcVmad(rcReal* dest, const rcReal* v1, const rcReal* v2, const rcReal s)
{
	dest[0] = v1[0]+v2[0]*s;
	dest[1] = v1[1]+v2[1]*s;
	dest[2] = v1[2]+v2[2]*s;
}

/// Performs a vector addition. (@p v1 + @p v2)
///  @param[out]	dest	The result vector. [(x, y, z)]
///  @param[in]		v1		The base vector. [(x, y, z)]
///  @param[in]		v2		The vector to add to @p v1. [(x, y, z)]
inline void rcVadd(rcReal* dest, const rcReal* v1, const rcReal* v2)
{
	dest[0] = v1[0]+v2[0];
	dest[1] = v1[1]+v2[1];
	dest[2] = v1[2]+v2[2];
}

/// Performs a vector subtraction. (@p v1 - @p v2)
///  @param[out]	dest	The result vector. [(x, y, z)]
///  @param[in]		v1		The base vector. [(x, y, z)]
///  @param[in]		v2		The vector to subtract from @p v1. [(x, y, z)]
inline void rcVsub(rcReal* dest, const rcReal* v1, const rcReal* v2)
{
	dest[0] = v1[0]-v2[0];
	dest[1] = v1[1]-v2[1];
	dest[2] = v1[2]-v2[2];
}

/// Selects the minimum value of each element from the specified vectors.
///  @param[in,out]	mn	A vector.  (Will be updated with the result.) [(x, y, z)]
///  @param[in]		v	A vector. [(x, y, z)]
inline void rcVmin(rcReal* mn, const rcReal* v)
{
	mn[0] = rcMin(mn[0], v[0]);
	mn[1] = rcMin(mn[1], v[1]);
	mn[2] = rcMin(mn[2], v[2]);
}

/// Selects the maximum value of each element from the specified vectors.
///  @param[in,out]	mx	A vector.  (Will be updated with the result.) [(x, y, z)]
///  @param[in]		v	A vector. [(x, y, z)]
inline void rcVmax(rcReal* mx, const rcReal* v)
{
	mx[0] = rcMax(mx[0], v[0]);
	mx[1] = rcMax(mx[1], v[1]);
	mx[2] = rcMax(mx[2], v[2]);
}

/// Performs a vector copy.
///  @param[out]	dest	The result. [(x, y, z)]
///  @param[in]		v		The vector to copy. [(x, y, z)]
inline void rcVcopy(rcReal* dest, const rcReal* v)
{
	dest[0] = v[0];
	dest[1] = v[1];
	dest[2] = v[2];
}

/// Returns the distance between two points.
///  @param[in]		v1	A point. [(x, y, z)]
///  @param[in]		v2	A point. [(x, y, z)]
/// @return The distance between the two points.
inline rcReal rcVdist(const rcReal* v1, const rcReal* v2)
{
	rcReal dx = v2[0] - v1[0];
	rcReal dy = v2[1] - v1[1];
	rcReal dz = v2[2] - v1[2];
	return rcSqrt(dx*dx + dy*dy + dz*dz);
}

/// Returns the square of the distance between two points.
///  @param[in]		v1	A point. [(x, y, z)]
///  @param[in]		v2	A point. [(x, y, z)]
/// @return The square of the distance between the two points.
inline rcReal rcVdistSqr(const rcReal* v1, const rcReal* v2)
{
	rcReal dx = v2[0] - v1[0];
	rcReal dy = v2[1] - v1[1];
	rcReal dz = v2[2] - v1[2];
	return dx*dx + dy*dy + dz*dz;
}

/// Normalizes the vector.
///  @param[in,out]	v	The vector to normalize. [(x, y, z)]
inline void rcVnormalize(rcReal* v)
{
	rcReal d = 1.0f / rcSqrt(rcSqr(v[0]) + rcSqr(v[1]) + rcSqr(v[2]));
	v[0] *= d;
	v[1] *= d;
	v[2] *= d;
}

//@UE BEGIN
/// Calculates the normals of each triangles in an array
///  @ingroup recast
///  @param[in]		verts	An array of vertices. [(x, y, z) * @p nv]
///  @param[in]		nv		The number of vertices in the @p verts array.
///  @param[in]		tris	The triangle vertex indices. [(vertA, vertB, vertC) * @p nt]
///  @param[in]		nt		The number of triangles.
///  @param[out]	norms	The normal vector of each triangle. [(x, y, z) * @p nt]
NAVMESH_API void rcCalcTriNormals(const rcReal* verts, const int nv, const int* tris, const int nt, rcReal* norms);
//@UE END

/// @}
/// @name Heightfield Functions
/// @see rcHeightfield
/// @{

/// Calculates the bounding box of an array of vertices.
///  @ingroup recast
///  @param[in]		verts	An array of vertices. [(x, y, z) * @p nv]
///  @param[in]		nv		The number of vertices in the @p verts array.
///  @param[out]	bmin	The minimum bounds of the AABB. [(x, y, z)] [Units: wu]
///  @param[out]	bmax	The maximum bounds of the AABB. [(x, y, z)] [Units: wu]
NAVMESH_API void rcCalcBounds(const rcReal* verts, int nv, rcReal* bmin, rcReal* bmax);

/// Calculates the grid size based on the bounding box and grid cell size.
///  @ingroup recast
///  @param[in]		bmin	The minimum bounds of the AABB. [(x, y, z)] [Units: wu]
///  @param[in]		bmax	The maximum bounds of the AABB. [(x, y, z)] [Units: wu]
///  @param[in]		cs		The xz-plane cell size. [Limit: > 0] [Units: wu]
///  @param[out]	w		The width along the x-axis. [Limit: >= 0] [Units: vx]
///  @param[out]	h		The height along the z-axis. [Limit: >= 0] [Units: vx]
NAVMESH_API void rcCalcGridSize(const rcReal* bmin, const rcReal* bmax, rcReal cs, int* w, int* h);

/// Initializes a new heightfield.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in,out]	hf		The allocated heightfield to initialize.
///  @param[in]		width	The width of the field along the x-axis. [Limit: >= 0] [Units: vx]
///  @param[in]		height	The height of the field along the z-axis. [Limit: >= 0] [Units: vx]
///  @param[in]		bmin	The minimum bounds of the field's AABB. [(x, y, z)] [Units: wu]
///  @param[in]		bmax	The maximum bounds of the field's AABB. [(x, y, z)] [Units: wu]
///  @param[in]		cs		The xz-plane cell size to use for the field. [Limit: > 0] [Units: wu]
///  @param[in]		ch		The y-axis cell size to use for field. [Limit: > 0] [Units: wu]
///  @param[in]		bAllocateTempSpanColumns	Indicate if we need to allocate the heightfield's tempSpanColumns.
NAVMESH_API bool rcCreateHeightfield(rcContext* ctx, rcHeightfield& hf, int width, int height,
						 const rcReal* bmin, const rcReal* bmax,
						 rcReal cs, rcReal ch,
						 bool bAllocateTempSpanColumns = false); //UE

/// Resets all spans of heightfield.
///  @ingroup recast
///  @param[in,out]	hf		The heightfield to reset.
///  @param[in]		bmin	The minimum bounds of the field's AABB. [(x, y, z)] [Units: wu]
///  @param[in]		bmax	The maximum bounds of the field's AABB. [(x, y, z)] [Units: wu]
///  @param[in,out]	hf		The heightfield toreset.
NAVMESH_API void rcResetHeightfield(rcHeightfield& hf);

/// Sets the area id of all triangles with a slope below the specified value
/// to #RC_WALKABLE_AREA.
///  @ingroup recast
///  @param[in,out]	ctx					The build context to use during the operation.
///  @param[in]		walkableSlopeAngle	The maximum slope that is considered walkable.
///  									[Limits: 0 <= value < 90] [Units: Degrees]
///  @param[in]		verts				The vertices. [(x, y, z) * @p nv]
///  @param[in]		nv					The number of vertices.
///  @param[in]		tris				The triangle vertex indices. [(vertA, vertB, vertC) * @p nt]
///  @param[in]		nt					The number of triangles.
///  @param[out]	areas				The triangle area ids. [Length: >= @p nt]
NAVMESH_API void rcMarkWalkableTriangles(rcContext* ctx, const rcReal walkableSlopeAngle, const rcReal* verts, int nv,
							 const int* tris, int nt, unsigned char* areas); 

/// Sets the area id of all triangles with a slope below the specified value
/// to #RC_WALKABLE_AREA.
///  @ingroup recast
///  @param[in,out]	ctx					The build context to use during the operation.
///  @param[in]		walkableSlopeCos	The cosine of maximum slope that is considered walkable.
///  									[Limits: 0 <= value < 1]
///  @param[in]		verts				The vertices. [(x, y, z) * @p nv]
///  @param[in]		nv					The number of vertices.
///  @param[in]		tris				The triangle vertex indices. [(vertA, vertB, vertC) * @p nt]
///  @param[in]		nt					The number of triangles.
///  @param[out]	areas				The triangle area ids. [Length: >= @p nt]
NAVMESH_API void rcMarkWalkableTrianglesCos(rcContext* ctx, const rcReal walkableSlopeCos, const rcReal* verts, int nv,
							    const int* tris, int nt, unsigned char* areas);

//@UE BEGIN
/// Replaces all RC_WALKABLE_AREA triangles with a area id.
///  @ingroup recast
///  @param[in,out]	areas				The triangle area ids. [Length: >= @p nt]
///  @param[in]		nt					The number of triangles.
///  @param[in]		areaId				The area id to assign instead of RC_WALKABLE_AREA.
/// @par
/// Used to apply an area id to all walkable triangles. Only replaces RC_WALKABLE_AREA, leaving other area ids unchanged.
NAVMESH_API void rcApplyAreaToTriangles(unsigned char* areas, const int nt, const unsigned char areaId);
//@UE END

/// Sets the area id of all triangles with a slope greater than or equal to the specified value to #RC_NULL_AREA.
///  @ingroup recast
///  @param[in,out]	ctx					The build context to use during the operation.
///  @param[in]		walkableSlopeAngle	The maximum slope that is considered walkable.
///  									[Limits: 0 <= value < 90] [Units: Degrees]
///  @param[in]		verts				The vertices. [(x, y, z) * @p nv]
///  @param[in]		nv					The number of vertices.
///  @param[in]		tris				The triangle vertex indices. [(vertA, vertB, vertC) * @p nt]
///  @param[in]		nt					The number of triangles.
///  @param[out]	areas				The triangle area ids. [Length: >= @p nt]
NAVMESH_API void rcClearUnwalkableTriangles(rcContext* ctx, const rcReal walkableSlopeAngle, const rcReal* verts, int nv,
								const int* tris, int nt, unsigned char* areas); 

/// Adds a span to the specified heightfield.
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in,out]	hf				An initialized heightfield.
///  @param[in]		x				The width index where the span is to be added.
///  								[Limits: 0 <= value < rcHeightfield::width]
///  @param[in]		y				The height index where the span is to be added.
///  								[Limits: 0 <= value < rcHeightfield::height]
///  @param[in]		smin			The minimum height of the span. [Limit: < @p smax] [Units: vx]
///  @param[in]		smax			The maximum height of the span. [Limit: <= #RC_SPAN_MAX_HEIGHT] [Units: vx]
///  @param[in]		area			The area id of the span. [Limit: <= #RC_WALKABLE_AREA)
///  @param[in]		flagMergeThr	The merge theshold. [Limit: >= 0] [Units: vx]
NAVMESH_API void rcAddSpan(rcContext* ctx, rcHeightfield& hf, const int x, const int y,
			   const unsigned short smin, const unsigned short smax,
			   const unsigned char area, const int flagMergeThr);

NAVMESH_API void rcAddSpans(rcContext* ctx, rcHeightfield& hf, const int flagMergeThr,
			    const rcSpanCache* cachedSpans, const int nspans);

NAVMESH_API int rcCountSpans(rcContext* ctx, rcHeightfield& hf);
NAVMESH_API void rcCacheSpans(rcContext* ctx, rcHeightfield& hf, rcSpanCache* cachedSpans);

/// Rasterizes a triangle into the specified heightfield.
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in]		v0				Triangle vertex 0 [(x, y, z)]
///  @param[in]		v1				Triangle vertex 1 [(x, y, z)]
///  @param[in]		v2				Triangle vertex 2 [(x, y, z)]
///  @param[in]		area			The area id of the triangle. [Limit: <= #RC_WALKABLE_AREA]
///  @param[in,out]	solid			An initialized heightfield.
///  @param[in]		flagMergeThr	The distance where the walkable flag is favored over the non-walkable flag.
///  								[Limit: >= 0] [Units: vx]
///  @param[in]     rtzFlags		Flags to change the rasterization behavior			//UE
///  @param[in]     rtzMasks		Mask for the rasterization flags [Size: hf.w*hf.h]	//UE
NAVMESH_API void rcRasterizeTriangle(rcContext* ctx, const rcReal* v0, const rcReal* v1, const rcReal* v2,
						 const unsigned char area, rcHeightfield& solid,
						 const int flagMergeThr = 1, 
						 const int rasterizationFlags = 0, const int* rasterizationMasks = nullptr); //UE

/// Rasterizes an indexed triangle mesh into the specified heightfield.
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in]		verts			The vertices. [(x, y, z) * @p nv]
///  @param[in]		nv				The number of vertices.
///  @param[in]		tris			The triangle indices. [(vertA, vertB, vertC) * @p nt]
///  @param[in]		areas			The area id's of the triangles. [Limit: <= #RC_WALKABLE_AREA] [Size: @p nt]
///  @param[in]		nt				The number of triangles.
///  @param[in,out]	solid			An initialized heightfield.
///  @param[in]		flagMergeThr	The distance where the walkable flag is favored over the non-walkable flag. 
///  								[Limit: >= 0] [Units: vx]
///  @param[in]     rtzFlags		Flags to change the rasterization behavior			//UE
///  @param[in]     rtzMasks		Mask for the rasterization flags [Size: hf.w*hf.h]	//UE
///  @param[in]     vertsbmin		Min location of the bounding box of verts [(x, y, z)] //UE
///  @param[in]     vertsbmax		Max location of the bounding box of verts [(x, y, z)] //UE
NAVMESH_API void rcRasterizeTriangles(rcContext* ctx, const rcReal* verts, const int nv,
						  const int* tris, const unsigned char* areas, const int nt,
						  rcHeightfield& solid, const int flagMergeThr = 1, 
						  const int rasterizationFlags = 0, const int* rasterizationMasks = nullptr, //UE
						  const rcReal* vertsbmin = nullptr, const rcReal* vertsbmax = nullptr); //UE

/// Rasterizes an indexed triangle mesh into the specified heightfield.
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in]		verts			The vertices. [(x, y, z) * @p nv]
///  @param[in]		nv				The number of vertices.
///  @param[in]		tris			The triangle indices. [(vertA, vertB, vertC) * @p nt]
///  @param[in]		areas			The area id's of the triangles. [Limit: <= #RC_WALKABLE_AREA] [Size: @p nt]
///  @param[in]		nt				The number of triangles.
///  @param[in,out]	solid			An initialized heightfield.
///  @param[in]		flagMergeThr	The distance where the walkable flag is favored over the non-walkable flag. 
///  								[Limit: >= 0] [Units: vx]
///  @param[in]     rtzFlags		Flags to change the rasterization behavior			//UE
///  @param[in]     rtzMasks		Mask for the rasterization flags [Size: hf.w*hf.h]	//UE
///  @param[in]     vertsbmin		Min location of the bounding box of verts [(x, y, z)] //UE
///  @param[in]     vertsbmax		Max location of the bounding box of verts [(x, y, z)] //UE
NAVMESH_API void rcRasterizeTriangles(rcContext* ctx, const rcReal* verts, const int nv,
						  const unsigned short* tris, const unsigned char* areas, const int nt,
						  rcHeightfield& solid, const int flagMergeThr = 1, 
						  const int rasterizationFlags = 0, const int* rasterizationMasks = nullptr, //UE
						  const rcReal* vertsbmin = nullptr, const rcReal* vertsbmax = nullptr); //UE

/// Rasterizes triangles into the specified heightfield.
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in]		verts			The triangle vertices. [(ax, ay, az, bx, by, bz, cx, by, cx) * @p nt]
///  @param[in]		areas			The area id's of the triangles. [Limit: <= #RC_WALKABLE_AREA] [Size: @p nt]
///  @param[in]		nt				The number of triangles.
///  @param[in,out]	solid			An initialized heightfield.
///  @param[in]		flagMergeThr	The distance where the walkable flag is favored over the non-walkable flag. 
///  								[Limit: >= 0] [Units: vx]
///  @param[in]     rtzFlags		Flags to change the rasterization behavior			//UE
///  @param[in]     rtzMasks		Mask for the rasterization flags [Size: hf.w*hf.h]	//UE
///  @param[in]     vertsbmin		Min location of the bounding box of verts [(x, y, z)] //UE
///  @param[in]     vertsbmax		Max location of the bounding box of verts [(x, y, z)] //UE
NAVMESH_API void rcRasterizeTriangles(rcContext* ctx, const rcReal* verts, const unsigned char* areas, const int nt,
						  rcHeightfield& solid, const int flagMergeThr = 1,
						  const int rasterizationFlags = 0, const int* rasterizationMasks = nullptr, //UE
						  const rcReal* vertsbmin = nullptr, const rcReal* vertsbmax = nullptr); //UE

/// Marks non-walkable spans as walkable if their maximum is within @p walkableClimp of a walkable neighbor. 
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in]		walkableClimb	Maximum ledge height that is considered to still be traversable. 
///  								[Limit: >=0] [Units: vx]
///  @param[in,out]	solid			A fully built heightfield.  (All spans have been added.)
NAVMESH_API void rcFilterLowHangingWalkableObstacles(rcContext* ctx, const int walkableClimb, rcHeightfield& solid);

/// Marks spans that are ledges as not-walkable, by a number of y coords at a time.
///  @ingroup recast
///  @param[in,out]	ctx							The build context to use during the operation.
///  @param[in]		walkableHeight				Minimum floor to 'ceiling' height that will still allow the floor area to 
///  											be considered walkable. [Limit: >= 3] [Units: vx]
///  @param[in]		walkableClimb				Maximum ledge height that is considered to still be traversable. [Limit: >=0] [Units: vx]
///  @param[in]		neighborSlopeFilterMode		Change the way neighbors slope filtering is done. //UE
///  @param[in]		maxStepFromWalkableSlope	Maximum step height in relation to cs and the walkable angle. [Limit: >= 0] [Units: wu] //UE
///  @param[in]		ch							Cell height. [Limit: >= 0] [Units: wu] //UE
///  @param[in,out]	solid						A fully built heightfield.  (All spans have been added.)
///  @param[in]		yStart						y coord to start at
///  @param[in]		maxYProcess					Max y coords to process (yStart + maxYProcess can be more than solid.height and will be capped to solid.height)
NAVMESH_API void rcFilterLedgeSpans(rcContext* ctx, const int walkableHeight, const int walkableClimb,
					const rcNeighborSlopeFilterMode neighborSlopeFilterMode, const rcReal maxStepFromWalkableSlope, const rcReal ch, const int yStart, const int maxYProcess, rcHeightfield& solid); //UE

/// Marks spans that are ledges as not-walkable. 
///  @ingroup recast
///  @param[in,out]	ctx							The build context to use during the operation.
///  @param[in]		walkableHeight				Minimum floor to 'ceiling' height that will still allow the floor area to 
///  											be considered walkable. [Limit: >= 3] [Units: vx]
///  @param[in]		walkableClimb				Maximum ledge height that is considered to still be traversable. [Limit: >=0] [Units: vx] 
///  @param[in]		neighborSlopeFilterMode		Change the way neighbors slope filtering is done. //UE
///  @param[in]		maxStepFromWalkableSlope	Maximum step height in relation to cs and the walkable angle. [Limit: >= 0] [Units: wu] //UE
///  @param[in]		ch							Cell height. [Limit: >= 0] [Units: wu] //UE
///  @param[in,out]	solid						A fully built heightfield.  (All spans have been added.)
NAVMESH_API void rcFilterLedgeSpans(rcContext* ctx, const int walkableHeight, const int walkableClimb,
					const rcNeighborSlopeFilterMode neighborSlopeFilterMode, const rcReal maxStepFromWalkableSlope, const rcReal ch, rcHeightfield& solid); //UE

/// Marks walkable spans as not walkable if the clearance above the span is less than the specified height. 
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in]		walkableHeight	Minimum floor to 'ceiling' height that will still allow the floor area to 
///  								be considered walkable. [Limit: >= 3] [Units: vx]
///  @param[in,out]	solid			A fully built heightfield.  (All spans have been added.)
NAVMESH_API void rcFilterWalkableLowHeightSpans(rcContext* ctx, int walkableHeight, rcHeightfield& solid);
NAVMESH_API void rcFilterWalkableLowHeightSpansSequences(rcContext* ctx, int walkableHeight, rcHeightfield& solid);

/// Returns the number of spans contained in the specified heightfield.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		hf		An initialized heightfield.
///  @returns The number of spans in the heightfield.
NAVMESH_API int rcGetHeightFieldSpanCount(rcContext* ctx, rcHeightfield& hf);

/// @}
/// @name Compact Heightfield Functions
/// @see rcCompactHeightfield
/// @{

/// Builds a compact heightfield representing open space, from a heightfield representing solid space.
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in]		walkableHeight	Minimum floor to 'ceiling' height that will still allow the floor area 
///  								to be considered walkable. [Limit: >= 3] [Units: vx]
///  @param[in]		walkableClimb	Maximum ledge height that is considered to still be traversable. 
///  								[Limit: >=0] [Units: vx]
///  @param[in]		hf				The heightfield to be compacted.
///  @param[out]	chf				The resulting compact heightfield. (Must be pre-allocated.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildCompactHeightfield(rcContext* ctx, const int walkableHeight, const int walkableClimb,
							   rcHeightfield& hf, rcCompactHeightfield& chf);

/// Erodes the walkable area within the heightfield by the specified radius. 
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		radius	The radius of erosion. [Limits: 0 < value < 255] [Units: vx]
///  @param[in,out]	chf		The populated compact heightfield to erode.
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcErodeWalkableArea(rcContext* ctx, int radius, rcCompactHeightfield& chf);

/// Erodes the walkable area within the heightfield by the specified radius. 
/// Additionally, it will mark all spans that are too low (rcMarkLowAreas)
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		radius	The radius of erosion. [Limits: 0 < value < 255] [Units: vx]
///  @param[in]		height	Height threshold [Units: vx]
///  @param[in]		areaId	The area id to apply [Limit: <= @RC_WALKABLE_AREA]
///  @param[in]		filterFlags	See: rcFilterLowAreaFlags
///  @param[in,out]	chf		The populated compact heightfield to erode.
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcErodeWalkableAndLowAreas(rcContext* ctx, int radius, unsigned int height,
											unsigned char areaId, unsigned char filterFlags,
											rcCompactHeightfield& chf);

/// Applies a median filter to walkable area types (based on area id), removing noise.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in,out]	chf		A populated compact heightfield.
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcMedianFilterWalkableArea(rcContext* ctx, rcCompactHeightfield& chf);

/// Marks all spans that have insufficient free space above
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in,out]	chf		A populated compact heightfield.
///  @param[in]		height	Height threshold [Units: vx]
///  @param[in]		areaId	The area id to apply [Limit: <= @RC_WALKABLE_AREA]
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcMarkLowAreas(rcContext* ctx, unsigned int height, unsigned char areaId, rcCompactHeightfield& chf);

/// Applies an area id to all spans within the specified bounding box. (AABB) 
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		bmin	The minimum of the bounding box. [(x, y, z)]
///  @param[in]		bmax	The maximum of the bounding box. [(x, y, z)]
///  @param[in]		areaId	The area id to apply. [Limit: <= #RC_WALKABLE_AREA]
///  @param[in,out]	chf		A populated compact heightfield.
NAVMESH_API void rcMarkBoxArea(rcContext* ctx, const rcReal* bmin, const rcReal* bmax, unsigned char areaId,
				   rcCompactHeightfield& chf);

/// Applies the area id to the all spans within the specified convex polygon. 
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		verts	The vertices of the polygon [Fomr: (x, y, z) * @p nverts]
///  @param[in]		nverts	The number of vertices in the polygon.
///  @param[in]		hmin	The height of the base of the polygon.
///  @param[in]		hmax	The height of the top of the polygon.
///  @param[in]		areaId	The area id to apply. [Limit: <= #RC_WALKABLE_AREA]
///  @param[in,out]	chf		A populated compact heightfield.
NAVMESH_API void rcMarkConvexPolyArea(rcContext* ctx, const rcReal* verts, const int nverts,
						  const rcReal hmin, const rcReal hmax, unsigned char areaId,
						  rcCompactHeightfield& chf);

/// Helper function to offset voncex polygons for rcMarkConvexPolyArea.
///  @ingroup recast
///  @param[in]		verts		The vertices of the polygon [Form: (x, y, z) * @p nverts]
///  @param[in]		nverts		The number of vertices in the polygon.
///  @param[out]	outVerts	The offset vertices (should hold up to 2 * @p nverts) [Form: (x, y, z) * return value]
///  @param[in]		maxOutVerts	The max number of vertices that can be stored to @p outVerts.
///  @returns Number of vertices in the offset polygon or 0 if too few vertices in @p outVerts.
NAVMESH_API int rcOffsetPoly(const rcReal* verts, const int nverts, const rcReal offset,
				 rcReal* outVerts, const int maxOutVerts);

/// Applies the area id to all spans within the specified cylinder.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		pos		The center of the base of the cylinder. [Form: (x, y, z)] 
///  @param[in]		r		The radius of the cylinder.
///  @param[in]		h		The height of the cylinder.
///  @param[in]		areaId	The area id to apply. [Limit: <= #RC_WALKABLE_AREA]
///  @param[in,out]	chf	A populated compact heightfield.
NAVMESH_API void rcMarkCylinderArea(rcContext* ctx, const rcReal* pos,
						const rcReal r, const rcReal h, unsigned char areaId,
						rcCompactHeightfield& chf);

/// Replaces an area id in spans with matching filter area within the specified bounding box. (AABB) 
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		bmin	The minimum of the bounding box. [(x, y, z)]
///  @param[in]		bmax	The maximum of the bounding box. [(x, y, z)]
///  @param[in]		areaId	The area id to apply. [Limit: <= #RC_WALKABLE_AREA]
///  @param[in,out]	chf		A populated compact heightfield.
NAVMESH_API void rcReplaceBoxArea(rcContext* ctx, const rcReal* bmin, const rcReal* bmax,
	unsigned char areaId, unsigned char filterAreaId,
	rcCompactHeightfield& chf);

/// Replaces an area id in spans with matching filter area within the specified convex polygon. 
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		verts	The vertices of the polygon [Fomr: (x, y, z) * @p nverts]
///  @param[in]		nverts	The number of vertices in the polygon.
///  @param[in]		hmin	The height of the base of the polygon.
///  @param[in]		hmax	The height of the top of the polygon.
///  @param[in]		areaId	The area id to apply. [Limit: <= #RC_WALKABLE_AREA]
///  @param[in,out]	chf		A populated compact heightfield.
NAVMESH_API void rcReplaceConvexPolyArea(rcContext* ctx, const rcReal* verts, const int nverts,
	const rcReal hmin, const rcReal hmax, unsigned char areaId, unsigned char filterAreaId,
	rcCompactHeightfield& chf);

/// Replaces an area id in spans with matching filter area within the specified cylinder.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		pos		The center of the base of the cylinder. [Form: (x, y, z)] 
///  @param[in]		r		The radius of the cylinder.
///  @param[in]		h		The height of the cylinder.
///  @param[in]		areaId	The area id to apply. [Limit: <= #RC_WALKABLE_AREA]
///  @param[in,out]	chf	A populated compact heightfield.
NAVMESH_API void rcReplaceCylinderArea(rcContext* ctx, const rcReal* pos,
	const rcReal r, const rcReal h, unsigned char areaId, unsigned char filterAreaId,
	rcCompactHeightfield& chf);

/// Builds the distance field for the specified compact heightfield. 
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in,out]	chf		A populated compact heightfield.
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildDistanceField(rcContext* ctx, rcCompactHeightfield& chf);

/// Builds region data for the heightfield using watershed partitioning. 
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in,out]	chf				A populated compact heightfield.
///  @param[in]		borderSize		The size of the non-navigable border around the heightfield.
///  								[Limit: >=0] [Units: vx]
///  @param[in]		minRegionArea	The minimum number of cells allowed to form isolated island areas.
///  								[Limit: >=0] [Units: vx].
///  @param[in]		mergeRegionArea		Any regions with a span count smaller than this value will, if possible,
///  								be merged with larger regions. [Limit: >=0] [Units: vx] 
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildRegions(rcContext* ctx, rcCompactHeightfield& chf,
					const int borderSize, const int minRegionArea, const int mergeRegionArea);

/// Builds region data for the heightfield using simple monotone partitioning.
///  @ingroup recast 
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in,out]	chf				A populated compact heightfield.
///  @param[in]		borderSize		The size of the non-navigable border around the heightfield.
///  								[Limit: >=0] [Units: vx]
///  @param[in]		minRegionArea	The minimum number of cells allowed to form isolated island areas.
///  								[Limit: >=0] [Units: vx].
///  @param[in]		mergeRegionArea	Any regions with a span count smaller than this value will, if possible, 
///  								be merged with larger regions. [Limit: >=0] [Units: vx] 
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildRegionsMonotone(rcContext* ctx, rcCompactHeightfield& chf,
							const rcBorderSize borderSize, const int minRegionArea, const int mergeRegionArea);		//@UE

/// Builds region data for the heightfield using simple monotone partitioning.
///  @ingroup recast 
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in,out]	chf				A populated compact heightfield.
///  @param[in]		borderSize		The size of the non-navigable border around the heightfield.
///  								[Limit: >=0] [Units: vx]
///  @param[in]		minRegionArea	The minimum number of cells allowed to form isolated island areas.
///  								[Limit: >=0] [Units: vx].
///  @param[in]		mergeRegionArea	Any regions with a span count smaller than this value will, if possible, 
///  								be merged with larger regions. [Limit: >=0] [Units: vx] 
///	 @param[in]		chunkSize		Size of subregion [Units: vx]
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildRegionsChunky(rcContext* ctx, rcCompactHeightfield& chf,
						  const rcBorderSize borderSize, const int minRegionArea, const int mergeRegionArea,		//@UE
						  const int chunkSize);

/// Sets the neighbor connection data for the specified direction.
///  @param[in]		s		The span to update.
///  @param[in]		dir		The direction to set. [Limits: 0 <= value < 4]
///  @param[in]		i		The index of the neighbor span.
inline void rcSetCon(rcCompactSpan& s, int dir, int i)
{
	const unsigned int shift = (unsigned int)dir * 8;
	unsigned int con = s.con;
	s.con = (con & ~(0xff << shift)) | (((unsigned int)i & 0xff) << shift);
}

/// Gets neighbor connection data for the specified direction.
///  @param[in]		s		The span to check.
///  @param[in]		dir		The direction to check. [Limits: 0 <= value < 4]
///  @return The neighbor connection data for the specified direction,
///  	or #RC_NOT_CONNECTED if there is no connection.
inline int rcGetCon(const rcCompactSpan& s, int dir)
{
	const unsigned int shift = (unsigned int)dir * 8;
	return (s.con >> shift) & 0xff;
}

/// Gets the standard width (x-axis) offset for the specified direction.
///  @param[in]		dir		The direction. [Limits: 0 <= value < 4]
///  @return The width offset to apply to the current cell position to move
///  	in the direction.
inline int rcGetDirOffsetX(int dir)
{
	const int offset[4] = { -1, 0, 1, 0, };
	return offset[dir&0x03];
}

/// Gets the standard height (z-axis) offset for the specified direction.
///  @param[in]		dir		The direction. [Limits: 0 <= value < 4]
///  @return The height offset to apply to the current cell position to move
///  	in the direction.
inline int rcGetDirOffsetY(int dir)
{
	const int offset[4] = { 0, 1, 0, -1 };
	return offset[dir&0x03];
}

/// @}
/// @name Layer, Contour, Polymesh, and Detail Mesh Functions
/// @see rcHeightfieldLayer, rcContourSet, rcPolyMesh, rcPolyMeshDetail
/// @{

// @UE BEGIN: renamed building layers to rcBuildHeightfieldLayersMonotone, added flood fill based implementation 

/// Builds a layer set from the specified compact heightfield.
///  @ingroup recast
///  @param[in,out]	ctx			The build context to use during the operation.
///  @param[in]		chf			A fully built compact heightfield.
///  @param[in]		borderSize	The size of the non-navigable border around the heightfield. [Limit: >=0] 
///  							[Units: vx]
///  @param[in]		walkableHeight	Minimum floor to 'ceiling' height that will still allow the floor area 
///  							to be considered walkable. [Limit: >= 3] [Units: vx]
///  @param[out]	lset		The resulting layer set. (Must be pre-allocated.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildHeightfieldLayers(rcContext* ctx, rcCompactHeightfield& chf,
							  const rcBorderSize borderSize, const int walkableHeight,		//@UE
							  rcHeightfieldLayerSet& lset);

/// Builds a layer set from the specified compact heightfield.
///  @ingroup recast
///  @param[in,out]	ctx			The build context to use during the operation.
///  @param[in]		chf			A fully built compact heightfield.
///  @param[in]		borderSize	The size of the non-navigable border around the heightfield. [Limit: >=0] 
///  							[Units: vx]
///  @param[in]		walkableHeight	Minimum floor to 'ceiling' height that will still allow the floor area 
///  							to be considered walkable. [Limit: >= 3] [Units: vx]
///  @param[out]	lset		The resulting layer set. (Must be pre-allocated.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildHeightfieldLayersMonotone(rcContext* ctx, rcCompactHeightfield& chf,
									  const rcBorderSize borderSize, const int walkableHeight,		//@UE
									  rcHeightfieldLayerSet& lset);

/// Builds a layer set from the specified compact heightfield.
///  @ingroup recast
///  @param[in,out]	ctx			The build context to use during the operation.
///  @param[in]		chf			A fully built compact heightfield.
///  @param[in]		borderSize	The size of the non-navigable border around the heightfield. [Limit: >=0] 
///  							[Units: vx]
///  @param[in]		walkableHeight	Minimum floor to 'ceiling' height that will still allow the floor area 
///  							to be considered walkable. [Limit: >= 3] [Units: vx]
///  @param[in]		chunkSize	Size of chunk of subregion [Units: vx]
///  @param[out]	lset		The resulting layer set. (Must be pre-allocated.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildHeightfieldLayersChunky(rcContext* ctx, rcCompactHeightfield& chf,
									const rcBorderSize borderSize, const int walkableHeight,		//@UE
									const int chunkSize,
									rcHeightfieldLayerSet& lset);

// @UE END

/// Builds a contour set from the region outlines in the provided compact heightfield.
///  @ingroup recast
///  @param[in,out]	ctx			The build context to use during the operation.
///  @param[in]		chf			A fully built compact heightfield.
///  @param[in]		maxError	The maximum distance a simplfied contour's border edges should deviate 
///  							the original raw contour. [Limit: >=0] [Units: wu]
///  @param[in]		maxEdgeLen	The maximum allowed length for contour edges along the border of the mesh. 
///  							[Limit: >=0] [Units: vx]
///  @param[out]	cset		The resulting contour set. (Must be pre-allocated.)
///  @param[in]		buildFlags	The build flags. (See: #rcBuildContoursFlags)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildContours(rcContext* ctx, rcCompactHeightfield& chf,
					 const rcReal maxError, const int maxEdgeLen,
					 rcContourSet& cset, const int flags = RC_CONTOUR_TESS_WALL_EDGES);

//@UE BEGIN
#if WITH_NAVMESH_CLUSTER_LINKS
/// Builds a cluster set from contours
/// @ingroup recast
/// @param[in,out]	ctx			The build context to use during the operation
/// @param[in]		cset		Contour set
/// @param[out]		clusters	Resulting cluster set
NAVMESH_API bool rcBuildClusters(rcContext* ctx, rcContourSet& cset, rcClusterSet& clusters);
#endif // WITH_NAVMESH_CLUSTER_LINKS
//@UE END

/// Builds a polygon mesh from the provided contours.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		cset	A fully built contour set.
///  @param[in]		nvp		The maximum number of vertices allowed for polygons generated during the 
///  						contour to polygon conversion process. [Limit: >= 3] 
///  @param[out]	mesh	The resulting polygon mesh. (Must be re-allocated.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildPolyMesh(rcContext* ctx, rcContourSet& cset, const int nvp, rcPolyMesh& mesh);

/// Merges multiple polygon meshes into a single mesh.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		meshes	An array of polygon meshes to merge. [Size: @p nmeshes]
///  @param[in]		nmeshes	The number of polygon meshes in the meshes array.
///  @param[in]		mesh	The resulting polygon mesh. (Must be pre-allocated.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcMergePolyMeshes(rcContext* ctx, rcPolyMesh** meshes, const int nmeshes, rcPolyMesh& mesh);

/// Builds a detail mesh from the provided polygon mesh.
///  @ingroup recast
///  @param[in,out]	ctx				The build context to use during the operation.
///  @param[in]		mesh			A fully built polygon mesh.
///  @param[in]		chf				The compact heightfield used to build the polygon mesh.
///  @param[in]		sampleDist		Sets the distance to use when samping the heightfield. [Limit: >=0] [Units: wu]
///  @param[in]		sampleMaxError	The maximum distance the detail mesh surface should deviate from 
///  								heightfield data. [Limit: >=0] [Units: wu]
///  @param[out]	dmesh			The resulting detail mesh.  (Must be pre-allocated.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcBuildPolyMeshDetail(rcContext* ctx, const rcPolyMesh& mesh, const rcCompactHeightfield& chf,
						   const rcReal sampleDist, const rcReal sampleMaxError,
						   rcPolyMeshDetail& dmesh);

/// Copies the poly mesh data from src to dst.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		src		The source mesh to copy from.
///  @param[out]	dst		The resulting detail mesh. (Must be pre-allocated, must be empty mesh.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcCopyPolyMesh(rcContext* ctx, const rcPolyMesh& src, rcPolyMesh& dst);

/// Merges multiple detail meshes into a single detail mesh.
///  @ingroup recast
///  @param[in,out]	ctx		The build context to use during the operation.
///  @param[in]		meshes	An array of detail meshes to merge. [Size: @p nmeshes]
///  @param[in]		nmeshes	The number of detail meshes in the meshes array.
///  @param[out]	mesh	The resulting detail mesh. (Must be pre-allocated.)
///  @returns True if the operation completed successfully.
NAVMESH_API bool rcMergePolyMeshDetails(rcContext* ctx, rcPolyMeshDetail** meshes, const int nmeshes, rcPolyMeshDetail& mesh);

/// @}

#endif // RECAST_H

///////////////////////////////////////////////////////////////////////////

// Due to the large amount of detail documentation for this file, 
// the content normally located at the end of the header file has been separated
// out to a file in /Docs/Extern.
