// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef RECASTALLOC_H
#define RECASTALLOC_H

#include "CoreMinimal.h"

/// Provides hint values to the memory allocator on how long the
/// memory is expected to be used.
enum rcAllocHint
{
	RC_ALLOC_PERM,		///< Memory will persist after a function call.
	RC_ALLOC_TEMP		///< Memory used temporarily within a function.
};

/// A memory allocation function.
//  @param[in]		size			The size, in bytes of memory, to allocate.
//  @param[in]		rcAllocHint	A hint to the allocator on how long the memory is expected to be in use.
//  @return A pointer to the beginning of the allocated memory block, or null if the allocation failed.
///  @see rcAllocSetCustom
typedef void* (rcAllocFunc)(int size, rcAllocHint hint);

/// A memory deallocation function.
///  @param[in]		ptr		A pointer to a memory block previously allocated using #rcAllocFunc.
/// @see rcAllocSetCustom
typedef void (rcFreeFunc)(void* ptr);

/// Sets the base custom allocation functions to be used by Recast.
///  @param[in]		allocFunc	The memory allocation function to be used by #rcAlloc
///  @param[in]		freeFunc	The memory de-allocation function to be used by #rcFree
NAVMESH_API void rcAllocSetCustom(rcAllocFunc *allocFunc, rcFreeFunc *freeFunc);

/// Allocates a memory block.
///  @param[in]		size	The size, in bytes of memory, to allocate.
///  @param[in]		hint	A hint to the allocator on how long the memory is expected to be in use.
///  @return A pointer to the beginning of the allocated memory block, or null if the allocation failed.
/// @see rcFree
void* rcAlloc(int size, rcAllocHint hint);

/// Deallocates a memory block.
///  @param[in]		ptr		A pointer to a memory block previously allocated using #rcAlloc.
/// @see rcAlloc
void rcFree(void* ptr);

void rcMemCpy(void* dst, void* src, int size);

/// A simple dynamic array of integers.
class rcIntArray
{
	int* m_data;
	int m_size, m_cap;
	inline rcIntArray(const rcIntArray&);
	inline rcIntArray& operator=(const rcIntArray&);
public:

	/// Constructs an instance with an initial array size of zero.
	inline rcIntArray() : m_data(0), m_size(0), m_cap(0) {}

	/// Constructs an instance initialized to the specified size.
	///  @param[in]		n	The initial size of the integer array.
	inline rcIntArray(int n) : m_data(0), m_size(0), m_cap(0) { resize(n); }
	inline ~rcIntArray() { rcFree(m_data); }

	/// Specifies the new size of the integer array.
	///  @param[in]		n	The new size of the integer array.
	void resize(int n);

	/// Push the specified integer onto the end of the array and increases the size by one.
	///  @param[in]		item	The new value.
	inline void push(int item) { resize(m_size+1); m_data[m_size-1] = item; }

	/// Returns the value at the end of the array and reduces the size by one.
	///  @return The value at the end of the array.
	inline int pop() { if (m_size > 0) m_size--; return m_data[m_size]; }

	/// The value at the specified array index.
	/// @warning Does not provide overflow protection.
	///  @param[in]		i	The index of the value.
	inline const int& operator[](int i) const { return m_data[i]; }

	/// The value at the specified array index.
	/// @warning Does not provide overflow protection.
	///  @param[in]		i	The index of the value.
	inline int& operator[](int i) { return m_data[i]; }

	/// The current size of the integer array.
	inline int size() const { return m_size; }

	// @UE BEGIN: added contains()
	bool contains(int n) const;
	// @UE END
};

/// A simple helper class used to delete an array when it goes out of scope.
/// @note This class is rarely if ever used by the end user.
template<class T> class rcScopedDelete
{
	T* ptr;
	int size;
	inline T* operator=(T* p);
public:

	/// Constructs an instance with a null pointer.
	inline rcScopedDelete() : ptr(0) {}
	inline rcScopedDelete(int n) { ptr = (T*)rcAlloc(sizeof(T)*n, RC_ALLOC_TEMP); size = n; }

	/// Constructs an instance with the specified pointer.
	///  @param[in]		p	An pointer to an allocated array.
	inline rcScopedDelete(T* p) : ptr(p) {}
	inline ~rcScopedDelete() { rcFree(ptr); ptr = 0; size = 0; }

	/// The root array pointer.
	///  @return The root array pointer.
	inline operator T*() { return ptr; }

	/// UE: resize and copy existing memory (n = element count), doesn't destruct elements!
	bool resizeGrow(int n);
};

template<class T>
bool rcScopedDelete<T>::resizeGrow(int n)
{
	if (n > size)
	{
		T* newData = (T*)rcAlloc(sizeof(T) * n, RC_ALLOC_TEMP);
		if (n && newData) rcMemCpy(newData, ptr, sizeof(T) * size);
		rcFree(ptr);
		ptr = newData;
		size = n;
	}

	return (ptr != 0);
}

/// A simple helper class used to delete an array of instances of structs,
///	that require cleaning up by calling destructor on every instance before release. 
/// Releasing is taking place when rcScopedStructArrayDelete instance goes out of scope.
/// @note if T doesn't require a destructor to be called use rcScopedDelete instead.
template<class T>
class rcScopedStructArrayDelete
{
	int itemCount;
	T* ptr;

	// on purpose made private to restrict copying
	inline T* operator=(T* p);
public:

	/// Constructs an array of instances of T
	inline rcScopedStructArrayDelete(const int n) : itemCount(n) { ptr = (T*)rcAlloc(sizeof(T)*n, RC_ALLOC_TEMP); }

	~rcScopedStructArrayDelete()
	{
		for (int itemIndex = 0; itemIndex < itemCount; ++itemIndex)
		{
			ptr[itemIndex].~T();
		}
		rcFree(ptr);
	}

	/// The root array pointer.
	///  @return The root array pointer.
	inline operator T*() { return ptr; }
};

#endif
