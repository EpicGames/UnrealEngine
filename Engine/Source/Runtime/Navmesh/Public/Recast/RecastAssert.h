// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

//
// Copyright (c) 2009-2010 Mikko Mononen memon@inside.org
//
// This software is provided 'as-is', without any express or implied
// warranty.  In no event will the authors be held liable for any damages
// arising from the use of this software.
// Permission is granted to anyone to use this software for any purpose,
// including commercial applications, and to alter it and redistribute it
// freely, subject to the following restrictions:
// 1. The origin of this software must not be misrepresented; you must not
//    claim that you wrote the original software. If you use this software
//    in a product, an acknowledgment in the product documentation would be
//    appreciated but is not required.
// 2. Altered source versions must be plainly marked as such, and must not be
//    misrepresented as being the original software.
// 3. This notice may not be removed or altered from any source distribution.
//

#ifndef RECASTASSERT_H
#define RECASTASSERT_H

#include "CoreMinimal.h"

// Note: This header file's only purpose is to include define assert.
// Feel free to change the file and include your own implementation instead.

#ifdef NDEBUG
// From http://cnicholson.net/2009/02/stupid-c-tricks-adventures-in-assert/
#	define rcAssert(x) do { (void)sizeof((x)); } while((void)(__LINE__==-1),false)  
#else
#	include <assert.h> 
#	define rcAssert(x) { assert(x); CA_ASSUME(x); } (void)0
#endif

#endif // RECASTASSERT_H
