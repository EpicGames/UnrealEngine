// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Recast/Detour's source file

#pragma once

#include <limits>

#define RC_LARGE_WORLD_COORDINATES_DISABLED 0

#if RC_LARGE_WORLD_COORDINATES_DISABLED

typedef float rcReal;

#else // RC_LARGE_WORLD_COORDINATES_DISABLED

typedef double rcReal;

#endif // RC_LARGE_WORLD_COORDINATES_DISABLED

constexpr rcReal RC_REAL_MAX = std::numeric_limits<rcReal>::max();

