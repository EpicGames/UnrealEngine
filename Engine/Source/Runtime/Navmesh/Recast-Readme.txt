
Recast & Detour Version 1.4


Recast

Recast is state of the art navigation mesh construction toolset for games.

    * It is automatic, which means that you can throw any level geometry
      at it and you will get robust mesh out
    * It is fast which means swift turnaround times for level designers
    * It is open source so it comes with full source and you can
      customize it to your hearts content. 

The Recast process starts with constructing a voxel mold from a level geometry 
and then casting a navigation mesh over it. The process consists of three steps, 
building the voxel mold, partitioning the mold into simple regions, peeling off 
the regions as simple polygons.

   1. The voxel mold is build from the input triangle mesh by rasterizing 
      the triangles into a multi-layer heightfield. Some simple filters are 
      then applied to the mold to prune out locations where the character 
      would not be able to move.
   2. The walkable areas described by the mold are divided into simple 
      overlayed 2D regions. The resulting regions have only one non-overlapping 
      contour, which simplifies the final step of the process tremendously.
   3. The navigation polygons are peeled off from the regions by first tracing 
      the boundaries and then simplifying them. The resulting polygons are 
      finally converted to convex polygons which makes them perfect for 
      pathfinding and spatial reasoning about the level. 

The toolset code is located in the Recast folder and demo application using the Recast
toolset is located in the RecastDemo folder.

The project files with this distribution can be compiled with Microsoft Visual C++ 2008
(you can download it for free) and XCode 3.1.


Detour

Recast is accompanied with Detour, path-finding and spatial reasoning toolkit. You can use any navigation mesh with Detour, but of course the data generated with Recast fits perfectly.

Detour offers simple static navigation mesh which is suitable for many simple cases, as well as tiled navigation mesh which allows you to plug in and out pieces of the mesh. The tiled mesh allows to create systems where you stream new navigation data in and out as the player progresses the level, or you may regenerate tiles as the world changes. 


Latest code available at http://code.google.com/p/recastnavigation/


--

Release Notes

----------------
* Recast 1.4
  Released August 24th, 2009

- Added detail height mesh generation (RecastDetailMesh.cpp) for single,
  tiled statmeshes as well as tilemesh.
- Added feature to contour tracing which detects extra vertices along
  tile edges which should be removed later.
- Changed the tiled stat mesh preprocess, so that it first generated
  polymeshes per tile and finally combines them.
- Fixed bug in the GUI code where invisible buttons could be pressed.

----------------
* Recast 1.31
  Released July 24th, 2009

- Better cost and heuristic functions.
- Fixed tile navmesh raycast on tile borders.

----------------
* Recast 1.3
  Released July 14th, 2009

- Added dtTileNavMesh which allows to dynamically add and remove navmesh pieces at runtime.
- Renamed stat navmesh types to dtStat* (i.e. dtPoly is now dtStatPoly).
- Moved common code used by tile and stat navmesh to DetourNode.h/cpp and DetourCommon.h/cpp.
- Refactores the demo code.

----------------
* Recast 1.2
  Released June 17th, 2009

- Added tiled mesh generation. The tiled generation allows to generate navigation for
  much larger worlds, it removes some of the artifacts that comes from distance fields
  in open areas, and allows later streaming and dynamic runtime generation
- Improved and added some debug draw modes
- API change: The helper function rcBuildNavMesh does not exists anymore,
  had to change few internal things to cope with the tiled processing,
  similar API functionality will be added later once the tiled process matures
- The demo is getting way too complicated, need to split demos
- Fixed several filtering functions so that the mesh is tighter to the geometry,
  sometimes there could be up error up to tow voxel units close to walls,
  now it should be just one.

----------------
* Recast 1.1
  Released April 11th, 2009

This is the first release of Detour.

----------------
* Recast 1.0
  Released March 29th, 2009

This is the first release of Recast.

The process is not always as robust as I would wish. The watershed phase sometimes swallows tiny islands
which are close to edges. These droppings are handled in rcBuildContours, but the code is not
particularly robust either.

Another non-robust case is when portal contours (contours shared between two regions) are always
assumed to be straight. That can lead to overlapping contours specially when the level has
large open areas.



Mikko Mononen
memon@inside.org
