s Solo Mesh Simple
f movement.obj
pf  -100.539185 -1.000000 54.028996  62.582016 15.757828 52.842243  0x3 0x0
pf  -100.539185 -1.000000 54.028996  -1.259964 -1.000000 50.116970  0x3 0x0
pf  -100.539185 -1.000000 54.028996  1.598934 -1.000000 23.528656  0x3 0x0
pf  -100.539185 -1.000000 54.028996  3.652847 -1.000000 -5.022881  0x3 0x0
pf  -100.539185 -1.000000 54.028996  -39.182816 8.999985 -24.697731  0x3 0x0
pf  -100.539185 -1.000000 54.028996  -66.847992 -1.000000 -28.908646  0x3 0x0
pf  -100.539185 -1.000000 54.028996  -90.966019 -1.000000 -3.219864  0x3 0x0
pf  -43.394421 -1.000000 13.312424  -90.966019 -1.000000 -3.219864  0x3 0x0
pf  -43.394421 -1.000000 13.312424  -36.447182 3.999992 -25.008087  0x3 0x0
pf  -43.394421 -1.000000 13.312424  26.394167 15.757812 -13.491264  0x3 0x0
pf  -43.394421 -1.000000 13.312424  -4.140746 6.944923 4.888435  0x3 0x0
pf  -43.394421 -1.000000 13.312424  -73.532791 -1.062469 23.137051  0x3 0x0
pf  -43.394421 -1.000000 13.312424  -72.902054 7.996834 15.076473  0x3 0x0