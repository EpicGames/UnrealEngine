s Solo Mesh Simple
f nav_test.obj
pf  18.138550 -2.370003 -21.319118  -19.206181 -2.369133 24.802742  0x3 0x0
pf  18.252758 -2.368240 -7.000238  -19.206181 -2.369133 24.802742  0x3 0x0
pf  18.252758 -2.368240 -7.000238  -22.759071 -2.369453 2.003946  0x3 0x0
pf  18.252758 -2.368240 -7.000238  -24.483898 -2.369728 -6.778278  0x3 0x0
pf  18.252758 -2.368240 -7.000238  -24.068850 -2.370285 -18.879251  0x3 0x0
pf  18.252758 -2.368240 -7.000238  12.124170 -2.369637 -21.222471  0x3 0x0
pf  10.830146 -2.366791 19.002508  12.124170 -2.369637 -21.222471  0x3 0x0
pf  10.830146 -2.366791 19.002508  -7.146484 -2.368736 -16.031403  0x3 0x0
pf  10.830146 -2.366791 19.002508  -21.615391 -2.368706 -3.264029  0x3 0x0
pf  10.830146 -2.366791 19.002508  -22.651268 -2.369354 1.053217  0x3 0x0
pf  10.830146 -2.366791 19.002508  19.181122 -2.368134 3.011776  0x3 0x0
pf  10.830146 -2.366791 19.002508  19.041592 -2.368713 -7.404587  0x3 0x0
pf  6.054083 -2.365402 3.330421  19.041592 -2.368713 -7.404587  0x3 0x0
pf  6.054083 -2.365402 3.330421  21.846087 -2.368568 17.918859  0x3 0x0
pf  6.054083 -2.365402 3.330421  0.967449 -2.368439 25.767756  0x3 0x0
pf  6.054083 -2.365402 3.330421  -17.518076 -2.368477 26.569633  0x3 0x0
pf  6.054083 -2.365402 3.330421  -22.141787 -2.369209 2.440046  0x3 0x0
pf  6.054083 -2.365402 3.330421  -23.296972 -2.369797 -17.411043  0x3 0x0
pf  6.054083 -2.365402 3.330421  -1.564062 -2.369926 -20.452827  0x3 0x0
pf  6.054083 -2.365402 3.330421  16.905643 -2.370193 -21.811655  0x3 0x0
pf  6.054083 -2.365402 3.330421  19.289761 -2.368813 -6.954918  0x3 0x0
